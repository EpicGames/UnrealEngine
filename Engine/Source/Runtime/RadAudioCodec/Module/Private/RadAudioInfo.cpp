// Copyright Epic Games, Inc. All Rights Reserved.

#include "RadAudioInfo.h"
#include "Interfaces/IAudioFormat.h"

#include "Modules/ModuleInterface.h"

#include "rada_file_header.h"
#include "rada_decode.h"

#if !defined(PLATFORM_LITTLE_ENDIAN) || !PLATFORM_LITTLE_ENDIAN
#error "RAD Audio hasn't been updated for big endian."
#endif

DEFINE_LOG_CATEGORY_STATIC(LogRadAudioDecoder, Log, All);
#define PTR_ADD(ptr,off) ((void*)(((uint8*)(ptr))+(off)))
#define Align32( val ) ( ( ( val ) + 31 ) & ~31 )

//-----------------------------------------------------------------------------
//
// All memory for the decoder is in one contiguous block:
//
// RadAudioDecoder structure
// RadAContainer (i.e. codec space)
// SeekTable
//
//-----------------------------------------------------------------------------
struct RadAudioDecoder
{
	uint32 SeekTableByteCount;

	// # of frames we need to eat before outputting data due to a sample
	// accurate seek.
	int32 ConsumeFrameCount;
	uint16 OutputReservoirValidFrames;
	uint16 OutputReservoirReadFrames;

	RadAContainer* Container()
	{
		return (RadAContainer*)(this + 1);
	}

};

FRadAudioInfo::FRadAudioInfo()
{
}

FRadAudioInfo::~FRadAudioInfo()
{
	
}

void FRadAudioInfo::PrepareToLoop()
{
#if WITH_RAD_AUDIO
	RadANotifySeek(Decoder->Container());
#endif
}

uint32 FRadAudioInfo::GetMaxFrameSizeSamples() const
{
	return RadADecodeBlock_MaxOutputFrames;
}

void FRadAudioInfo::SeekToTime(const float SeekTimeSeconds)
{	
	// If there's no seek table on the header, fall-back to Super implementation.
	const RadAFileHeader* FileHeader = RadAGetFileHeaderFromContainer(Decoder->Container());
	if (FileHeader->seek_table_entry_count == 0)
	{
		Super::SeekToTime(SeekTimeSeconds);
		RadANotifySeek(Decoder->Container());
		return;
	}

	// convert seconds to frames and call SeekToFrame
	uint32 SeekTimeFrames = 0;
	if (SeekTimeSeconds > 0)
	{
		SeekTimeFrames = (uint32)(SeekTimeSeconds * RadASampleRateFromEnum(FileHeader->sample_rate));
	}

	SeekToFrame(SeekTimeFrames);
}

void FRadAudioInfo::SeekToFrame(uint32 InSeekTimeFrames)
{
	// If there's no seek table on the header, fall-back to Super implementation.
	RadAContainer* Container = Decoder->Container();
	const RadAFileHeader* FileHeader = RadAGetFileHeaderFromContainer(Container);
	if (FileHeader->seek_table_entry_count == 0)
	{
		Super::SeekToFrame(InSeekTimeFrames);
		RadANotifySeek(Container);
		return;
	}

	if (InSeekTimeFrames >= FileHeader->frame_count)
	{
		InSeekTimeFrames = FileHeader->frame_count - 1;
	}
	
	// Since we opened the header and passed the overflow check, this is safe.
	this->CurrentSampleCount = InSeekTimeFrames * NumChannels;
	
	size_t FrameAtLocation = 0;
	size_t OffsetToBlockS = RadASeekTableLookup(
		Container,
		InSeekTimeFrames,
		&FrameAtLocation,
		nullptr);

	RadANotifySeek(Container);

	if (OffsetToBlockS > TNumericLimits<int32>::Max())
	{
		UE_LOGF(LogRadAudioDecoder, Error, "Seek block destination passed frame count caps but offset exceeds int32 limits (got: %zu", OffsetToBlockS);
		return;
	}
	int32 OffsetToBlock = (int32)OffsetToBlockS;

	// Block based codec - we start decoding on a block boundary and need
	// to eat frames to get to our actual spot.
	Decoder->ConsumeFrameCount = InSeekTimeFrames - (int32)FrameAtLocation;

	// After a seek we need to decode anew
	Decoder->OutputReservoirValidFrames = 0;
	Decoder->OutputReservoirReadFrames = 0;

	//
	// Here we need to set up the data we get to point at the right spot.
	//
	if (StreamingSoundWaveData == nullptr)
	{
		// If we aren't streaming we can just go directly to the offset we need.
		this->SrcBufferOffset = OffsetToBlock;
	}
	else
	{
		const uint32 TotalStreamingChunks = StreamingSoundWaveData->GetNumChunks();
		uint32 RemnOffset = OffsetToBlock;

		bool bFoundBlock = false;

		uint64 TotalChunkSizesExamined = 0;

		// Find the chunk and offset to the block we need.
		for (uint32 BlockIndex = 0; BlockIndex < TotalStreamingChunks; ++BlockIndex)
		{
			const uint32 SizeOfChunk = StreamingSoundWaveData->GetSizeOfChunk(BlockIndex);
			TotalChunkSizesExamined += SizeOfChunk;

			if (SizeOfChunk > RemnOffset)
			{
				// This is the block we need
				// If we are in the current block *and* the current block doesn't need to be loaded,
				// only then can we set the block offset directly. This is because AudioDecompress.cpp
				// sets SrcBufferOffset to zero when switching to the next block.
				if (this->CurrentChunkIndex != BlockIndex ||
					this->SrcBufferData == nullptr)
				{
					// Need to seek to another block
					this->StreamSeekBlockIndex = BlockIndex;
					this->StreamSeekBlockOffset = RemnOffset;
				}
				else
				{
					// Seek within this block
					this->SrcBufferOffset = RemnOffset;
				}
				bFoundBlock = true;
				break;
			}

			RemnOffset -= SizeOfChunk;
		}

		if (bFoundBlock == false)
		{
			UE_LOGF(LogRadAudioDecoder, Error, "Unable to find seek chunk for offset %u: ChunkCount: %d Total Chunk Size: %llu", OffsetToBlock, TotalStreamingChunks, TotalChunkSizesExamined);
		}
	}
}

//-----------------------------------------------------------------------------
//-----------------------------------------------------------------------------
bool FRadAudioInfo::ParseHeader(const uint8* InSrcBufferData, uint32 InSrcBufferDataSize, struct FSoundQualityInfo* QualityInfo)
{
	SrcBufferData = InSrcBufferData;
	SrcBufferDataSize = InSrcBufferDataSize;
	SrcBufferOffset = 0;
	CurrentSampleCount = 0;

	const RadAFileHeader* FileHeader = RadAGetFileHeader(InSrcBufferData, InSrcBufferDataSize);
	if (FileHeader == nullptr)
	{
		return false;
	}

	if (FPlatformMath::MultiplyAndCheckForOverflow<uint32>(FileHeader->frame_count, FileHeader->channels, TrueSampleCount) == false)
	{
		return false;
	}

	NumChannels = FileHeader->channels;

	// seek_table_entry_count is 16 bit, no overflow possible
	uint32 SeekTableSize = FileHeader->seek_table_entry_count * sizeof(uint16);
	if (sizeof(RadAFileHeader) + SeekTableSize > InSrcBufferDataSize)
	{
		return false;
	}

	// Store the offset to where the audio data begins.
	AudioDataOffset = RadAGetBytesToOpen(FileHeader);

	// Check we have all headers and seek table data in memory.
	if (AudioDataOffset > InSrcBufferDataSize)
	{
		return false;
	}

	// Write out the the header info
	if (QualityInfo)
	{
		QualityInfo->SampleRate = RadASampleRateFromEnum(FileHeader->sample_rate);
		QualityInfo->NumChannels = FileHeader->channels;
		QualityInfo->SampleDataSize = FileHeader->frame_count * QualityInfo->NumChannels * sizeof(int16);
		if (QualityInfo->SampleRate)
		{
			QualityInfo->Duration = (float)FileHeader->frame_count / QualityInfo->SampleRate;
		}
	}

	return true;
}

//-----------------------------------------------------------------------------
//-----------------------------------------------------------------------------
bool FRadAudioInfo::CreateDecoder()
{
	check(SrcBufferOffset == 0);

	const RadAFileHeader* FileHeader = RadAGetFileHeader(SrcBufferData, SrcBufferDataSize);
	if (FileHeader == nullptr)
	{
		return false;
	}

	uint32 DecoderMemoryNeeded = 0;
	if (RadAGetMemoryNeededToOpen(SrcBufferData, SrcBufferDataSize, &DecoderMemoryNeeded) != 0)
	{
		UE_LOGF(LogRadAudioDecoder, Error, "Invalid/insufficient data in FRadAudioInfo::CreateDecoder - bad buffer passed / bad cook? Size = %d", SrcBufferDataSize);
		if (SrcBufferDataSize > 8)
		{
			UE_LOGF(LogRadAudioDecoder, Error, "First 8 bytes: 0x%llx", *(uint64*)SrcBufferData);
		}
		return false; // we should have valid and sufficient data at this point.
	}

	uint32 TotalMemory = DecoderMemoryNeeded + sizeof(RadAudioDecoder);

	//
	// Allocate and save offsets
	//
	RawMemory.SetNumZeroed(TotalMemory);	
	Decoder = (RadAudioDecoder*)RawMemory.GetData();

	// See layout discussion in class declaration
	RadAContainer* Container = Decoder->Container();

	if (RadAOpenDecoder(SrcBufferData, SrcBufferDataSize, Container, DecoderMemoryNeeded) == 0)
	{
		UE_LOGF(LogRadAudioDecoder, Error, "Failed to open decoder, likely corrupted data.");
		RawMemory.Empty();
		return false;
	}

	// Set our buffer at the start of the audio data.
	SrcBufferOffset = RadAGetBytesToOpen(FileHeader);
	return true;
}

//-----------------------------------------------------------------------------
//-----------------------------------------------------------------------------
int32 FRadAudioInfo::GetFrameSize()
{
	uint32 BufferSizeNeeded = 0;
	RadAExamineBlockResult Result = RadAExamineBlock(Decoder->Container(), SrcBufferData + SrcBufferOffset, SrcBufferDataSize - SrcBufferOffset, &BufferSizeNeeded);
	
	if (Result != RadAExamineBlockResult::Valid ||
		BufferSizeNeeded > INT_MAX)
	{
		// Flag this as error so that the owning logic can clean up this decode.
		bErrorStateLatch = true;
		
		// Either malformed data, or not enough data. Since we break up the encoded file
		// one  block boundary, this should never happen and is considered a failure.
		return 0;
	}

	return (int32)BufferSizeNeeded;
}

//-----------------------------------------------------------------------------
//-----------------------------------------------------------------------------
FDecodeResult FRadAudioInfo::Decode(const uint8* CompressedData, const int32 CompressedDataSize, uint8* OutPCMData, const int32 OutputPCMDataSize)
{
	uint32 RemnOutputFrames = OutputPCMDataSize / SampleStride;
	uint32 RemnCompressedDataSize = CompressedDataSize;
	const uint8* CompressedDataEnd = CompressedData + CompressedDataSize;

	// \todo use a system-wide shared deinterleave buffer since it's virtually guaranteed we won't ever be
	// using them at the same time.
	TArray<float> DeinterleavedDecodeBuffer;

	// \todo When we are used as a streamed sound the calling code consumes the entirety of the output
	// reservoir in to its own reservoir, so we have duplicate reservoirs. UE-199172
	// This could be done by making the output reservoir use the system wide temp buffer like the deinterleave buffer
	// and just not request it if OutPCMData is big enough to go to directly. When streaming this will never happen,
	// under normal reading it will happen consistently. We don't particularly need the output reservoir to exist
	// next to us.
	while (RemnOutputFrames)
	{
		// Drain the output reservoir before attempting a decode.
		if (Decoder->OutputReservoirReadFrames < Decoder->OutputReservoirValidFrames)
		{
			uint32 AvailableFrames = Decoder->OutputReservoirValidFrames - Decoder->OutputReservoirReadFrames;
			uint32 CopyFrames = FMath::Min(AvailableFrames, RemnOutputFrames);

			uint32 CopyByteCount = SampleStride * CopyFrames;
			uint32 CopyOffset = SampleStride * Decoder->OutputReservoirReadFrames;
			FMemory::Memcpy(OutPCMData, OutputReservoir.GetData() + CopyOffset, CopyByteCount);

			Decoder->OutputReservoirReadFrames += CopyFrames;
			RemnOutputFrames -= CopyFrames;
			OutPCMData += CopyByteCount;

			if (RemnOutputFrames == 0)
			{
				// we filled entirely from the output reservoir
				break;
			}
		}

		if (RemnCompressedDataSize == 0)
		{
			// This is the normal termination condition
			break;
		}

		uint32 CompressedBytesNeeded = 0;
		RadAExamineBlockResult BlockResult = RadAExamineBlock(Decoder->Container(), CompressedData, RemnCompressedDataSize, &CompressedBytesNeeded);

		if (BlockResult != RadAExamineBlockResult::Valid)
		{
			// The splitting system should ensure that we only ever get complete blocks - so this is bizarre.
			UE_LOGF(LogRadAudioDecoder, Warning, "Invalid block in FRadAudioInfo::Decode: Result = %d, RemnSize = %d", EnumToUnderlyingType(BlockResult), RemnCompressedDataSize);
			if (RemnCompressedDataSize >= 8)
			{
				UE_LOGF(LogRadAudioDecoder, Warning, "First 8 bytes of buffer: 0x%02x 0x%02x 0x%02x 0x%02x:0x%02x 0x%02x 0x%02x 0x%02x", 
					CompressedData[0], CompressedData[1], CompressedData[2], CompressedData[3], CompressedData[4], CompressedData[5], CompressedData[6], CompressedData[7]);
			}
			break;
		}

		// RadAudio outputs deinterleaved 32 bit float buffers - we don't want to carry around
		// those buffers all the time and rad audio uses a pretty healthy amount of stack already
		// so we drop these in the temp buffers.
		if (DeinterleavedDecodeBuffer.Num() == 0)
		{
			DeinterleavedDecodeBuffer.AddUninitialized(RadADecodeBlock_MaxOutputFrames * NumChannels);
		}

		size_t CompressedDataConsumed = 0;
		int16 DecodeResult = RadADecodeBlock(Decoder->Container(), CompressedData, RemnCompressedDataSize, DeinterleavedDecodeBuffer.GetData(), RadADecodeBlock_MaxOutputFrames, &CompressedDataConsumed);
		if (DecodeResult == RadADecodeBlock_Error)
		{
			UE_LOGF(LogRadAudioDecoder, Error, "Failed to decode block that passed validation checks, corrupted buffer?");
			bErrorStateLatch = true;
			return FDecodeResult();
		}
		else if (DecodeResult == RadADecodeBlock_Done)
		{
			// There's no more data - return what we have.
			break;
		}

		CompressedData += CompressedDataConsumed;
		RemnCompressedDataSize -= CompressedDataConsumed;

		// Where to start reading the decoded results from, for trimming.
		int16 DecodeResultOffset = 0;

		// Check if we need to eat some frames due to a sample-accurate seek.
		if (Decoder->ConsumeFrameCount)
		{
			int16 ConsumedThisTime = DecodeResult;
			if (Decoder->ConsumeFrameCount < DecodeResult)
			{
				ConsumedThisTime = (int16)Decoder->ConsumeFrameCount;
			}

			if (ConsumedThisTime)
			{
				DecodeResultOffset = ConsumedThisTime;
				DecodeResult -= ConsumedThisTime;
			}
			Decoder->ConsumeFrameCount -= ConsumedThisTime;
		}

		// It's entirely valid to get 0 frames after a seek or otherwise - so we just try again. 
		if (DecodeResult == 0)
		{
			continue;
		}

		// If we can write directly to the output, then do so and avoid ever allocating an output reservoir if possible.
		int16* InterleaveDestination = (int16_t*)OutPCMData;
		if (RemnOutputFrames < RadADecodeBlock_MaxOutputFrames)
		{
			// we need to route through the output reservoir. Always fill to max capacity.
			if (OutputReservoir.Num() == 0)
			{
				OutputReservoir.AddUninitialized(RadADecodeBlock_MaxOutputFrames * SampleStride);
			}
			InterleaveDestination = (int16*)OutputReservoir.GetData();
		}

		// Interleave in to the output buffer, and convert to 16 bit.
		// The buffers are [0..DecodeResult...1024..1024+DecodeResult...n*1024..n*1024+DecodeResult]
		// We want: [0.1...n, DecodeResult...DecodeResult+n]
#ifdef RADA_HAS_INTERLEAVING
		const float* DeinterleavedSources[RADA_MAX_CHANNELS];
		for (int32 ChannelIdx = 0; ChannelIdx < NumChannels; ChannelIdx++)
		{
			DeinterleavedSources[ChannelIdx] = DeinterleavedDecodeBuffer.GetData() + RadADecodeBlock_MaxOutputFrames * ChannelIdx + DecodeResultOffset;
		}
		RadAInterleave(InterleaveDestination, DeinterleavedSources, NumChannels, DecodeResult);

#else
		for (int32 ChannelIdx = 0; ChannelIdx < NumChannels; ChannelIdx++)
		{
			const float* InBuffer = DeinterleavedDecodeBuffer.GetData() + RadADecodeBlock_MaxOutputFrames*ChannelIdx + DecodeResultOffset;
			
			for (int32 SampleIdx = 0; SampleIdx < DecodeResult; SampleIdx++)
			{
				float InBufferFloat = InBuffer[SampleIdx] * 32768.0f;
				if (InBufferFloat > 32767)
				{
					InBufferFloat = 32767;
				}
				else if (InBufferFloat < -32768)
				{
					InBufferFloat = -32768;
				}

				InterleaveDestination[ChannelIdx + (SampleIdx * NumChannels)] = (int16)InBufferFloat;
			}
		}
#endif

		if (InterleaveDestination == (int16*)OutputReservoir.GetData())
		{
			Decoder->OutputReservoirValidFrames = DecodeResult;
			Decoder->OutputReservoirReadFrames = 0;
			// Fall through to the next loop to copy the decoded pcm data out of the reservoir.
		}
		else
		{
			// we need to update the dest pointer since we went direct.
			RemnOutputFrames -= DecodeResult;
			OutPCMData += DecodeResult*SampleStride;
		}
		
	} // while need output pcm data

	// We get here if we filled the output buffer or not.
	FDecodeResult Result;
	Result.NumPcmBytesProduced = OutputPCMDataSize - (RemnOutputFrames * SampleStride);
	Result.NumAudioFramesProduced = Result.NumPcmBytesProduced / SampleStride;
	Result.NumCompressedBytesConsumed = CompressedDataSize - RemnCompressedDataSize;
	return Result;
}

bool FRadAudioInfo::HasError() const
{
	return bErrorStateLatch || Super::HasError();
}

class RADAUDIODECODER_API FRadAudioDecoderModule : public IModuleInterface
{
public:	
	TUniquePtr<IAudioInfoFactory> Factory;

	virtual void StartupModule() override
	{	
		Factory = MakeUnique<FSimpleAudioInfoFactory>([] { return new FRadAudioInfo(); }, Audio::NAME_RADA);
	}

	virtual void ShutdownModule() override {}
};

IMPLEMENT_MODULE(FRadAudioDecoderModule, RadAudioDecoder)
