// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AudioDecompress.h"

/**
*	Decoder thunk to the RAD Audio libraries. Also manages file parsing
*	for the cooked data.
*/
class  FRadAudioInfo : public IStreamedCompressedInfo
{
public:
	RADAUDIODECODER_API FRadAudioInfo();
	RADAUDIODECODER_API virtual ~FRadAudioInfo();

	//~ Begin IStreamedCompressedInfo Interface
	virtual bool ParseHeader(const uint8* InSrcBufferData, uint32 InSrcBufferDataSize, FSoundQualityInfo* QualityInfo) override;
	virtual int32 GetFrameSize() override;
	virtual uint32 GetMaxFrameSizeSamples() const override;
	virtual bool CreateDecoder() override;
	virtual void SeekToTime(const float SeekToTimeSeconds) override;
	virtual void SeekToFrame(const uint32 SeekTimeFrames) override;
	virtual void PrepareToLoop() override;
	virtual FDecodeResult Decode(const uint8* CompressedData, const int32 CompressedDataSize, uint8* OutPCMData, const int32 OutputPCMDataSize) override;
	virtual bool HasError() const override;
	//~ End IStreamedCompressedInfo Interface

protected:
	using Super = IStreamedCompressedInfo;

	// This is allocated on demand so that streaming sources can avoid allocating the memory.
	// These are all int16 but left as uint8 so we can use SampleStride.
	TArray<uint8> OutputReservoir;
	struct RadAudioDecoder* Decoder = 0;
	TArray<uint8> RawMemory;
	bool bErrorStateLatch = false;
};
