// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using System.Collections.Generic;
using System.Security.Cryptography;
using UnrealBuildTool;

public class RadAudioDecoder : ModuleRules
{
    // bIsWindowsLib means the library name just has .lib as the extension, otherwise we assume
    // it needs "lib" prefixed and ends with ".a"
    // Returns an array because some platforms (android) want to register the different architectures
    // and then filter them out later during the build process.
    // NDA platforms will override this.
    protected virtual string[] GetRadAudioPlatformString(ReadOnlyTargetRules Target, out bool bIsWindowsLib)
    {
        bIsWindowsLib = false; // most platforms are not windows style.
        if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows))
        {
            bIsWindowsLib = true;
            if (Target.Architecture == UnrealArch.Arm64)
            {
                return new string[] { "winarm64" };
            }
            if (Target.Architecture == UnrealArch.X64)
            {
                return new string[] { "win64" };
            }
            if (Target.Architecture == UnrealArch.Arm64ec)
            {
                return new string[] { "win64" };
            }
            System.Console.WriteLine("Unknown windows architecture ({0} in RadAudioDecoder.Build.cs", Target.Architecture);
            return null;
        }
        if (Target.Platform == UnrealTargetPlatform.Linux)
        {
            if (Target.Architecture == UnrealArch.Arm64)
            {
                return new string[] { "linuxarm64" };
            }
            if (Target.Architecture == UnrealArch.X64)
            {
                return new string[] { "linux64" };
            }
            
            System.Console.WriteLine("Unknown linux architecture ({0} in RadAudioDecoder.Build.cs", Target.Architecture);
            return null;
        }
        if (Target.Platform == UnrealTargetPlatform.LinuxArm64)
        {
            return new string[] { "linuxarm64" };
        }
        if (Target.Platform == UnrealTargetPlatform.Mac)
        {
            return new string[] { "osx" }; // arm64 and x64 are lipod together
        }
        if (Target.Platform == UnrealTargetPlatform.IOS)
        {
            // aside from sim, everything else is lipod.
            if (Target.Architecture == UnrealArch.IOSSimulator)
            {
                return new string[] { "iossim" };
            }
            return new string[] { "ios" };
        }
        if (Target.Platform == UnrealTargetPlatform.TVOS)
        {
            if (Target.Architecture == UnrealArch.TVOSSimulator)
            {
                return new string[] { "tvossim" };
            }
            return new string[] { "tvos" };
        }
        if (Target.Platform == UnrealTargetPlatform.VisionOS)
        {
            if (Target.Architecture == UnrealArch.IOSSimulator) // this is actually right for now :/
            {
                return new string[] { "visionossim" };
            }
            return new string[] { "visionos" };
        }
        if (Target.Platform == UnrealTargetPlatform.Android)
        {
            return new string[] { "androidx64", "androidarm64" };
        }

        return null;
    }

    public RadAudioDecoder(ReadOnlyTargetRules Target) : base(Target)
    {
        ShortName = "RadAudioDecoder";

        PublicDependencyModuleNames.AddRange(
            new string[]
            {
                "Core",
                "CoreUObject",
                "Engine"
            }
        );

        PublicSystemIncludePaths.Add(Path.Combine(ModuleDirectory, "..", "SDK", "Include"));

        // We need this on all the time otherwise we can't compile the includes.
        PublicDefinitions.Add("RADA_WRAP=UERA");

        bool bIsWindowsLib = false;
        string[] ArchitectureStrings = GetRadAudioPlatformString(Target, out bIsWindowsLib);

        if (ArchitectureStrings == null)
        {
            System.Console.WriteLine("Disabling RadAudio for platform {0} - does that sound right? Should be on for everything.", Target.Platform);
            PublicDefinitions.Add("WITH_RAD_AUDIO=0");
        }
        else
        {
            PublicDefinitions.Add("WITH_RAD_AUDIO=1");

            string BasePath = Path.Combine(GetModuleDirectoryForSubClass(GetType()).FullName, "..", "SDK", "Lib");
            foreach (string Architecture in ArchitectureStrings)
            {
                // radaudio_decoder is the codec.
                // rada_decode is the container.
                if (bIsWindowsLib)
                {
                    //System.Console.WriteLine("RadAudio: {0} {1}", 
                    //    Path.Combine(BasePath, "radaudio_decoder_" + Architecture + ".lib"),
                    //    Path.Combine(BasePath, "rada_decode_" + Architecture + ".lib"));

                    PublicAdditionalLibraries.Add(Path.Combine(BasePath, "radaudio_decoder_" + Architecture + ".lib"));
                }
                else
                {
                    PublicAdditionalLibraries.Add(Path.Combine(BasePath, "libradaudio_decoder_" + Architecture + ".a"));
                }
            }
        }
    }
}
