#!/bin/bash
# Copyright Epic Games, Inc. All Rights Reserved.

set -e

# We usually build this from the RAD source tree which has different folders - detect whether we
# are building in UE or RAD.
if [[ "$PWD" = */audio ]]; then
	UE_MODULE_LOCATION=`pwd`
	RADA_SOURCE_LOCATION="$UE_MODULE_LOCATION/../../miles/src/support/rada_encode/"
	RADAUDIO_SOURCE_LOCATION="$UE_MODULE_LOCATION/"

	BUILD_ROOT="$UE_MODULE_LOCATION/Intermediate"
	OUTPUT_LOCATION="$UE_MODULE_LOCATION/build"
	
	rm -rf $BUILD_ROOT
	mkdir $BUILD_ROOT
	
	[ -d $OUTPUT_LOCATION ] || mkdir $OUTPUT_LOCATION
	
	EXTRA_INCLUDES=(
		-I$UE_MODULE_LOCATION/../../shared/radrtl
		
	)
else
	UE_MODULE_LOCATION=`pwd`
	
	RADA_SOURCE_LOCATION="$UE_MODULE_LOCATION/Src/RadA"
	RADAUDIO_SOURCE_LOCATION="$UE_MODULE_LOCATION/Src/RadAudio"

	BUILD_ROOT="$UE_MODULE_LOCATION/Intermediate"
	OUTPUT_LOCATION="$UE_MODULE_LOCATION/Lib"

	EXTRA_INCLUDES=(
		-IInclude
	)
	
	p4 edit $OUTPUT_LOCATION/*osx.a
	p4 edit $OUTPUT_LOCATION/*ios{,sim}.a
	p4 edit $OUTPUT_LOCATION/*tvos.a
	p4 edit $OUTPUT_LOCATION/*visionos{,sim}.a

	rm -rf $BUILD_ROOT
	mkdir $BUILD_ROOT
fi

echo "It's normal to see empty symbol ranlib warnings"



VISIONOS_ARM64_ARGS=(
	-arch arm64
	--target=arm64-apple-xros1.0
)

VISIONOS_SIM_ARM64_ARGS=(
	-arch arm64
	--target=arm64-apple-xros1.0-simulator
)

TVOS_ARM64_ARGS=(
	-arch arm64
	--target=arm64-apple-tvos15
)

IOS_ARM64_ARGS=(
	-arch arm64
	--target=arm-apple-ios10
)

IOS_SIM_ARGS_X64=(
	-arch x86_64
	--target=x86_64-apple-ios10
	-mmmx
)

IOS_SIM_ARGS_ARM64=(
	-arch arm64
	-miphonesimulator-version-min=15
)

OSX_X64_ARGS=(
	-arch x86_64
	-msse
	-msse2
	-msse3
	-mssse3
	-mmacosx-version-min=10.14
)

OSX_ARM64_ARGS=(
	-arch arm64
	-mmacosx-version-min=10.14
)


AVX2_ARGS=(
	-mavx
	-mavx2
)

CPP_ARGS=(
	-std=c++14
	)

COMMON_ARGS=(
	-c
	-DNDEBUG
	-DUSING_EGT	# tell rrCore.h we are using egttypes.h
	-D__RADINSTATICLIB__
	-DRADAUDIO_WRAP=UERA # Prefix symbols so that mulitple libs using the same source dont get reused by the linker
	-DRADA_WRAP=UERA # Prefix symbols so that mulitple libs using the same source dont get reused by the linker
	-ffp-contract=off # Prevent FMA contraction for consistency between arm/x64
	-fno-exceptions
	-fno-omit-frame-pointer
	-fno-rtti
	-fno-strict-aliasing # prevent optimizations from introducing random silent bugs
	-fno-unroll-loops
	-fno-vectorize # Weve vectorized everything so this just makes the tail computation get unrolled unnecessarily
	-fvisibility=hidden
	-ggdb
	-mllvm
	-inline-threshold=64 # Pass inline threhold to llvm to prevent binary size explosion
	-momit-leaf-frame-pointer
	-O2
	
	-I$RADAUDIO_SOURCE_LOCATION
)

RADAUDIO_SOURCES=(
	radaudio_decoder.c
	radaudio_mdct.cpp
	radaudio_mdct_neon.cpp
	radaudio_mdct_sse2.cpp
	cpux86.cpp
	radaudio_decoder_sse2.c
	radaudio_decoder_sse4.c
	radaudio_decoder_neon.c
	radaudio_interleave.c
	rada_decode.cpp
)

RADAUDIO_SOURCES_AVX2=(
	radaudio_decoder_avx2.c
	radaudio_mdct_avx2.cpp
)

RADAUDIO_ENCODER_SOURCES=(
	radaudio_encoder.c
	radaudio_mdct.cpp
	radaudio_mdct_neon.cpp
	radaudio_mdct_sse2.cpp
	cpux86.cpp
	radaudio_encoder_sse.c
	radaudio_encoder_neon.c
	rada_encode.cpp
)


# We expect:
#	CLANG_ARGS
#	CLANG_SOURCES
#	BUILD_LOCATION
#	CPP_ARGS
#	COMMON_ARGS
#	APPLE_SDK
# We modify and update:
#	OUTPUT

function CallClang()
{
	local CLANG_PATH=$(xcrun --sdk $APPLE_SDK --find clang)
	local SYSROOT=(
		-isysroot
		$(xcrun --sdk $APPLE_SDK --show-sdk-path)
		)

	mkdir -p $BUILD_LOCATION
	for source_file in "${CLANG_SOURCES[@]}"
	do
	
		if [[ "$source_file" =~ "rada_" ]]; then
			SOURCE_PATH=$RADA_SOURCE_LOCATION
		else
			SOURCE_PATH=$RADAUDIO_SOURCE_LOCATION
		fi
		
		if [ "${source_file##*\.}" == "cpp" ]
		then
			$CLANG_PATH ${SYSROOT[@]} ${CPP_ARGS} ${CLANG_ARGS[@]} ${COMMON_ARGS[@]} ${EXTRA_INCLUDES[@]} -o $BUILD_LOCATION/${source_file%.*}.o $SOURCE_PATH/$source_file
		else
			$CLANG_PATH ${SYSROOT[@]} ${CLANG_ARGS[@]} ${COMMON_ARGS[@]} ${EXTRA_INCLUDES[@]} -o $BUILD_LOCATION/${source_file%.*}.o $SOURCE_PATH/$source_file
		fi

		OUTPUT+=( $BUILD_LOCATION/${source_file%.*}.o )
	done
}

#
#
# ----- RADAUDIO DECODER -------------------------------------
#
#
echo Rad Audio Decoder...

#
# OSX
#
APPLE_SDK=macosx
echo ...OSX
# ---- OSX X64 ----
OUTPUT=()
BUILD_LOCATION=$BUILD_ROOT/radaudio_osx_x64

CLANG_ARGS=(${OSX_X64_ARGS[@]})
CLANG_SOURCES=(${RADAUDIO_SOURCES[@]})
CallClang

CLANG_ARGS=(${OSX_X64_ARGS[@]} ${AVX2_ARGS[@]})
CLANG_SOURCES=(${RADAUDIO_SOURCES_AVX2[@]})
CallClang

rm -f $OUTPUT_LOCATION/libradaudio_decoder_osx_x64_static.a
ar rcs $OUTPUT_LOCATION/libradaudio_decoder_osx_x64_static.a ${OUTPUT[@]}

# ---- OSX ARM64 ----
OUTPUT=()
BUILD_LOCATION=$BUILD_ROOT/radaudio_osx_arm64

CLANG_ARGS=(${OSX_ARM64_ARGS[@]})
CLANG_SOURCES=(${RADAUDIO_SOURCES[@]})
CallClang

rm -f $OUTPUT_LOCATION/libradaudio_decoder_osx_arm64_static.a
ar rcs $OUTPUT_LOCATION/libradaudio_decoder_osx_arm64_static.a ${OUTPUT[@]}

# ---- LIPO ----
rm -f $OUTPUT_LOCATION/libradaudio_decoder_osx.a
lipo -create $OUTPUT_LOCATION/libradaudio_decoder_osx_x64_static.a $OUTPUT_LOCATION/libradaudio_decoder_osx_arm64_static.a -output $OUTPUT_LOCATION/libradaudio_decoder_osx.a
rm $OUTPUT_LOCATION/libradaudio_decoder_osx_arm64_static.a
rm $OUTPUT_LOCATION/libradaudio_decoder_osx_x64_static.a


#
# iOS
#
echo ...IOS
APPLE_SDK=iphoneos

# ---- iOS ARM64 ----
OUTPUT=()
BUILD_LOCATION=$BUILD_ROOT/radaudio_ios_arm64

CLANG_ARGS=(${IOS_ARM64_ARGS[@]})
CallClang

rm -f $OUTPUT_LOCATION/libradaudio_decoder_ios.a
ar rcs $OUTPUT_LOCATION/libradaudio_decoder_ios.a ${OUTPUT[@]}

# ---- iOS Sim x64 ----
APPLE_SDK=iphonesimulator

OUTPUT=()
BUILD_LOCATION=$BUILD_ROOT/radaudio_iossim_x64

CLANG_ARGS=(${IOS_SIM_ARGS_X64[@]})
CallClang

rm -f $OUTPUT_LOCATION/libradaudio_decoder_iossim_x64_static.a
ar rcs $OUTPUT_LOCATION/libradaudio_decoder_iossim_x64_static.a ${OUTPUT[@]}

# ---- iOS Sim ARM64 ----
OUTPUT=()
BUILD_LOCATION=$BUILD_ROOT/radaudio_iossim_arm64

CLANG_ARGS=(${IOS_SIM_ARGS_ARM64[@]})
CallClang

rm -f $OUTPUT_LOCATION/libradaudio_decoder_iossim_arm64_static.a
ar rcs $OUTPUT_LOCATION/libradaudio_decoder_iossim_arm64_static.a ${OUTPUT[@]}

# ---- LIPO SIM ----
rm -f $OUTPUT_LOCATION/libradaudio_decoder_iossim.a
lipo -create $OUTPUT_LOCATION/libradaudio_decoder_iossim_x64_static.a $OUTPUT_LOCATION/libradaudio_decoder_iossim_arm64_static.a -output $OUTPUT_LOCATION/libradaudio_decoder_iossim.a
rm $OUTPUT_LOCATION/libradaudio_decoder_iossim_x64_static.a
rm $OUTPUT_LOCATION/libradaudio_decoder_iossim_arm64_static.a


#
# TVOS
#
echo ...TVOS
APPLE_SDK=appletvos

OUTPUT=()
BUILD_LOCATION=$BUILD_ROOT/radaudio_tvos_arm64

CLANG_ARGS=(${TVOS_ARM64_ARGS[@]})
CallClang

rm -f $OUTPUT_LOCATION/libradaudio_decoder_tvos.a
ar rcs $OUTPUT_LOCATION/libradaudio_decoder_tvos.a ${OUTPUT[@]}

#
# VisionOS
#
echo ...VisionOS
APPLE_SDK=xros

OUTPUT=()
BUILD_LOCATION=$BUILD_ROOT/radaudio_visionos_arm64

CLANG_ARGS=(${VISIONOS_ARM64_ARGS[@]})
CallClang

rm -f $OUTPUT_LOCATION/libradaudio_decoder_visionos.a
ar rcs $OUTPUT_LOCATION/libradaudio_decoder_visionos.a ${OUTPUT[@]}

# ---- sim ----
APPLE_SDK=xrsimulator

OUTPUT=()
BUILD_LOCATION=$BUILD_ROOT/radaudio_visionossim_arm64

CLANG_ARGS=(${VISIONOS_SIM_ARM64_ARGS[@]})
CallClang

rm -f $OUTPUT_LOCATION/libradaudio_decoder_visionossim.a
ar rcs $OUTPUT_LOCATION/libradaudio_decoder_visionossim.a ${OUTPUT[@]}



#
#
# ----- RADAUDIO ENCODER -------------------------------------
#
#
echo RadAudio Encoder...

#
# OSX
#
APPLE_SDK=macosx

# ---- OSX X64 ----
OUTPUT=()
BUILD_LOCATION=$BUILD_ROOT/radaudio_encoder_osx_x64

CLANG_ARGS=(${OSX_X64_ARGS[@]})
CLANG_SOURCES=(${RADAUDIO_ENCODER_SOURCES[@]})
CallClang

rm -f $OUTPUT_LOCATION/libradaudio_encoder_osx_x64_static.a
ar rcs $OUTPUT_LOCATION/libradaudio_encoder_osx_x64_static.a ${OUTPUT[@]}

# ---- OSX ARM64 ----
OUTPUT=()
BUILD_LOCATION=$BUILD_ROOT/radaudio_encoder_osx_arm64

CLANG_ARGS=(${OSX_ARM64_ARGS[@]})
CLANG_SOURCES=(${RADAUDIO_ENCODER_SOURCES[@]})
CallClang

rm -f $OUTPUT_LOCATION/libradaudio_encoder_osx_arm64_static.a
ar rcs $OUTPUT_LOCATION/libradaudio_encoder_osx_arm64_static.a ${OUTPUT[@]}

# ---- LIPO ----
rm -f $OUTPUT_LOCATION/libradaudio_encoder_osx.a
lipo -create $OUTPUT_LOCATION/libradaudio_encoder_osx_x64_static.a $OUTPUT_LOCATION/libradaudio_encoder_osx_arm64_static.a -output $OUTPUT_LOCATION/libradaudio_encoder_osx.a
rm $OUTPUT_LOCATION/libradaudio_encoder_osx_arm64_static.a
rm $OUTPUT_LOCATION/libradaudio_encoder_osx_x64_static.a



echo Done.
