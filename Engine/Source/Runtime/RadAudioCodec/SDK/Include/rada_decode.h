// Copyright Epic Games Tools, LLC. All Rights Reserved.
#pragma once
#include <stdint.h> // uint*_t
#include <stddef.h> // size_t

#include "rada_file_header.h"

struct RadAContainer;

/*
    Rad Audio Container

    This is a file format containing Rad Audio encoded data. It provides
    multi channel streams with a seek table, as well as some metadata to
    make decoding a bit more predictable memory-wise.

    The data on disk is ordered as:
    FileHeader
    SeekHeader, if seek table is present
    Rad Audio Stream Headers
    SeekTable, if seek table is present.
    Encoded blocks.
*/

// RADA compatibility version - if these are the same the exports are named the same so we expect
// that if the linker selects a different copy they all work.
#define RADA_LIBRARY_VERSION    1

#ifndef RR_STRING_JOIN3
#define RR_STRING_JOIN3(arg1, arg2, arg3)            RR_STRING_JOIN_DELAY3(arg1, arg2, arg3)
#define RR_STRING_JOIN_DELAY3(arg1, arg2, arg3)      RR_STRING_JOIN_IMMEDIATE3(arg1, arg2, arg3)
#define RR_STRING_JOIN_IMMEDIATE3(arg1, arg2, arg3)  arg1 ## arg2 ## arg3
#endif

#ifndef RR_STRING_JOIN
#define RR_STRING_JOIN(arg1, arg2)              RR_STRING_JOIN_DELAY(arg1, arg2)
#define RR_STRING_JOIN_DELAY(arg1, arg2)        RR_STRING_JOIN_IMMEDIATE(arg1, arg2)
#define RR_STRING_JOIN_IMMEDIATE(arg1, arg2)    arg1 ## arg2
#endif

#ifdef RADA_WRAP
#define RADA_NAME(name) RR_STRING_JOIN3(RADA_WRAP, name##_, RADA_LIBRARY_VERSION )
#else
#error RADA_WRAP must be defined (MIRA or UERA)
#endif

#define RadAGetFileHeader           RADA_NAME(RadAGetFileHeader)
#define RadAGetSeekTableHeader      RADA_NAME(RadAGetSeekTableHeader)
#define RadAStripSeekTable          RADA_NAME(RadAStripSeekTable)
#define RadAGetMemoryNeededToOpen   RADA_NAME(RadAGetMemoryNeededToOpen)
#define RadAOpenDecoder             RADA_NAME(RadAOpenDecoder)
#define RadADecodeSeekTable         RADA_NAME(RadADecodeSeekTable)
#define RadADirectSeekTableLookup   RADA_NAME(RadADirectSeekTableLookup)
#define RadASeekTableLookup         RADA_NAME(RadASeekTableLookup)
#define RadAExamineBlock            RADA_NAME(RadAExamineBlock)
#define RadANotifySeek              RADA_NAME(RadANotifySeek)
#define RadADecodeBlock             RADA_NAME(RadADecodeBlock)
#define RadAInterleave              RADA_NAME(RadAInterleave)

// The number of bytes the seek table takes up on disk. It exists after
// RadABytesToOpen, and before RadABytesToFirstBlock
static uint32_t RadAGetSeekTableSizeOnDisk(const RadAFileHeader* FileHeader)
{    
    if (FileHeader == 0 || FileHeader->seek_table_entry_count == 0)
        return 0;
    // align the bit count to 64 then convert to bytes;
    uint32_t seek_table_bits = FileHeader->bits_for_seek_table_samples + FileHeader->bits_for_seek_table_bytes;
    uint32_t seek_table_bytes = 8 * ((seek_table_bits * FileHeader->seek_table_entry_count + 63) / 64);
    return seek_table_bytes;
}

// Returns the number of bytes of the rada file have to be in memory in order to call
// RadAOpenDecoder. Since this requires a valid file header, getting the size to open
// is a two phase operation: read the file header (sizeof(FileHeader)), then query the file header
// with this function.
//
// This is also the offset to the first encoded block.
static uint32_t RadAGetBytesToOpen(const RadAFileHeader* FileHeader)
{
    if (FileHeader == 0)
        return 0;
    uint32_t SizeWithoutSeekTable = sizeof(RadAFileHeader) + FileHeader->rada_header_bytes;
    if (FileHeader->seek_table_entry_count)
    {
        return SizeWithoutSeekTable + sizeof(RadASeekTableHeader) + RadAGetSeekTableSizeOnDisk(FileHeader);
    }
    return SizeWithoutSeekTable;
}

static uint32_t RadAGetOffsetToSeekTable(const RadAFileHeader* FileHeader)
{
    if (FileHeader == 0 ||
        FileHeader->seek_table_entry_count == 0)
        return 0;
    return sizeof(RadAFileHeader) + sizeof(RadASeekTableHeader) + FileHeader->rada_header_bytes;
}

// Returns a pointer to the file header from the given data. The header will be validated.
// This function needs at least sizeof(RadAFileHeader) passed to it to succeed. If insufficient
// data is provided, or the header is incorrect in any way, 0 is returned.
const RadAFileHeader* RadAGetFileHeader(const uint8_t* InFileData, size_t InFileDataLenBytes);
const RadASeekTableHeader* RadAGetSeekTableHeader(const uint8_t* InFileData, size_t InFileDataLenBytes);

// Retrieves the file header from an opened decoder.
static const RadAFileHeader* RadAGetFileHeaderFromContainer(const RadAContainer* InContainer) { return (const RadAFileHeader*)InContainer; }

// Removes the seek table, if present, from the given file and returns the new size of the data.
// Returns 0 on invalid file.
size_t RadAStripSeekTable(uint8_t* InFileData, size_t InFileDataLenBytes);

// Gets the size of the RadAContainer to pass to RadAOpenDecoder. You use this to determine how much to allocate
// for the entirely of the decoder structure (note: you are responsible for allocating seek table space yourself)
// returns:
//  0 on success
//  -1 on bad data
//  >0 is the amount of file data we need to make progress in getting the result. This can happen multiple times
//      in a row.
int32_t RadAGetMemoryNeededToOpen(const uint8_t* InFileData, size_t InFileDataLenBytes, uint32_t* OutMemoryRequired);

// InData is the file data loaded in to memory. The amount to read is specified by RadABytesToOpen(). InContainer
// must be allocated to the size specified by RadAGetMemoryNeeded. When done decoding, no function needs to be called,
// just free the container.
// RadAContainer* Container = malloc(RadAGetMemoryNeeded());
// RadAOpenDecoder(FileData, FileDataSize, Container, RadAGetMemoryNeeded());
// Returns 1 on success, 0 on fail.
int32_t RadAOpenDecoder(const uint8_t* InFileData, size_t InFileDataLenBytes, RadAContainer* InContainer, size_t InContainerBytes);


struct SeekTableEnumerationState
{
    uint64_t state[2];
    SeekTableEnumerationState() { state[0] = state[1] = 0; }
};

enum class RadASeekTableReturn
{
    // All seek table entries have been decoded
    Done,
    InvalidParam,
    // The input data is invalid in some way
    InvalidData,
    // The buffer provided needs to be advanced by the bytes consumed, filled up,
    // and the function called again.
    NeedsMoreData,

    // The seek table output is too big for 32 bits, start over with 64 bits.
    Needs64Bits
};

/*
* Iteratively decode the seek table in to your own management structures. This is to facilitate allocating during opening or needing a large stack.
* 
* InData is the on disk encoded seek table. This is offset from the beginning of the file at RadABytesToOpen(). This can be streamed
* in chunks, but to get the entire table you'll need to overall provide RadASeekTableSizeOnDisk() to the function (at which point
* it will return ::Done.
* 
* OutSeekTableSamples and OutSeekTableBytes need to be point to arrays of either uint32s or uint64s based on whether InSeekTableIs64Bits, of
* size InFileHeader->seek_table_entry_count.
* 
* OutConsumed will emit how much of the buffer got consumed this run. This might not be completely consumed before more data is needed.
* 
* You can of course just pass the entire seek table in one go.
* 
* Roughly:
* 
* 
* SeekTableEnumerationState EnumState;
* for (;;)
* {
*     size_t bytes_consumed = 0;
*     RadASeekTableReturn result = RadADecodeSeekTable(file_header, seek_table_buffer, seek_table_buffer_valid_bytes, false, &EnumState, ptr_to_samples, ptr_to_bytes, &bytes_consumed);
*      
*     if (result == RadASeekTableReturn::Done)
*         break;
*     if (result != RadASeekTableReturn::NeedsMoreData)
*         return failure;
*
*     seek_table_bytes_remaining -= bytes_consumed;
*     seek_table_buffer_valid -= bytes_consumed;
*     if (seek_table_bytes_remaining == 0)
*         return MilesDecoderOpenReturnFail; // they asked for more data but exceed what they said they'd need!
* }
*/
RadASeekTableReturn RadADecodeSeekTable(const RadAFileHeader* InFileHeader, const RadASeekTableHeader* InSeekHeader, uint8_t* InSeekTableData, size_t InSeekTableDataLenBytes, bool InSeekTableIs64Bits, SeekTableEnumerationState* InOutEnumerationState, uint8_t* OutSeekTableSamples, uint8_t* OutSeekTableBytes, size_t* OutConsumed);

// Look up where a given frame is in the file without doing a full decoder open. This is a binary search on the seek table
// and should be fairly fast.
// The return is 0 on failure, or the byte location of the encoded block that will provide the given frame. OutFrameAtLocation
// returns the frame that first gets emitted when decoding starts at that location, and OutFrameBlockSize will return the size
// required to decode the first block at that location.
//
// NOTE!! OutFrameBlockSize will allow you to decode at least 1 block, however due to the mDCT nature of the codec,
// one block must be "wasted" to prime the decoder before it starts to emit. So do not expect to get the desired frame out
// after one decode, and you may need more than that amount. That just gets you *a* block.
size_t RadADirectSeekTableLookup(const uint8_t* InFileData, size_t InFileDataLenBytes, uint64_t InTargetFrame, size_t* OutFrameAtLocation, size_t* OutFrameBlockSize);

// Same as above, except if you've opened the decoder.
size_t RadASeekTableLookup(const RadAContainer* InContainer, uint64_t InTargetFrame, size_t* OutFrameAtLocation, size_t* OutFrameBlockSize);

enum class RadAExamineBlockResult : uint8_t
{
    // The buffer provided can be passed to RadADecodeBlock and expect to succeed.
    Valid,
    // We need more data in order to determine. OutBufferLenBytesNeeded contained how much we need.
    Incomplete,
    // The buffer can't be decoded.
    Invalid,
};

// Examine encoded data to determine how much data is needed to decode the next block.
RadAExamineBlockResult RadAExamineBlock(const RadAContainer* InContainer, const uint8_t* InBuffer, size_t InBufferLenBytes, uint32_t* OutBufferLenBytesNeeded);

// This function must be called before RadADecodeBlock if data from the file is not provided sequentially.
void RadANotifySeek(RadAContainer* InContainer);

// A single call to RadADecodeBlock can never put more than this number of frames in to the output reservoir.
constexpr uint32_t RadADecodeBlock_MaxOutputFrames = 1024;

#define RadADecodeBlock_Error -2
#define RadADecodeBlock_Done -1

/*
* Decode a single block of encoded rad audio data.
* 
* The data provided shoulid have passed a RadAExamineBlock check beforehand.
* The return value is the number of frames decoded, or one of RadADecodeBlock_Error or
* RadADecodeBlock_Done, which indicates end of file.
*
* At most RadADecodeBlock_MaxOutputFrames will be emitted into the output buffer.
*
* It is normal for 0 frames to get emitted after a seek or the beginning of the file.
*
* InBuffer should be advanced by OutConsumedBytes, which should then point at the next
* block to pass to RadAExamineBlock.
*/
int16_t RadADecodeBlock(RadAContainer* InContainer,
    const uint8_t* InBuffer, size_t InBufferLenBytes, 
    float* InOutputBuffer, size_t InOutBufferStrideInFloats, 
    size_t* OutConsumedBytes);

// utility function to interleave N buffers of float data into one output buffer of S16 (scales and clamps from -1.0 to 1 into -32768 to 32767)
#define RADA_HAS_INTERLEAVING
void  RadAInterleave( int16_t * output, float const ** inputs, uint32_t input_count, uint32_t samples );

