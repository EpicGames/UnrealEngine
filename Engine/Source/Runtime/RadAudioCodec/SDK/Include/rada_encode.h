// Copyright Epic Games Tools, LLC. All Rights Reserved.
#pragma once
#include <stdint.h>

#define RADA_LIBRARY_VERSION    1

#ifndef RR_STRING_JOIN3
#define RR_STRING_JOIN3(arg1, arg2, arg3)            RR_STRING_JOIN_DELAY3(arg1, arg2, arg3)
#define RR_STRING_JOIN_DELAY3(arg1, arg2, arg3)      RR_STRING_JOIN_IMMEDIATE3(arg1, arg2, arg3)
#define RR_STRING_JOIN_IMMEDIATE3(arg1, arg2, arg3)  arg1 ## arg2 ## arg3
#endif

#ifndef RR_STRING_JOIN
#define RR_STRING_JOIN(arg1, arg2)              RR_STRING_JOIN_DELAY(arg1, arg2)
#define RR_STRING_JOIN_DELAY(arg1, arg2)        RR_STRING_JOIN_IMMEDIATE(arg1, arg2)
#define RR_STRING_JOIN_IMMEDIATE(arg1, arg2)    arg1 ## arg2
#endif


#ifdef RADA_WRAP
#define RADA_NAME(name) RR_STRING_JOIN3(RADA_WRAP, name##_, RADA_LIBRARY_VERSION )
#else
#error RADA_WRAP must be defined (MIRA or UERA)
#endif


typedef void* RadACompressAllocFnType(uintptr_t ByteCount);
typedef void RadACompressFreeFnType(void* Ptr);

#define RADA_COMPRESS_SUCCESS 0
#define RADA_COMPRESS_ERROR_CHANS 1
#define RADA_COMPRESS_ERROR_SAMPLES 2
#define RADA_COMPRESS_ERROR_RATE 3
#define RADA_COMPRESS_ERROR_QUALITY 4
#define RADA_COMPRESS_ERROR_ALLOCATORS 5
#define RADA_COMPRESS_ERROR_OUTPUT 6
#define RADA_COMPRESS_ERROR_SEEKTABLE 7
#define RADA_COMPRESS_ERROR_SIZE 8
#define RADA_COMPRESS_ERROR_ENCODER 9
#define RADA_COMPRESS_ERROR_ALLOCATION 10

#define RadAErrorString     RADA_NAME(RadAErrorString)
#define EncodeRadAFile      RADA_NAME(EncodeRadAFile)
#define RadAGetBuildVersion RADA_NAME(RadAGetBuildVersion)

// The build version of the rad audio library in it's entirety. This is distinct from the
// compilation version that determine linker compatibility. This is incremented each time
// the library is released
#define RADA_BUILD_VERSION (0x00000001U)

// Return the build version that the library was compiled with. This should match RADA_BUILD_VERSION!
uint32_t RadAGetBuildVersion();

// Pass in one of the above values to get a human readable error, returned
// as a utf8 string.
const char* RadAErrorString(uint8_t InError);

static const uint32_t RADA_MAX_CHANS = 32;
static const uint32_t RADA_VALID_RATES[] = {48000, 44100, 32000, 24000};

//
// Create a "RADA" file, which is a container format for blocks of rad audio encoded
// sound data.
//
// Compresses up to 32 channels. Only RADA_VALID_RATES sample rates are allowed.
//
// Quality is between 1 and 9, with 9 being high quality. 5 is a sane default.
// PcmData should be interleaved 16 bit pcm data.
// PcmDataLen is in bytes.
// OutData will be filled with a buffer allocated by MemAlloc that contains the file data.
// SeekTableMaxEntries is used to cap the amount of disk space used on the seek table.
// If GenerateSeekTable is set, this should be at least 1k or so to be useful.
// if InSeamlessLooping is set, the encoder provides the ends of the stream to the encoder for
//  use in padding the mDCT. This coerces the math to provide outputs that sync across the file endpoints.
#if defined(__RADINDLL__) && defined(_MSC_VER) // we have a windows dll target.
__declspec(dllexport)
#endif
uint8_t EncodeRadAFile(
    void* InPcmData, 
    uint64_t InPcmDataLen, 
    uint32_t InPcmRate, 
    uint8_t InPcmChannels, 
    uint8_t InQuality, 
    uint8_t InSeamlessLooping,
    uint8_t InGenerateSeekTable, 
    uint16_t InSeekTableMaxEntries,
    RadACompressAllocFnType* InMemAlloc,
    RadACompressFreeFnType* InMemFree,
    void** OutData, 
    uint64_t* OutDataLen);

