// Copyright Epic Games Tools, LLC. All Rights Reserved.

#pragma once
#include <stdint.h>

#define RADA_SYNC_BYTE (0b01010101)
#define RADA_MAX_CHANNELS 32

enum class ERadASampleRate : uint8_t
{
    Rate_24000,
    Rate_32000,
    Rate_44100,
    Rate_48000,
    Rate_COUNT,
};

struct RadAFileHeader
{
    uint32_t tag; // RADA

    // Bytes from the file necessary to completely decode the first block of audio.
    uint32_t bytes_for_first_decode;

    uint8_t version;
    uint8_t channels;

    // Bytes of rad audio stream header data, located right after RadAFileHeader.
    uint16_t rada_header_bytes;
    uint8_t shift_bits_for_seek_table_samples;
    uint8_t bits_for_seek_table_bytes;
    uint8_t bits_for_seek_table_samples;
    ERadASampleRate sample_rate;

    // total number of samples/frames in the file.
    uint64_t frame_count;

    uint64_t file_size;

    uint16_t seek_table_entry_count;

    // The maximum size of any block of audio, including any RadA block header. This can be
    // used to allocate a buffer that can be used to decode the entire file one block at a time.
    uint16_t max_block_size;
    
    uint16_t PADDING[2];
};

struct RadASeekTableHeader
{
    int64_t byte_line_slope[2];
    int64_t byte_intercept;
    int64_t byte_bias;
    int64_t sample_line_slope[2];
    int64_t sample_intercept;
    int64_t sample_bias;
};

static uint32_t RadASampleRateFromEnum(ERadASampleRate InRate)
{
    switch (InRate)
    {
    case ERadASampleRate::Rate_24000: return 24000;
    case ERadASampleRate::Rate_32000: return 32000;
    case ERadASampleRate::Rate_44100: return 44100;
    case ERadASampleRate::Rate_48000: return 48000;
    default: return 0;
    }
}

// FileHeader
// SeekTableHeader, if present.
// StreamCountx[EncoderHeader], totalling rada_header_bytes
// SeekTable
// Encoded blocks
