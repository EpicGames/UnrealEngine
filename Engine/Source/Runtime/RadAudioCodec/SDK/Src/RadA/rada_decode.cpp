// Copyright Epic Games Tools, LLC. All Rights Reserved.
#include "rada_decode.h"
#include "radaudio_decoder.h"

#include <string.h>

struct RadAContainer
{
    RadAFileHeader Header;
    //
    // The memory layout is:
    // Container
    // Decoder Pointers
    // Decoder Memory
    // Seek header, if present
    // SeekTable, if present
    size_t OffsetToSeekHeader;
};

static uint8_t Container_StreamCount(RadAContainer* rada) { return (rada->Header.channels + 1) >> 1; }
static RadAudioDecoder** Container_Decoders(RadAContainer* rada) { return (RadAudioDecoder**)(rada + 1); }
static RadAudioDecoder** Container_Decoders(const RadAContainer* rada) { return (RadAudioDecoder**)(rada + 1); }
static uint8_t* Container_DecoderMemory(RadAContainer* rada)
{
    return (uint8_t*)(Container_Decoders(rada) + Container_StreamCount(rada));
}
static RadASeekTableHeader* Container_SeekHeader(RadAContainer* rada)
{
    return rada->OffsetToSeekHeader ? (RadASeekTableHeader*)((uint8_t*)rada + rada->OffsetToSeekHeader) : 0;
}
static const RadASeekTableHeader* Container_SeekHeader(const RadAContainer* rada)
{
    return rada->OffsetToSeekHeader ? (const RadASeekTableHeader*)((uint8_t*)rada + rada->OffsetToSeekHeader) : 0;
}
static uint8_t* Container_SeekTable(RadAContainer* rada)
{
    return rada->OffsetToSeekHeader ? (uint8_t*)(Container_SeekHeader(rada) + 1) : 0;
}
static const uint8_t* Container_SeekTable(const RadAContainer* rada)
{
    return rada->OffsetToSeekHeader ? (const uint8_t*)(Container_SeekHeader(rada) + 1) : 0;
}


// If positive: the bytes necessary in order to advance (might take multiple passes to succeed)
// If 0: Success
// If negative: error
const RadAFileHeader* RadAGetFileHeader(const uint8_t* InData, size_t InDataSize)
{
    if (InDataSize < sizeof(RadAFileHeader))
        return 0;

    if (InData == 0)
        return 0;

    const RadAFileHeader* FileHeader = (const RadAFileHeader*)InData;

    // Check file type...
    if (FileHeader->tag != 'RADA')
        return 0;

    // Check version
    if (FileHeader->version > 1)
        return 0;

    if (FileHeader->channels > RADA_MAX_CHANNELS)
        return 0;

    return FileHeader;
}

const RadASeekTableHeader* RadAGetSeekTableHeader(const uint8_t* InData, size_t InDataSize)
{
    const RadAFileHeader* FileHeader = RadAGetFileHeader(InData, InDataSize);
    if (FileHeader == 0 || FileHeader->seek_table_entry_count == 0)
        return 0;

    if (InDataSize < (sizeof(RadAFileHeader) + sizeof(RadASeekTableHeader)))
        return 0;

    return (const RadASeekTableHeader*)(FileHeader + 1);
}


static size_t align8(size_t n) { return (n + 7) & ~7; }

size_t RadAStripSeekTable(uint8_t* InData, size_t InDataSize)
{
    RadAFileHeader* FileHeader = (RadAFileHeader*)RadAGetFileHeader(InData, InDataSize);
    if (FileHeader == 0 || InDataSize != FileHeader->file_size) // we must have the whole file.
        return 0;

    if (FileHeader->seek_table_entry_count == 0)
        return InDataSize;

    // Layout is:
    // FileHeader
    // SeekHeader
    // StreamHeaders
    // SeekTable
    // EncodedBlocks
    // 
    // want:
    // 
    // FileHeader
    // StreamHeaders
    // EncodedBlocks
    // 
    // So we move: 
    //      StreamHeaders -> after FileHeader
    //      EncodedBlocks -> after StreamHeaders
    // 
    size_t OffsetToData = RadAGetBytesToOpen(FileHeader);
    size_t BytesToRemove = RadAGetSeekTableSizeOnDisk(FileHeader) + sizeof(RadASeekTableHeader);
    size_t EncodedDataSize = InDataSize - OffsetToData;

    // StreamHeaders -> after FileHeader
    memmove(InData + sizeof(RadAFileHeader), InData + sizeof(RadAFileHeader) + sizeof(RadASeekTableHeader), FileHeader->rada_header_bytes);

    // EncodedBlocks -> after StreamHeaders.
    memmove(InData + sizeof(RadAFileHeader) + FileHeader->rada_header_bytes, InData + OffsetToData, EncodedDataSize);
    
    // Update size.
    FileHeader->seek_table_entry_count = 0;
    FileHeader->file_size -= BytesToRemove;

    return InDataSize - BytesToRemove;
}

int32_t RadAGetMemoryNeededToOpen(const uint8_t* InData, size_t InDataSize, uint32_t* OutMemoryRequired)
{
    if (OutMemoryRequired == 0)
        return -1;

    if (InDataSize < sizeof(RadAFileHeader))
        return sizeof(RadAFileHeader);

    // We need to ask the codec how much memory they need, which means we need the
    // headers.

    // we need memory for:
    // -> The actual codec memory
    // -> The pointers to our codec memory.
    // -> The container structure
    // -> The seek table + header.

    const RadAFileHeader* FileHeader = (const RadAFileHeader*)InData;
    uint8_t StreamCount = (FileHeader->channels + 1) >> 1;

    const RadASeekTableHeader* SeekHeader = FileHeader->seek_table_entry_count ? (const RadASeekTableHeader*)(FileHeader + 1) : 0;

    // Stream headers are after the seek header if it exists, otherwise the file header.
    const uint8_t* StreamHeaders = SeekHeader ? (const uint8_t*)(SeekHeader + 1) : (const uint8_t*)(FileHeader + 1);

    // Check if we have enough of InData to determine the memory requirements. We need enough data to get to
    // the stream headers, plus the actual size of the stream headers.
    size_t BytesToStreamHeaders = StreamHeaders - InData;
    if (BytesToStreamHeaders + FileHeader->rada_header_bytes > InDataSize) // can't overflow - this is just a bunch of sizeofs() and an uint16.
        return sizeof(RadAFileHeader) + FileHeader->rada_header_bytes;

    size_t RemainingHeaderBytes = FileHeader->rada_header_bytes;
    size_t CodecMemoryRequired = 0;
    for (uint8_t StreamIndex = 0; StreamIndex < StreamCount; StreamIndex++)
    {
        if (RemainingHeaderBytes < 64) // min required to parse header
            return -1;

        CodecMemoryRequired += align8(RadAudioDecoderMemoryRequired((uint8_t*)StreamHeaders, RemainingHeaderBytes));
        RadAudioInfo Info = {};
        size_t HeaderSizeBytes = RadAudioDecoderGetInfoHeader((uint8_t*)StreamHeaders, RemainingHeaderBytes, &Info);
        if (HeaderSizeBytes == 0 ||
            HeaderSizeBytes > RemainingHeaderBytes)
            return -1;
        RemainingHeaderBytes -= HeaderSizeBytes;
        StreamHeaders += HeaderSizeBytes;
    }

    size_t TotalMemoryRequired = sizeof(RadAContainer) + CodecMemoryRequired + sizeof(RadAudioDecoder*) * StreamCount;
    if (SeekHeader)
        TotalMemoryRequired += sizeof(RadASeekTableHeader) + RadAGetSeekTableSizeOnDisk(FileHeader);

    if (TotalMemoryRequired >= ~0U)
        return -1;

    *OutMemoryRequired = (uint32_t)TotalMemoryRequired;
    return 0;
}


int32_t RadAOpenDecoder(const uint8_t* InData, size_t InDataSize, RadAContainer* InContainerPtr, size_t InContainerBytes)
{
    if (InContainerBytes < sizeof(RadAContainer))
        return 0;

    RadAContainer* Container = (RadAContainer*)InContainerPtr;

    // We should have enough data to open at this point.
    const RadAFileHeader* FileHeader = RadAGetFileHeader(InData, InDataSize);
    if (FileHeader == 0)
    	return 0;
    
    Container->Header = *FileHeader;

    if (InDataSize < RadAGetBytesToOpen(&Container->Header))
        return 0; // they didn't listen to us.

    // The stream headers are right after the file/seek header
    uint32_t StreamHeaderBytes = Container->Header.rada_header_bytes;

    // sanity the header bytes and read them.
    uint8_t StreamCount = Container_StreamCount(Container);
    if (StreamHeaderBytes > (uint32_t)(RADAUDIO_STREAM_HEADER_SIZE * StreamCount))
        return 0;

    uint32_t MemoryRequired = 0;
    if (RadAGetMemoryNeededToOpen(InData, InDataSize, &MemoryRequired) != 0 ||
        MemoryRequired > InContainerBytes)
    {
        return 0;
    }

    uint8_t* ContainerEnd = (uint8_t*)InContainerPtr + InContainerBytes;

    RadAudioDecoder** Decomps = Container_Decoders(Container);
    uint8_t* StreamMemory = Container_DecoderMemory(Container);

    if (StreamMemory + StreamHeaderBytes >= ContainerEnd)
        return 0;

    uint32_t RemainingHeaderBytes = StreamHeaderBytes;
    const uint8_t* HeaderBytes = InData + sizeof(RadAFileHeader) + (Container->Header.seek_table_entry_count ? sizeof(RadASeekTableHeader) : 0);

    for (uint8_t stream = 0; stream < StreamCount; stream++)
    {
        if (RemainingHeaderBytes < 64)
            return 0;

        size_t codec_bytes = RadAudioDecoderMemoryRequired((uint8_t*)HeaderBytes, RemainingHeaderBytes);
        if (StreamMemory + codec_bytes > ContainerEnd)
            return 0;

        size_t header_bytes_this_stream = 0;
        Decomps[stream] = RadAudioDecoderOpen((uint8_t*)HeaderBytes, RemainingHeaderBytes, StreamMemory, codec_bytes, &header_bytes_this_stream);
        if (Decomps[stream] == 0 || header_bytes_this_stream > RemainingHeaderBytes)
            return 0;

        HeaderBytes += header_bytes_this_stream;
        RemainingHeaderBytes -= (uint32_t)header_bytes_this_stream;

        StreamMemory += align8(codec_bytes);
    }

    const RadASeekTableHeader* SeekHeader = RadAGetSeekTableHeader(InData, InDataSize);
    if (SeekHeader)
    {
        // Copy over the seek table structures.
        Container->OffsetToSeekHeader = (StreamMemory - (uint8_t*)Container);
        *Container_SeekHeader(Container) = *SeekHeader;
        memcpy(Container_SeekTable(Container), InData + RadAGetOffsetToSeekTable(FileHeader), RadAGetSeekTableSizeOnDisk(FileHeader));
    }

    if (RemainingHeaderBytes != 0)
        return 0;
    return 1;
}

static bool BitContainerExtract(const uint8_t* bit_data, size_t bit_data_len_words, uint64_t bit_position, uint32_t bit_count, uint64_t& out_bits)
{
    // extract the bits.
    uint64_t base_bit = bit_position;
    uint64_t base_word = base_bit / 64;
    uint64_t base_offset = base_bit - base_word * 64;

    if (base_word >= bit_data_len_words)
        return false;

    uint64_t output = 0;

    uint64_t bits_avail = 64 - base_offset;

    uint64_t bits_from_first = bit_count;
    if (bits_from_first > bits_avail)
        bits_from_first = bits_avail;

    uint64_t bits_to_clear = 64 - (base_offset + bits_from_first);
    uint64_t read_bits;
    memcpy(&read_bits, bit_data + base_word*8, 8);
    output = (read_bits << bits_to_clear) >> (bits_to_clear + base_offset);
    
    if (bits_from_first != bit_count)
    {
        // need some from the next word
        base_word++;
        if (base_word >= bit_data_len_words)
            return false;

        base_offset = 0;
        bits_to_clear = 64 - (bit_count - bits_from_first);

        memcpy(&read_bits, bit_data + base_word*8, 8);
        output |= ((read_bits << bits_to_clear) >> bits_to_clear) << bits_from_first;
    }
    out_bits = output;
    return true;
}

struct SeekTableEnumerationStateInternal
{
    uint64_t consumed_qwords;

    uint16_t seek_table_position;
    uint8_t done;
};

static_assert(sizeof(SeekTableEnumerationStateInternal) <= sizeof(SeekTableEnumerationState), "seek table enum state size mismatch");


// In order to avoid requiring a temp allocation for the entire seek table,
// we set it up so it can be streamed in chunks. The seek table isn't big, but it's big enough
// that we don't want to toss it on the stack as a temp.
RadASeekTableReturn RadADecodeSeekTable(const RadAFileHeader* InFileHeader, const RadASeekTableHeader* InSeekHeader, uint8_t* InData, size_t InDataLenBytes, bool InSeekTableIs64Bits, SeekTableEnumerationState* InOutEnumerationState, uint8_t* OutSeekTableSamples, uint8_t* OutSeekTableBytes, size_t* OutConsumed)
{
    if (InOutEnumerationState == 0 ||
        OutSeekTableBytes == 0 ||
        OutSeekTableSamples == 0 || 
        InData == 0 ||
        InDataLenBytes == 0 ||
        InFileHeader == 0 ||
        InSeekHeader == 0 ||
        OutConsumed == 0)
        return RadASeekTableReturn::InvalidParam;

    // If they passed the entire file (or some other larger containing buffer) then we might not
    // actually get even qword count, however we know that we can only consume qwords.
    size_t DataLenWords = InDataLenBytes / 8;

    SeekTableEnumerationStateInternal EnumState;
    memcpy(&EnumState, InOutEnumerationState, sizeof(SeekTableEnumerationState));
    if (EnumState.done)
    {
        *OutConsumed = 0;
        return RadASeekTableReturn::Done;
    }

    const uint8_t sample_bits = InFileHeader->bits_for_seek_table_samples;
    const uint8_t byte_bits = InFileHeader->bits_for_seek_table_bytes;
    const uint8_t sample_shift_bits = InFileHeader->shift_bits_for_seek_table_samples;

    uint32_t offset_to_data = RadAGetBytesToOpen(InFileHeader);
    int64_t avg_index = InFileHeader->seek_table_entry_count/2;

    for (; EnumState.seek_table_position < InFileHeader->seek_table_entry_count; EnumState.seek_table_position++)
    {
        // we need to offset our request by the amount we have streamed out.
        uint64_t request_bit_position = (sample_bits + byte_bits) * EnumState.seek_table_position;
        request_bit_position -= EnumState.consumed_qwords * 64;

        uint64_t sample_biased_error = 0;
        uint64_t byte_biased_error = 0;
        if (BitContainerExtract(InData, DataLenWords, request_bit_position, sample_bits, sample_biased_error) == false ||
            BitContainerExtract(InData, DataLenWords, request_bit_position + sample_bits, byte_bits, byte_biased_error) == false)
        {
            // We can get rid of everything up to our base that we need. This might be zero
            // if they didn't give us enough data to do _anything_.
            uint64_t base_need_qword = request_bit_position / 64;
            if (base_need_qword)
                base_need_qword--;

            *OutConsumed = base_need_qword * 8;

            EnumState.consumed_qwords += base_need_qword;
            memcpy(InOutEnumerationState, &EnumState, sizeof(SeekTableEnumerationState));            
            return RadASeekTableReturn::NeedsMoreData;
        }

        int64_t sample_predicted = InSeekHeader->sample_intercept + ((int64_t)EnumState.seek_table_position - avg_index) * InSeekHeader->sample_line_slope[0] / InSeekHeader->sample_line_slope[1];
        int64_t sample_unbiased_error = sample_biased_error + InSeekHeader->sample_bias;
        uint64_t sample_result = (sample_predicted + sample_unbiased_error) << sample_shift_bits;

        int64_t byte_predicted = InSeekHeader->byte_intercept + ((int64_t)EnumState.seek_table_position - avg_index) * InSeekHeader->byte_line_slope[0] / InSeekHeader->byte_line_slope[1];
        int64_t byte_unbiased_error = byte_biased_error + InSeekHeader->byte_bias;
        uint64_t byte_result = (byte_predicted + byte_unbiased_error) + offset_to_data;

        if (InSeekTableIs64Bits)
        {
            memcpy(OutSeekTableSamples + sizeof(uint64_t) * EnumState.seek_table_position, &sample_result, sizeof(uint64_t));
            memcpy(OutSeekTableBytes + sizeof(uint64_t) * EnumState.seek_table_position, &byte_result, sizeof(uint64_t));
        }
        else
        {
            if (sample_result > ~0U ||
                byte_result > ~0U)
                return RadASeekTableReturn::Needs64Bits;

            uint32_t sample_32 = (uint32_t)sample_result;
            uint32_t byte_32 = (uint32_t)byte_result;

            memcpy(OutSeekTableSamples + sizeof(uint32_t) * EnumState.seek_table_position, &sample_32, sizeof(uint32_t));
            memcpy(OutSeekTableBytes + sizeof(uint32_t) * EnumState.seek_table_position, &byte_32, sizeof(uint32_t));
        }
    }

    EnumState.done = 1;
    memcpy(InOutEnumerationState, &EnumState, sizeof(SeekTableEnumerationState));
    {
        uint64_t request_bit_position = ((sample_bits + byte_bits) * EnumState.seek_table_position + 63) & ~63;
        request_bit_position -= EnumState.consumed_qwords * 64;        
        uint64_t last_success_qword = (request_bit_position) / 64;
        *OutConsumed = last_success_qword * 8;
    }
    return RadASeekTableReturn::Done;
}

static uint64_t RadAEvaluateSeekTableSamplePosition(const RadAFileHeader* InFileHeader, const RadASeekTableHeader* InSeekHeader, const uint8_t* InSeekTable, size_t InSeekTableQWords, uint32_t InIndex)
{
    int32_t avg_index = InFileHeader->seek_table_entry_count / 2;
    uint64_t request_bit_position = (InFileHeader->bits_for_seek_table_samples + InFileHeader->bits_for_seek_table_bytes) * InIndex;

    uint64_t sample_biased_error = 0;
    BitContainerExtract(InSeekTable, InSeekTableQWords, request_bit_position, InFileHeader->bits_for_seek_table_samples, sample_biased_error);

    int64_t sample_predicted = InSeekHeader->sample_intercept + ((int64_t)InIndex - avg_index) * InSeekHeader->sample_line_slope[0] / InSeekHeader->sample_line_slope[1];
    int64_t sample_unbiased_error = sample_biased_error + InSeekHeader->sample_bias;
    uint64_t sample_result = (sample_predicted + sample_unbiased_error) << InFileHeader->shift_bits_for_seek_table_samples;
    return sample_result;
}

static uint64_t RadAEvaluateSeekTableBytePosition(const RadAFileHeader* InFileHeader, const RadASeekTableHeader* InSeekHeader, const uint8_t* InSeekTable, size_t InSeekTableQWords, uint32_t InIndex)
{
    int32_t avg_index = InFileHeader->seek_table_entry_count / 2;

    uint64_t request_bit_position = (InFileHeader->bits_for_seek_table_samples + InFileHeader->bits_for_seek_table_bytes) * InIndex;

    uint64_t biased_error = 0;
    BitContainerExtract(InSeekTable, InSeekTableQWords, request_bit_position + InFileHeader->bits_for_seek_table_samples, InFileHeader->bits_for_seek_table_bytes, biased_error);

    int64_t predicted = InSeekHeader->byte_intercept + ((int64_t)InIndex - avg_index) * InSeekHeader->byte_line_slope[0] / InSeekHeader->byte_line_slope[1];
    int64_t unbiased_error = biased_error + InSeekHeader->byte_bias;
    uint64_t result = (predicted + unbiased_error);
    return result;
}

static size_t RadASeekTableLookupInternal(const RadAFileHeader* InFileHeader, const RadASeekTableHeader* InSeekHeader, const uint8_t* InSeekTableData, size_t InSeekTableSizeBytes, uint64_t InTargetFrame, size_t* OutFrameAtLocation, size_t* OutFrameBlockSize)
{
    if (InTargetFrame >= InFileHeader->frame_count)
        InTargetFrame = InFileHeader->frame_count - 1;

    size_t seek_table_qwords = InSeekTableSizeBytes / 8;

    // binary search
    {
        uint32_t block_index = 0;
        uint32_t count = InFileHeader->seek_table_entry_count;
        while (count > 0)
        {
            uint32_t examine = block_index;
            uint32_t step = count / 2;
            examine += step;
            uint64_t sample_for_index = RadAEvaluateSeekTableSamplePosition(InFileHeader, InSeekHeader, InSeekTableData, seek_table_qwords, examine);
            if (!(InTargetFrame < sample_for_index)) // upper_bound
            //if (sample_for_index < InTargetFrame) // lower_bound
            {
                block_index = examine + 1;
                count -= step + 1;
            }
            else
                count = step;
        }

        // We did an upper_bound search which means our result is strictly greater than our
        // search term - so this must be the block _after_ the one we want - so evaluate
        // the one before us and return it.
        if (block_index)
            block_index--;

        if (OutFrameAtLocation)
            *OutFrameAtLocation = RadAEvaluateSeekTableSamplePosition(InFileHeader, InSeekHeader, InSeekTableData, seek_table_qwords, block_index);;

        uint64_t byte_result = RadAEvaluateSeekTableBytePosition(InFileHeader, InSeekHeader, InSeekTableData, seek_table_qwords, block_index);

        if (OutFrameBlockSize)
        {
            // We are somewhere in this seek block, but we don't know exactly where, data wise. Since we want to _guarantee_ that the frame can
            // be reached, the only option is to report the entire seek table entry.
            //
            size_t next_entry_bytes = 0;
            if ((block_index + 1) < InFileHeader->seek_table_entry_count)
                next_entry_bytes = RadAEvaluateSeekTableBytePosition(InFileHeader, InSeekHeader, InSeekTableData, seek_table_qwords, block_index + 1);
            else
                next_entry_bytes = InFileHeader->file_size - RadAGetBytesToOpen(InFileHeader); // we are doing math in seek table space...

            *OutFrameBlockSize = next_entry_bytes - byte_result;
        }

        return byte_result + RadAGetBytesToOpen(InFileHeader);
    }
}


size_t RadASeekTableLookup(const RadAContainer* InContainer, uint64_t InTargetFrame, size_t* OutFrameAtLocation, size_t* OutFrameBlockSize)
{
    if (InContainer == 0 ||
        InContainer->OffsetToSeekHeader == 0)
        return 0;

    const RadAFileHeader* FileHeader = &InContainer->Header;
    if (InTargetFrame >= FileHeader->frame_count)
        InTargetFrame = FileHeader->frame_count - 1;

    const RadASeekTableHeader* SeekHeader = Container_SeekHeader(InContainer);
    const uint8_t* SeekData = Container_SeekTable(InContainer);

    return RadASeekTableLookupInternal(FileHeader, SeekHeader, SeekData, RadAGetSeekTableSizeOnDisk(FileHeader), InTargetFrame, OutFrameAtLocation, OutFrameBlockSize);
}


//-----------------------------------------------------------------------------
//-----------------------------------------------------------------------------
size_t RadADirectSeekTableLookup(const uint8_t* InFileData, size_t InFileDataLenBytes, uint64_t InTargetFrame, size_t* OutFrameAtLocation, size_t* OutFrameBlockSize)
{
    // seek table lookup without opening the file.

    // We need the seek table, which means we definitely need all the header.    
    const RadAFileHeader* FileHeader = RadAGetFileHeader(InFileData, InFileDataLenBytes);
    if (FileHeader == 0)
    	return 0;

    const RadASeekTableHeader* SeekHeader = RadAGetSeekTableHeader(InFileData, InFileDataLenBytes);
    if (SeekHeader == 0)
        return 0;

    uint32_t bytes_to_seek_table = RadAGetOffsetToSeekTable(FileHeader);
    uint32_t seek_table_len = RadAGetSeekTableSizeOnDisk(FileHeader);

    uint32_t bytes_to_evaluate = bytes_to_seek_table + seek_table_len;
    if (InFileDataLenBytes < bytes_to_evaluate)
        return 0; // need the seek table in memory.

    const uint8_t* seek_table_bytes = InFileData + bytes_to_seek_table;
    return RadASeekTableLookupInternal(FileHeader, SeekHeader, seek_table_bytes, seek_table_len, InTargetFrame, OutFrameAtLocation, OutFrameBlockSize);
}



// the most we'll ever need is:
// 6 - rada chunk header to get size
// 1 - sync
// 2 - multistream bytes
// = 9
constexpr uint32_t input_reservoir_min_size = 9;

// out_input_reservoir_needed is only valid when returning INPUT_BUFFER_VALID
RadAExamineBlockResult RadAExamineBlock(const RadAContainer* rada, const uint8_t* input_reservoir, size_t input_reservoir_len, uint32_t* out_input_reservoir_needed)
{
    size_t original_input_reservoir_len = input_reservoir_len;
    if (input_reservoir_len < input_reservoir_min_size)
    {
        if (out_input_reservoir_needed)
            *out_input_reservoir_needed = input_reservoir_min_size;
        return RadAExamineBlockResult::Incomplete; // no space for header.
    }

    // Check for file header
    uint32_t header_check;
    memcpy(&header_check, input_reservoir, 4);
    if (header_check == 'RADA')
    {
        // We are at the beginning of the file - we have 8 bytes so we know we have
        // access to the amount we need in the header.
        constexpr size_t offset_to_bytes_for_first_decode = offsetof(RadAFileHeader, bytes_for_first_decode);

        static_assert(offset_to_bytes_for_first_decode == 4, "incorrect offset in RadAFileHeader!");
        
        uint32_t bytes_for_first_decode;
        memcpy(&bytes_for_first_decode, input_reservoir + offset_to_bytes_for_first_decode, 4);

        if (out_input_reservoir_needed)
            *out_input_reservoir_needed = bytes_for_first_decode;

        if (input_reservoir_len < bytes_for_first_decode)
            return RadAExamineBlockResult::Incomplete;
        return RadAExamineBlockResult::Valid;
    }

    if (input_reservoir[0] != RADA_SYNC_BYTE)
        return RadAExamineBlockResult::Invalid;

    input_reservoir++;
    input_reservoir_len--;

    uint16_t multi_stream_bytes = 0;
    if (rada->Header.channels > 2)
    {
        // multistream - we have 2 bytes of extra size info
        memcpy(&multi_stream_bytes, input_reservoir, 2);
        input_reservoir += 2;
        input_reservoir_len -= 2;
    }

    // get the first stream's chunk length
    int chunk_length_result = RadAudioDecoderGetChunkLength(Container_Decoders(rada)[0], input_reservoir, input_reservoir_len);
    if (chunk_length_result < 0)
    {
        if (chunk_length_result == RADAUDIO_DECODER_INCOMPLETE_DATA)
        {
            // this should only happen when we don't get the function enough data.. which should never happen.
            // so we sanity this by asking for double the min.
            if (out_input_reservoir_needed)
                *out_input_reservoir_needed = input_reservoir_min_size * 2;
            return RadAExamineBlockResult::Incomplete;
        }
        return RadAExamineBlockResult::Invalid;
    }

    uint32_t total_frame_size = multi_stream_bytes + chunk_length_result;

    if (total_frame_size > rada->Header.max_block_size)
        return RadAExamineBlockResult::Invalid;

    if (out_input_reservoir_needed)
        *out_input_reservoir_needed = (uint32_t)(total_frame_size + (original_input_reservoir_len - input_reservoir_len));
    
    if (input_reservoir_len < total_frame_size)
        return RadAExamineBlockResult::Incomplete;

    if (input_reservoir_len > total_frame_size)
    {
        // if we have more data, check for the next sync.
        if (input_reservoir[total_frame_size] != RADA_SYNC_BYTE)
            return RadAExamineBlockResult::Invalid;
    }
    return RadAExamineBlockResult::Valid;
}

void RadANotifySeek(RadAContainer* rada)
{
    uint8_t stream_count = Container_StreamCount(rada);
    for (uint8_t stream_index = 0; stream_index < stream_count; stream_index++)
    {
        RadAudioDecoderDidSeek(Container_Decoders(rada)[stream_index]);
    }

}

#define RadADecodeBlock_Error -2
#define RadADecodeBlock_Done -1

//-----------------------------------------------------------------------------
//-----------------------------------------------------------------------------
int16_t RadADecodeBlock(RadAContainer* rada,
    const uint8_t* input_reservoir, size_t input_reservoir_len, 
    float* output_reservoir, size_t output_reservoir_stride_in_floats, 
    size_t* consumed_bytes)
{
    //
    // This function assumes that the input reseroivr passed decoder checks and is valid
    // with enough space.
    //
    const uint8_t* input_buffer = input_reservoir;
    input_buffer++; // sync bits
    input_reservoir_len--;

    uint8_t stream_count = Container_StreamCount(rada);
    if (stream_count > 1)
    {
        input_buffer+=2; // multistream size
        input_reservoir_len-=2;
    }

    //
    // Get the decompression results.
    //
    float* current_output_reservoir = output_reservoir;
    int16_t decoded_samples = 0;
    for (uint8_t stream_index = 0; stream_index < stream_count; stream_index++)
    {
        float* channel_outputs[2];

        channel_outputs[0] = current_output_reservoir;
        current_output_reservoir += output_reservoir_stride_in_floats;
        channel_outputs[1] = current_output_reservoir;
        current_output_reservoir += output_reservoir_stride_in_floats;

        int chunk_length_result = RadAudioDecoderGetChunkLength(Container_Decoders(rada)[stream_index], input_buffer, input_reservoir_len);
        if ((uint32_t)chunk_length_result > input_reservoir_len)
            return RadADecodeBlock_Error;

        size_t codec_consumed_bytes = 0;
        int got_samples = RadAudioDecoderDecodeChunk(Container_Decoders(rada)[stream_index], input_buffer, input_reservoir_len, &codec_consumed_bytes, channel_outputs, 1024);
        if (got_samples == RADAUDIO_DECODER_AT_EOF)
        {
            *consumed_bytes = 0;
            return RadADecodeBlock_Done;
        }

        if (got_samples < 0)
            return RadADecodeBlock_Error;
        if (decoded_samples && got_samples != decoded_samples)
            return RadADecodeBlock_Error;
        if (decoded_samples > 1024) // max block size is 1024.
            return RadADecodeBlock_Error;
        decoded_samples = (int16_t)got_samples;

        if (input_reservoir_len < codec_consumed_bytes)
            return RadADecodeBlock_Error;

        input_buffer += codec_consumed_bytes;
        input_reservoir_len -= codec_consumed_bytes;
    }

    *consumed_bytes = input_buffer - input_reservoir;
    return decoded_samples;
}

#ifdef RADAUDIO_DECODER_HAS_INTERLEAVING

void  RadAInterleave( int16_t * output, float const ** inputs, uint32_t input_count, uint32_t samples )
{
    RadAudioInterleave( output, inputs, input_count, samples );
}

#endif
