// Copyright Epic Games Tools, LLC. All Rights Reserved.
#include "rrCore.h"

#include "rada_encode.h"
#include "rada_file_header.h"
#include "radaudio_encoder.h"

#include <stdio.h>
#include <string.h>

#define MAX_STREAMS (RADA_MAX_CHANNELS / 2)

namespace {

static_assert(sizeof(size_t) == sizeof(uint64_t), "requires 64 bit size_t"); // we cast between these freely

static void sanity(bool to_test)
{
    if (to_test == false)
        *(volatile int*)0 = 0;
}

typedef void* (allocator_fn_type)(uintptr_t bytes);
typedef void (free_fn_type)(void* ptr);

struct UInt64Array
{
    uint64_t* data;
    size_t count;
    size_t allocated;
    allocator_fn_type* memalloc;
    free_fn_type* memfree;
    void construct(allocator_fn_type* _memalloc, free_fn_type* _memfree) { data = nullptr; count = allocated = 0; memalloc = _memalloc; memfree = _memfree; }
    void destroy() { memfree(data); }
};

static size_t UInt64ArrayLowerBound(UInt64Array* sa, uint64_t search_value)
{
    uint64_t* first = sa->data;
    size_t count = sa->count;
    while (count > 0)
    {
        uint64_t* it = first;
        size_t step = count / 2;
        it += step;
        if (*it < search_value)
        {
            first = it + 1;
            count -= step + 1;
        }
        else
            count = step;
    }
    return first - sa->data;
}

static void UInt64ArrayMakeFit(UInt64Array* sa, size_t count_to_add)
{
    if (sa->count + count_to_add <= sa->allocated)
        return;

    size_t new_allocated = sa->allocated * 2;
    if (new_allocated < count_to_add)
        new_allocated = count_to_add;
    if (new_allocated < 16)
        new_allocated = 16;

    size_t* new_data = (size_t*)sa->memalloc(sizeof(size_t) * new_allocated);
    memcpy(new_data, sa->data, sizeof(size_t)*sa->count);
    sa->memfree(sa->data);
    sa->data = (uint64_t*)new_data;
    sa->allocated = (uint64_t)new_allocated;
}

static void UInt64ArrayAdd(UInt64Array* sa, size_t to_add)
{
    UInt64ArrayMakeFit(sa, 1);
    sa->data[sa->count] = to_add;
    sa->count++;
}

static void UInt64ArrayFree(UInt64Array* sa)
{
    sa->memfree(sa->data);
    sa->data = 0; sa->count = 0; sa->allocated = 0;
}

struct MemBufferEntry
{
    size_t allocated;
    size_t count;
    MemBufferEntry* next;
    uint8_t bytes[1];

    size_t extra_needed(size_t request_size) const { return (count + request_size <= allocated) ? 0 : (request_size - (allocated - count)); }
};

struct MemBuffer
{
    allocator_fn_type* memalloc;
    free_fn_type* memfree;
    MemBufferEntry* head;
    MemBufferEntry* tail;
    size_t total_bytes;

    void construct(allocator_fn_type* _memalloc, free_fn_type* _memfree) { head = tail = nullptr; total_bytes = 0; memalloc = _memalloc; memfree = _memfree; }
    void destroy()
    {
        MemBufferEntry* entry = head;
        while (entry)
        {
            MemBufferEntry* next = entry->next;
            memfree(entry);
            entry = next;
        }
        total_bytes = 0;
        head = 0;
        tail = 0;
    }
};

static const size_t membuffer_default_buffer_size = 64 << 10;

//-----------------------------------------------------------------------------
//-----------------------------------------------------------------------------
static MemBufferEntry* MemBufferMakeFit(MemBuffer* mem, size_t amount_needed)
{
    if (mem->tail == nullptr)
    {
        // first buffer.
        size_t allocate_amount = membuffer_default_buffer_size;
        if (amount_needed > allocate_amount)
            amount_needed = allocate_amount;

        // allocate the buffer to hold the entire thing.
        MemBufferEntry* entry = (MemBufferEntry*)mem->memalloc(sizeof(MemBufferEntry) + allocate_amount);
        entry->allocated = allocate_amount;
        entry->count = 0;
        entry->next = 0;
        mem->head = entry;
        mem->tail = entry;
        return entry;
    }

    // buffer exists - does it fit in current allocation?
    MemBufferEntry* existing = mem->tail;
    size_t extra_needed = existing->extra_needed(amount_needed);
    if (extra_needed == 0)
        return existing;

    // doesn't fit, need another one.
    // make sure the next buffer is bigger than min size.
    if (extra_needed < membuffer_default_buffer_size)
        extra_needed = membuffer_default_buffer_size;

    MemBufferEntry* new_entry = (MemBufferEntry*)mem->memalloc(sizeof(MemBufferEntry) + extra_needed);
    new_entry->allocated = extra_needed;
    new_entry->count = 0;
    new_entry->next = 0;
    mem->tail->next = new_entry;
    mem->tail = new_entry;
    
    // return the first buffer that has space
    if (existing->allocated == existing->count)
        return new_entry; // no space, it goes in the new entry;
    return existing; // at least some bytes can go in the existing.
}

//-----------------------------------------------------------------------------
//-----------------------------------------------------------------------------
static void MemBufferAdd(MemBuffer* mem, void* data, size_t data_len)
{
    mem->total_bytes += data_len;

    MemBufferEntry* dest = MemBufferMakeFit(mem, data_len);
    size_t extra_needed = dest->extra_needed(data_len);
    if (extra_needed == 0)
    {
        // single memcpy
        memcpy(dest->bytes + dest->count, data, data_len);
        dest->count += data_len;
        return;
    }

    // split
    size_t bytes_first = data_len - extra_needed;
    memcpy(dest->bytes + dest->count, data, bytes_first);
    dest->count += bytes_first;
    dest = dest->next;
    memcpy(dest->bytes, (uint8_t*)data + bytes_first, extra_needed);
    dest->count += extra_needed;
}


//-----------------------------------------------------------------------------
//-----------------------------------------------------------------------------
static MemBufferEntry* MemBufferCopy(MemBuffer* dest, MemBuffer* source, MemBufferEntry* source_entry, size_t start_offset, size_t byte_count, size_t* end_offset)
{
    // copy starting at source_entry->data + start_offset.
    while (byte_count)
    {
        size_t available_bytes = source_entry->count - start_offset;
        sanity(available_bytes != 0);
        if (available_bytes == 0)
            return nullptr;
        size_t from_this = byte_count;
        if (available_bytes < from_this)
            from_this = available_bytes;

        MemBufferAdd(dest, source_entry->bytes + start_offset, from_this);

        byte_count -= from_this;
        start_offset += from_this;

        if (start_offset == source_entry->count)
        {
            source_entry = source_entry->next;
            start_offset = 0;
        }
    }

    *end_offset = start_offset;
    return source_entry;
}


//-----------------------------------------------------------------------------
//-----------------------------------------------------------------------------
static uint8_t* MemBufferCanCopyDirect(MemBuffer* mem, size_t data_len)
{
    MemBufferEntry* dest = MemBufferMakeFit(mem, data_len);
    size_t extra_needed = dest->extra_needed(data_len);
    if (extra_needed == 0)
        return dest->bytes + dest->count;
    return nullptr;
}

//-----------------------------------------------------------------------------
//-----------------------------------------------------------------------------
static void MemBufferCommitDirect(MemBuffer* mem, size_t data_len)
{
    // if we were direct, then by def its the last entry. undefined if commit more than requested...
    MemBufferEntry* dest = mem->tail;
    dest->count += data_len;
    mem->total_bytes += data_len;
}


//-----------------------------------------------------------------------------
//-----------------------------------------------------------------------------
static uint8_t* MemBufferWriteBuffer(MemBuffer* mem, uint8_t* buffer)
{
    uint8_t* write_cursor = buffer;
    MemBufferEntry* entry = mem->head;
    while (entry)
    {
        MemBufferEntry* next = entry->next;
        memcpy(write_cursor, entry->bytes, entry->count);
        write_cursor += entry->count;
        entry = next;
    }
    return write_cursor;
}

//-----------------------------------------------------------------------------
//-----------------------------------------------------------------------------
static bool SeekTableBufferTrim(UInt64Array* samples, UInt64Array* offsets, size_t max_entry_count)
{
    sanity(offsets->count == samples->count);

    while (samples->count > max_entry_count)
    {
        // collapse pairs.
        size_t read_index = 0;
        size_t write_index = 0;
        while (read_index < samples->count)
        {
            { // samples
                uint64_t total_samples = samples->data[read_index];
                if (read_index + 1 < samples->count)
                {
                    total_samples += samples->data[read_index + 1];
                }

                if (total_samples > 65535)
                {
                    // we can't fit the offset in the table - fail to trim, this file is too big
                    // with the given max seek table size.
                    return false;
                }

                samples->data[write_index] = total_samples;
            }
            { // offsets
                uint64_t total_offsets = offsets->data[read_index];
                if (read_index + 1 < offsets->count)
                {
                    total_offsets += offsets->data[read_index + 1];
                }

                if (total_offsets > 65535)
                {
                    // we can't fit the offset in the table - fail to trim, this file is too big
                    // with the given max seek table size.
                    return false;
                }

                offsets->data[write_index] = total_offsets;
            }
            write_index++;
            read_index += 2;
        }

        samples->count = write_index;
        offsets->count = write_index;
    }

    return true;
}


struct BitContainer
{
    UInt64Array container;
    size_t total_bit_count;
    size_t working_bits;
    uint32_t working_bit_count;

    void construct(allocator_fn_type* allocator_fn, free_fn_type* free_fn)
    {
        container.construct(allocator_fn, free_fn);
        working_bits = 0;
        working_bit_count = 0;
        total_bit_count = 0;
    }
    void destroy()
    {
        container.destroy();
    }
};

static uint64_t BitContainerExtract(BitContainer* bit_container, uint64_t bit_position, uint32_t bit_count)
{
    // extract the bits.
    uint64_t base_bit = bit_position;
    uint64_t base_word = base_bit / 64;
    uint64_t base_offset = base_bit - base_word * 64;

    sanity(base_word < bit_container->container.count);
    uint64_t output = 0;

    uint64_t bits_avail = 64 - base_offset;

    uint64_t bits_from_first = bit_count;
    if (bits_from_first > bits_avail)
        bits_from_first = bits_avail;

    uint64_t bits_to_clear = 64 - (base_offset + bits_from_first);
    output = (bit_container->container.data[base_word] << bits_to_clear) >> (bits_to_clear + base_offset);
    
    if (bits_from_first != bit_count)
    {
        // need some from the next word
        base_word++;
        base_offset = 0;
        bits_to_clear = 64 - (bit_count - bits_from_first);

        output |= ((bit_container->container.data[base_word] << bits_to_clear) >> bits_to_clear) << bits_from_first;
    }
    return output;
}

static void BitContainerPut(BitContainer* bit_container, uint64_t bits, uint32_t bit_count_to_add)
{
    bit_container->total_bit_count += bit_count_to_add;

    // clear any bits we aren't adding.
    uint64_t sanitized_bits = (bits << (64 - bit_count_to_add)) >> (64 - bit_count_to_add);

    // add the bits in at our position... any bits off the "end" are lost, we handle later.
    bit_container->working_bits |= sanitized_bits << bit_container->working_bit_count;

    // update the position
    bit_container->working_bit_count += bit_count_to_add;

    // did we not fit/finish the entry?
    if (bit_container->working_bit_count >= 64)
    { 
        // flush the working bits.
        UInt64ArrayAdd(&bit_container->container, bit_container->working_bits);

        // reset our working state.
        bit_container->working_bit_count -= 64;
        bit_container->working_bits = 0;

        // if there are some left over, we need to add them to our new working.
        if (bit_container->working_bit_count)
        { 
            bit_container->working_bits = sanitized_bits >> (bit_count_to_add - bit_container->working_bit_count);
        } 
    } 
}

// ensure all working bits are in the container.
static void BitContainerFlush(BitContainer* bit_container)
{
    uint32_t bits_remaining_in_working = (64 - bit_container->working_bit_count) & (64 - 1);
    if (bits_remaining_in_working)
    {
        BitContainerPut(bit_container, 0, bits_remaining_in_working);
    }
}


// Scope guard to set up FP state as desired and reset it on exit
struct FPStateScope
{
    U32 saved_state;

    FPStateScope();
    ~FPStateScope();
};

#if defined(__RADSSE2__)
    FPStateScope::FPStateScope()
    {
        saved_state = _mm_getcsr();
        // Set up our expected FP state: no exception flags set,
        // all exceptions masked (suppressed), round to nearest,
        // flush to zero and denormals are zero both off.
        _mm_setcsr(_MM_MASK_MASK /* all exceptions masked */ | _MM_ROUND_NEAREST | _MM_FLUSH_ZERO_OFF);
    }

    FPStateScope::~FPStateScope()
    {
        _mm_setcsr(saved_state);
    }

#elif defined(__RADARM__) && defined(__RAD64__)

    #ifdef _MSC_VER

        #include <intrin.h>
        static U32 read_fpcr()
        {
            // The system register R/W instructions use 64-bit GPRs, but the
            // architectural FPCR is 32b
            return (U32)_ReadStatusReg(ARM64_FPCR);
        }

        static void write_fpcr(U32 state)
        {
            _WriteStatusReg(ARM64_FPCR, state);
        }

    #elif defined(__clang__) || defined(__GNUC__)

        static U32 read_fpcr()
        {
            // The system register R/W instructions use 64-bit GPRs, but the
            // architectural FPCR is 32b
            U64 value;
            __asm__ volatile("mrs %0, fpcr" : "=r"(value));
            return (U32)value;
        }

        static void write_fpcr(U32 state)
        {
            U64 state64 = state;
            __asm__ volatile("msr fpcr, %0" : : "r"(state64));
        }

    #else

        #error compiler? Not clang or msvc

    #endif

    FPStateScope::FPStateScope()
    {
        saved_state = read_fpcr();

        // IEEE compliant mode in FPCR is just all-0
        write_fpcr(0);
    }

    FPStateScope::~FPStateScope()
    {
        write_fpcr(saved_state);
    }

#else // neither SSE2 nor ARM64

    FPStateScope::FPStateScope()
        : saved_state(0)
    {
    }

    FPStateScope::~FPStateScope()
    {
    }

#endif


static uint8_t GetBitsToShift(uint64_t val) // count trailing zeros, pulled from rrbits.
{
    // ctz4(x)
    static uint8_t const lut[16] = { 4,0,1,0, 2,0,1,0, 3,0,1,0, 2,0,1,0 };

    uint8_t nz = 0;
    if ((val & 0xffffffff) == 0) { nz += 32; val >>= 32; }
    if ((val & 0x0000ffff) == 0) { nz += 16; val >>= 16; }
    if ((val & 0x000000ff) == 0) { nz += 8; val >>= 8; }
    if ((val & 0x0000000f) == 0) { nz += 4; val >>= 4; }
    return nz + lut[val & 0xf];
}


static uint8_t GetBitsRequiredToSend(uint64_t value)
{
    uint8_t bits = 0;

    if (value & 0xffffffff00000000ULL)  { bits += 32; value >>= 32; }
    if (value & 0xffff0000U)            { bits += 16; value >>= 16; }
    if (value & 0xff00)                 { bits += 8; value >>= 8; }
    if (value & 0xf0)                   { bits += 4; value >>= 4; }
    static U8 table[16] = { 
     // 0  1  10  11  100  101  110  111
        0, 1, 2, 2, 3, 3, 3, 3, 
     // 1000  1001  1010  1011  1100  1101  1110  1111
        4, 4, 4, 4, 4, 4, 4, 4 };

    return bits + table[value & 0xf];

}

static size_t align4(size_t in) { return (in + 3) & ~3; }
static size_t align64(size_t in) { return (in + 63) & ~63; }

struct LineEquation
{
    int64_t slope_numerator, slope_denomenator;
    int64_t intercept;
    int64_t bias;
};

static void LinearRegressionDeltasToBits(UInt64Array* resolved_deltas, 
    uint8_t delta_shift, uint64_t* deltas, size_t count, 
    LineEquation* out_regression, uint8_t* out_bits_per_entry)
{
    resolved_deltas->count = 0;

    if (count == 0)
    {
        out_regression->intercept = 0;
        out_regression->slope_numerator = 0;
        out_regression->slope_denomenator = 1;
        out_regression->bias = 0;
        *out_bits_per_entry = 0;
        return;
    }

    // We are delivered deltas, but we want to regress on the resolved positions, so sum in to 
    // our temp space
    uint64_t delta_sum = 0;
    uint64_t resolved_delta_sum = 0;
    for (size_t index = 0; index < count; index++)
    {
        UInt64ArrayAdd(resolved_deltas, delta_sum);

        resolved_delta_sum += resolved_deltas->data[index];
        delta_sum += deltas[index] >> delta_shift;
    }

    int64_t resolved_delta_avg = resolved_delta_sum / count;
    int64_t index_avg = count / 2;

    // beta = sum( (xi - xavg) ( yi - yavg) ) / sum ( (xi - xavg) * (xi -xavg) )
    int64_t beta_numerator = 0;
    int64_t beta_denomenator = 0;
    for (size_t index = 0; index < count; index++)
    {
        beta_numerator += (index - index_avg) * (resolved_deltas->data[index] - resolved_delta_avg);
        beta_denomenator += (index - index_avg) * (index - index_avg);
    }

    // y = a + Bx
    // a = resolved_delta_avg - (beta_numerator / beta_denomenator)*index_avg;
    // b = (beta_numerator / beta_denomenator)
    // y = resolved_delta_avg - (beta_numerator / beta_denomenator)*index_avg + (beta_numerator / beta_denomenator)*X
    // y = resolved_delta_avg + (beta_numerator / beta_denomenator)*X - (beta_numerator / beta_denomenator)*index_avg 
    // y = resolved_delta_avg + (beta_numerator / beta_denomenator)(X - index_avg)
    // y = resolved_delta_avg + (beta_numerator * (X - index_avg)) / beta_denomenator

    // To send bits we need to bias in to positive, so find the most-negative error so we can offset it.
    int64_t bias = resolved_deltas->data[0] - (resolved_delta_avg + (beta_numerator * (0 - index_avg)) / beta_denomenator);
    for (size_t index = 1; index < count; index++)
    {
        int64_t estimated_seek_table = resolved_delta_avg + (beta_numerator * ((int64_t)index - index_avg)) / beta_denomenator;
        int64_t error = resolved_deltas->data[index] - estimated_seek_table;

        if (error < bias)
            bias = error;
    }

    // Find how many bits we need to encode the residuals
    uint8_t max_bits_required = 0;
    for (size_t index = 0; index < count; index++)
    {
        int64_t estimated_seek_table = resolved_delta_avg + (beta_numerator * ((int64_t)index - index_avg)) / beta_denomenator;
        int64_t error = resolved_deltas->data[index] - estimated_seek_table;
        int64_t biased_error = error - bias;

        uint8_t bits_required = GetBitsRequiredToSend(biased_error);

        if (bits_required > max_bits_required)
            max_bits_required = bits_required;
    }


    out_regression->intercept = resolved_delta_avg;
    out_regression->slope_numerator = beta_numerator;
    out_regression->slope_denomenator = beta_denomenator;
    out_regression->bias = bias;
    *out_bits_per_entry = max_bits_required;
}

struct SeekTableInfo
{
    LineEquation sample_line;
    LineEquation byte_line;
    uint8_t sample_bits_per_entry, byte_bits_per_entry;

    RadASeekTableHeader get_header()
    {
        RadASeekTableHeader header = {};
        header.byte_bias = byte_line.bias;
        header.byte_intercept = byte_line.intercept;
        header.byte_line_slope[0] = byte_line.slope_numerator;
        header.byte_line_slope[1] = byte_line.slope_denomenator;

        header.sample_bias = sample_line.bias;
        header.sample_intercept = sample_line.intercept;
        header.sample_line_slope[0] = sample_line.slope_numerator;
        header.sample_line_slope[1] = sample_line.slope_denomenator;
        return header;
    }


    void construct(allocator_fn_type* _memalloc, free_fn_type* _memfree)
    { 
        memset(this, 0, sizeof(*this));
    }
    void destroy()
    {
    }
};

struct stream_info
{
    free_fn_type* free_fn;
    float* samples;
    uint8_t channels; // 1 or 2.
    uint8_t channel_offset; // where our chans starts in the source streams
    uint8_t header_size;

    radaudio_encoder encoder;
    unsigned char encoder_header_buffer[128];
    radaudio_blocktype first_block_type;

    uint64_t current_offset_frame;
    size_t consumed_frames_last_block;

    UInt64Array bytes_in_block;
    UInt64Array samples_in_block;
    MemBuffer encoded_data;

    uint64_t block_count;
    MemBufferEntry* writing_entry;
    uint64_t writing_entry_offset;

    void construct(allocator_fn_type* allocator_fn, free_fn_type* in_free_fn)
    {
        free_fn = in_free_fn;
        bytes_in_block.construct(allocator_fn, in_free_fn);
        samples_in_block.construct(allocator_fn, in_free_fn);
        encoded_data.construct(allocator_fn, in_free_fn);
        samples = nullptr;
        channels = 0;
        channel_offset = 0;
        current_offset_frame = 0;
        writing_entry = nullptr;
        writing_entry_offset = 0;
        block_count = 0;
        header_size = 0;
        first_block_type = (radaudio_blocktype)0;
        consumed_frames_last_block = 0;
    }
    void destroy()
    {
        bytes_in_block.destroy();
        samples_in_block.destroy();
        encoded_data.destroy();
        free_fn(samples);
    }
};

} // end anon namespace


//-----------------------------------------------------------------------------
//-----------------------------------------------------------------------------
uint8_t EncodeRadAFile(
    void* WavData, uint64_t WavDataLen, uint32_t WavRate, uint8_t WavChannels, 
    uint8_t Quality, uint8_t SeamlessLooping, uint8_t GenerateSeekTable, uint16_t SeekTableMaxEntries,
    RadACompressAllocFnType* MemAlloc, RadACompressFreeFnType* MemFree,
    void** OutData, uint64_t* OutDataLen)
{
    FPStateScope FixFloatingPoint;

    if (WavChannels == 0 || WavChannels > (MAX_STREAMS*2))
        return RADA_COMPRESS_ERROR_CHANS;
    if (Quality > 9 || Quality < 1)
        return RADA_COMPRESS_ERROR_QUALITY;
    if (MemAlloc == nullptr ||
        MemFree == nullptr)
        return RADA_COMPRESS_ERROR_ALLOCATORS;
    if (OutData == nullptr ||
        OutDataLen == nullptr)
        return RADA_COMPRESS_ERROR_OUTPUT;
    if (GenerateSeekTable && SeekTableMaxEntries < 2)
        return RADA_COMPRESS_ERROR_SEEKTABLE;

    {
        uint32_t rate_index = 0;
        for (; rate_index < sizeof(RADA_VALID_RATES) / sizeof(RADA_VALID_RATES[0]); rate_index++)
        {
            if (RADA_VALID_RATES[rate_index] == WavRate)
                break;
        }
        if (rate_index == sizeof(RADA_VALID_RATES) / sizeof(RADA_VALID_RATES[0]))
            return RADA_COMPRESS_ERROR_RATE;
    }

    //
    // Deinterlace the input.
    //
    uint64_t SamplesPerChannel = WavDataLen / (sizeof(short) * WavChannels);
    uint8_t StreamCount = (WavChannels / 2) + (WavChannels & 1);
    if (SamplesPerChannel == 0)
        return RADA_COMPRESS_ERROR_SAMPLES;
    if (StreamCount > MAX_STREAMS)
        return RADA_COMPRESS_ERROR_CHANS;

    //
    // Input wav data needs to be in float and in stereo pairs.
    //
    

    stream_info* streams = (stream_info*)MemAlloc(sizeof(stream_info) * StreamCount);

    for (S32 i = 0; i < StreamCount; i++)
    {
        streams[i].construct(MemAlloc, MemFree);

        if (i)
            streams[i].channel_offset = streams[i-1].channel_offset + streams[i-1].channels;
        streams[i].channels = (WavChannels - streams[i].channel_offset) > 1 ? 2 : 1;
        streams[i].samples = (float*)MemAlloc(sizeof(float) * SamplesPerChannel * streams[i].channels);

        int16_t* input_samples = (int16_t*)WavData;

        if (streams[i].channels == 2)
        {
            for (uint64_t sample = 0; sample < SamplesPerChannel; sample++)
            {
                streams[i].samples[2*sample + 0] = input_samples[WavChannels*sample + streams[i].channel_offset + 0] / 32768.0f;
                streams[i].samples[2*sample + 1] = input_samples[WavChannels*sample + streams[i].channel_offset + 1] / 32768.0f;
            }
        }
        else
        {
            for (uint64_t sample = 0; sample < SamplesPerChannel; sample++)
            {
                streams[i].samples[sample] = input_samples[WavChannels*sample + streams[i].channel_offset + 0] / 32768.0f;
            }
        }
    }

    size_t encoder_total_size = 0;
    for (S32 i = 0; i < StreamCount; i++)
    {
        size_t result = radaudio_encode_create(
            &streams[i].encoder, streams[i].encoder_header_buffer, streams[i].channels, 
            WavRate, Quality, SeamlessLooping ? RADAUDIO_ENC_FLAG_improve_seamless_loop : 0);
        if (result == 0)
        {
            return RADA_COMPRESS_ERROR_ENCODER;
        }
        encoder_total_size += align4(result);
        sanity(result <= 255);
        streams[i].header_size = (uint8_t)align4(result);
    }

    uint32_t block_header_bytes = 1;
    if (StreamCount > 1)
        block_header_bytes += 2;

    radaudio_blocktype first_block_type = radaudio_determine_preferred_first_block_length(&streams[0].encoder, streams[0].samples, SamplesPerChannel);;
    for (uint8_t StreamIndex = 1; StreamIndex < StreamCount; StreamIndex++)
    {
        radaudio_blocktype check_block_type = radaudio_determine_preferred_first_block_length(&streams[StreamIndex].encoder, streams[StreamIndex].samples, SamplesPerChannel);;
        if (check_block_type == RADAUDIO_BLOCKTYPE_short)
            first_block_type = RADAUDIO_BLOCKTYPE_short;
    }
    for (uint8_t StreamIndex = 0; StreamIndex < StreamCount; StreamIndex++)
        streams[StreamIndex].first_block_type = first_block_type;

    // track how much data we'll need to read in to start getting audio out.
    size_t bytes_for_inital_data = 0;
    uint32_t max_compressed_size = 0;
    for (;;)
    {
        bool all_done = false;

        // Determine the block size we need to use. We need to keep all streams in sync, so we check all
        // streams and if any want to be short, then we are short, otherwise we are long.
        radaudio_blocktype block_type_for_all_streams = RADAUDIO_BLOCKTYPE_long;

        for (uint8_t StreamIndex = 0; StreamIndex < StreamCount; StreamIndex++)
        {
            stream_info* stream = streams + StreamIndex;
            radaudio_blocktype block_type = (radaudio_blocktype)radaudio_determine_preferred_next_block_length(&stream->encoder, stream->first_block_type, stream->samples, SamplesPerChannel, stream->current_offset_frame);
            if (block_type == RADAUDIO_BLOCKTYPE_short)
                block_type_for_all_streams = RADAUDIO_BLOCKTYPE_short;
        }

        uint32_t compressed_size_sum = 0;

        for (uint8_t StreamIndex = 0; StreamIndex < StreamCount; StreamIndex++)
        {
            stream_info* stream = streams + StreamIndex;
            uint8_t encoded_buffer[MAX_ENCODED_BLOCK_SIZE];

            radaudio_encode_info encode_info = {};
            encode_info.force_first_blocktype = first_block_type;
            encode_info.force_next_blocktype = block_type_for_all_streams;

            // If desired for looping, set up the padding so that we get seamless across the whole file.
            if (SeamlessLooping)
            {
                if (SamplesPerChannel < 2048)
                {
                    // it's so close we just always provide the whole thing
                    encode_info.padding = stream->samples;
                    encode_info.padding_len = SamplesPerChannel;
                }
                else if (stream->current_offset_frame < 1024)
                {
                    // We're at the beginning, so we want the padding to be the end of the stream.
                    encode_info.padding = stream->samples + SamplesPerChannel - 2048*WavChannels;
                    encode_info.padding_len = 2048;
                }
                else // just always provide the beginning, it won't use it if it doesn't need it.
                {
                    encode_info.padding = stream->samples;
                    encode_info.padding_len = SamplesPerChannel;
                }
            }

            uint8_t* contiguous_dest = MemBufferCanCopyDirect(&stream->encoded_data, MAX_ENCODED_BLOCK_SIZE);
            uint8_t* encode_dest = encoded_buffer;
            if (contiguous_dest)
                encode_dest = contiguous_dest;

            size_t starting_offset_frame = stream->current_offset_frame;
            int32_t encode_result = radaudio_encode_block_ext(&stream->encoder,
                stream->samples,
                SamplesPerChannel,
                (size_t*)&stream->current_offset_frame,
                encode_dest, 
                MAX_ENCODED_BLOCK_SIZE,
                &encode_info);

            if (encode_result == RADAUDIOENC_AT_EOF)
            {
                // done.
                all_done = true;
                break;
            }
            else if (encode_result <= 0)
            {
                // misc error
                all_done = true;
                break;
            }

            if (contiguous_dest)
                MemBufferCommitDirect(&stream->encoded_data, encode_result);
            else
                MemBufferAdd(&stream->encoded_data, encoded_buffer, encode_result);

            compressed_size_sum += encode_result;

            // we need 2 decodes to start getting data.
            if (stream->block_count <= 1)
                bytes_for_inital_data += encode_result; 

            UInt64ArrayAdd(&stream->bytes_in_block, encode_result);

            size_t produced_samples = stream->consumed_frames_last_block;
            UInt64ArrayAdd(&stream->samples_in_block, produced_samples);

            stream->consumed_frames_last_block = stream->current_offset_frame - starting_offset_frame;
            stream->block_count++;
        }

        if (compressed_size_sum > max_compressed_size)
            max_compressed_size = compressed_size_sum;

        if (all_done)
            break;
    }

    // since everything is the same size, we should have the same block count.
    for (uint8_t stream_index = 1; stream_index < StreamCount; stream_index++)
    {
        sanity(streams[stream_index].block_count == streams[0].block_count);
    }

    //
    // Create an interleaved output of the encoded streams. The streams will
    // be different sized blocks, but we need to ensure that when the decoder
    // runs a stream out, that is the next block ready to decode.
    //

    // for encoding, just add in whichever one is farthest behind on the output
    // sample.

    MemBuffer encoded_stream;
    encoded_stream.construct(MemAlloc, MemFree);

    for (uint8_t stream = 0; stream < StreamCount; stream++)
    {
        streams[stream].writing_entry = streams[stream].encoded_data.head;
    }

    
    for (size_t current_block_index = 0; current_block_index < streams[0].block_count; current_block_index++)
    {
        // Get the block size for each stream. We can get the first stream from the bitstream
        // itself, but we need to be able to get the total chunk size in a small amount of data,
        // so for multistream files, we have a bit of a bigger header.

        uint64_t multi_stream_bytes = 0;
        for (uint8_t stream_index = 1; stream_index < StreamCount; stream_index++)
        {
            stream_info* stream = streams + stream_index;
            
            uint64_t block_bytes = stream->bytes_in_block.data[current_block_index];
            sanity(block_bytes < 4096);

            multi_stream_bytes += block_bytes;
        }

        uint8_t block_header = RADA_SYNC_BYTE;
        MemBufferAdd(&encoded_stream, &block_header, 1);

        if (StreamCount > 1)
        {
            sanity(multi_stream_bytes < 65535);
            uint16_t multi_stream_bytes_16 = (uint16_t)multi_stream_bytes;
            MemBufferAdd(&encoded_stream, &multi_stream_bytes_16, 2);
        }

        for (uint8_t stream_index = 0; stream_index < StreamCount; stream_index++)
        {
            stream_info* stream = streams + stream_index;

            uint64_t block_bytes = stream->bytes_in_block.data[current_block_index];
            sanity(block_bytes < 4096);

            stream->writing_entry = MemBufferCopy(&encoded_stream, &stream->encoded_data, stream->writing_entry, stream->writing_entry_offset, block_bytes, (size_t*)&stream->writing_entry_offset);
        }
    }

    // Seeking requires priming the decoder with a chunk, so we can't actually seek with
    // less than 3 chunks as 2 chunks just means you're starting at the beginning.
    if (streams[0].samples_in_block.count <= 2)
        GenerateSeekTable = 0;

    // \todo... seek table ends up returning the block size for the seek for the entire seek chunk,
    // not just what's needed to decode the _next_ block. To fix this properly requires a whole nother stream
    // which is likely not worth it. So instead we cap the seek table request with the max encoded block size
    
    UInt64Array seek_table_bytes, seek_table_samples;
    seek_table_bytes.construct(MemAlloc, MemFree);
    seek_table_samples.construct(MemAlloc, MemFree);

    // Collapse the seek tables to the correct size.
    uint8_t Result = RADA_COMPRESS_SUCCESS;

    //
    // The seek table for a mDCT codec is a bit more complex because we need to
    // adjust the target such that we decode an extra block. This block emits no samples,
    // so we don't need to touch the samples at all.
    //
    // This functionally just means that the byte position that we emit is
    // current position + [-1, blocks_per_entry-1] rather than [0, blocks_per_entry].
    
    size_t seek_table_entries = 0;
    if (GenerateSeekTable)
    {
        uint64_t frames_per_seek_table_block = SamplesPerChannel / SeekTableMaxEntries;

        // we don't want to emit a _ton_ of seek table entries if we have a bunch of small blocks,
        // so sanity limit the density. We're expecting basically everything to be at 48000, and certainly
        // stuff that isn't we're not likely to be interested in seek density, so this makes about 17ms max density.
        if (frames_per_seek_table_block < 8192)
            frames_per_seek_table_block = 8192;

        uint64_t current_seek_table_frame = 0;

        size_t current_block_index = 1;
        for (; current_block_index < streams[0].samples_in_block.count; )
        {
            // Since we are tracking on an uneven chunk size, we can end up
            // with more than desired by one or two - just cap.
            if (seek_table_bytes.count == SeekTableMaxEntries)
                break;

            //
            // The first entry already compensates for this as there was no way for the encoder
            // to emit anything... so we don't need to do this adjustment for the first block.
            //

            // find how many blocks to catch up to where we should be in frames.
            size_t end = current_block_index + 1;
            uint64_t end_seek_table_frame = current_seek_table_frame;
            for (; end < streams[0].samples_in_block.count; end++)
            {
                end_seek_table_frame += streams[0].samples_in_block.data[end];
                if (end_seek_table_frame > frames_per_seek_table_block * seek_table_bytes.count)
                    break;
            }
            size_t blocks_per_entry = end - current_block_index;

            current_seek_table_frame = end_seek_table_frame;

            uint64_t entry_bytes = 0;
            uint64_t entry_samples = 0;

            size_t entry_block_start_index = current_block_index ? current_block_index - 1 : current_block_index;
            size_t entry_block_end_index = current_block_index + blocks_per_entry - 1;
            if (entry_block_end_index >= streams[0].samples_in_block.count)
                entry_block_end_index = streams[0].samples_in_block.count - 1;

            size_t entry_block_end_index_unadjusted = current_block_index + blocks_per_entry;
            if (entry_block_end_index_unadjusted >= streams[0].samples_in_block.count)
                entry_block_end_index_unadjusted = streams[0].samples_in_block.count - 1;


            for (uint8_t stream_index = 0; stream_index < StreamCount; stream_index++)
            {
                for (size_t entry_block_index = entry_block_start_index; entry_block_index < entry_block_end_index; entry_block_index++)
                    entry_bytes += streams[stream_index].bytes_in_block.data[entry_block_index];
            }
            for (size_t entry_block_index = current_block_index; entry_block_index < entry_block_end_index_unadjusted; entry_block_index++)
                entry_samples += streams[0].samples_in_block.data[entry_block_index];

            entry_bytes += block_header_bytes * (entry_block_end_index - entry_block_start_index);

            current_block_index = entry_block_end_index + 1;
            UInt64ArrayAdd(&seek_table_bytes, entry_bytes);
            UInt64ArrayAdd(&seek_table_samples, entry_samples);
        }

        seek_table_entries = seek_table_samples.count;
    }

    // find the shift for the sample offsets. likely 64, could be larger.
    uint8_t seek_table_samples_shift_bits = GenerateSeekTable ? 255 : 0;
    for (size_t entry_index = 0; entry_index < seek_table_entries; entry_index++)
    {
        uint8_t shift_bits = GetBitsToShift(seek_table_samples.data[entry_index]);
        if (shift_bits < seek_table_samples_shift_bits)
            seek_table_samples_shift_bits = shift_bits;
    }

    // NOTE: Tried to subtract off the min seek table size and it didn't really save much.
    SeekTableInfo seek_table_info;
    seek_table_info.construct(MemAlloc, MemFree);
    BitContainer encoded_residuals;
    encoded_residuals.construct(MemAlloc, MemFree);

    // We do a linear regression on the sample and byte locations of the seek table entries,
    // then encode the residuals.
    {
        UInt64Array sample_positions;
        UInt64Array byte_positions;
        sample_positions.construct(MemAlloc, MemFree);
        byte_positions.construct(MemAlloc, MemFree);

        LinearRegressionDeltasToBits(&sample_positions, seek_table_samples_shift_bits, seek_table_samples.data, seek_table_entries, &seek_table_info.sample_line, &seek_table_info.sample_bits_per_entry);
        LinearRegressionDeltasToBits(&byte_positions, 0, seek_table_bytes.data, seek_table_entries, &seek_table_info.byte_line, &seek_table_info.byte_bits_per_entry);


        // Encode the residuals.    
        int64_t index_avg = seek_table_entries / 2;
        for (size_t index = 0; index < seek_table_entries; index++)
        {
            //printf("%zd - %lld %lld\n", index, sample_positions.data[index] << seek_table_samples_shift_bits, byte_positions.data[index]);

            int64_t predicted_sample = seek_table_info.sample_line.intercept + (seek_table_info.sample_line.slope_numerator * ((int64_t)index - index_avg)) / seek_table_info.sample_line.slope_denomenator;
            int64_t unbiased_sample_error = sample_positions.data[index] - predicted_sample;
            int64_t biased_sample_error = unbiased_sample_error - seek_table_info.sample_line.bias;

            int64_t predicted_byte = seek_table_info.byte_line.intercept + (seek_table_info.byte_line.slope_numerator * ((int64_t)index - index_avg)) / seek_table_info.byte_line.slope_denomenator;
            int64_t unbiased_byte_error = byte_positions.data[index] - predicted_byte;
            int64_t biased_byte_error = unbiased_byte_error - seek_table_info.byte_line.bias;

            BitContainerPut(&encoded_residuals, biased_sample_error, seek_table_info.sample_bits_per_entry);
            BitContainerPut(&encoded_residuals, biased_byte_error, seek_table_info.byte_bits_per_entry);
        }
        BitContainerFlush(&encoded_residuals);

        sample_positions.destroy();
        byte_positions.destroy();    

        if (false)
        {
            // Now, make sure it worked!
            uint64_t current_sample_sum = 0;
            uint64_t current_byte_sum = 0;
            for (int64_t entry_index = 0; entry_index < (int64_t)seek_table_entries; entry_index++)
            {
                int64_t avg_index = seek_table_entries / 2;

                int64_t estimated_seek_table = seek_table_info.sample_line.intercept + (entry_index - avg_index) * seek_table_info.sample_line.slope_numerator / seek_table_info.sample_line.slope_denomenator;

                uint64_t biased_error = BitContainerExtract(&encoded_residuals, (seek_table_info.sample_bits_per_entry + seek_table_info.byte_bits_per_entry) * entry_index, seek_table_info.sample_bits_per_entry);
                int64_t unbiased_error = biased_error + seek_table_info.sample_line.bias;
                uint64_t sample_result = (estimated_seek_table + unbiased_error) << seek_table_samples_shift_bits;

                estimated_seek_table = seek_table_info.byte_line.intercept + (entry_index - avg_index) * seek_table_info.byte_line.slope_numerator / seek_table_info.byte_line.slope_denomenator;

                biased_error = BitContainerExtract(&encoded_residuals, (seek_table_info.sample_bits_per_entry + seek_table_info.byte_bits_per_entry) * entry_index + seek_table_info.sample_bits_per_entry, seek_table_info.byte_bits_per_entry);
                unbiased_error = biased_error + seek_table_info.byte_line.bias;
                uint64_t byte_result = estimated_seek_table + unbiased_error;

                if (byte_result != current_byte_sum ||
                    sample_result != current_sample_sum)
                {
                    // error!
                    printf("miss\n");
                }

                current_byte_sum += seek_table_bytes.data[entry_index];
                current_sample_sum += seek_table_samples.data[entry_index];
            }
        }
    }

    size_t seek_table_header_bytes = GenerateSeekTable ? sizeof(RadASeekTableHeader) : 0;
    size_t packed_seek_table_bytes = encoded_residuals.container.count * sizeof(uint64_t);
    

    size_t bytes_to_data = sizeof(RadAFileHeader);
    bytes_to_data += encoder_total_size;
    bytes_to_data += packed_seek_table_bytes;
    bytes_to_data += seek_table_header_bytes;

    uint64_t output_file_size = bytes_to_data + encoded_stream.total_bytes;

    if (encoder_total_size >= 65535)
        Result = RADA_COMPRESS_ERROR_SIZE;
    if (output_file_size > ~0U)
        Result = RADA_COMPRESS_ERROR_SIZE;
    if (bytes_to_data + bytes_for_inital_data > ~0U)
        Result = RADA_COMPRESS_ERROR_SIZE;

    if (Result == RADA_COMPRESS_SUCCESS)
    {
        RadAFileHeader Header = {};
        Header.tag = 'RADA';
        Header.version = 1;
        Header.channels = (U8)WavChannels;
        Header.rada_header_bytes = (uint16_t)encoder_total_size;
        Header.frame_count = SamplesPerChannel;
        Header.bits_for_seek_table_bytes = seek_table_info.byte_bits_per_entry;
        Header.shift_bits_for_seek_table_samples = seek_table_samples_shift_bits;
        Header.bits_for_seek_table_samples = seek_table_info.sample_bits_per_entry;
        Header.seek_table_entry_count = (uint16_t)seek_table_entries;

        switch (WavRate)
        {
        case 24000: Header.sample_rate = ERadASampleRate::Rate_24000; break;
        case 32000: Header.sample_rate = ERadASampleRate::Rate_32000; break;
        case 44100: Header.sample_rate = ERadASampleRate::Rate_44100; break;
        case 48000: Header.sample_rate = ERadASampleRate::Rate_48000; break;
        default: sanity(0); // encoding will have failed by here.
        }

        sanity(packed_seek_table_bytes == 8 * (align64((seek_table_info.byte_bits_per_entry + seek_table_info.sample_bits_per_entry) * seek_table_entries) / 64));

        // Save off the header byte count + the initial data chunk size.
        Header.bytes_for_first_decode = (uint32_t)(bytes_to_data + bytes_for_inital_data);
        
        //printf("seek table total bytes: %zd\n", packed_seek_table_byte_offsets_bytes + packed_seek_table_sample_offsets_bytes);

        Header.file_size = output_file_size;

        uint32_t block_header_size = 1;
        if (StreamCount)
            block_header_size += 2;
        Header.max_block_size = (uint16_t)(block_header_size + max_compressed_size);

        // [FileHeader]
        // Nx[EncoderHeader]
        // SeekTableSampleDeltas,SeekTableByteOffsets
        // Blocks

        // Create the actual output buffer.
        uint8_t* Output = (uint8_t*)MemAlloc(Header.file_size);
        if (Output == 0)
        {
            Result = RADA_COMPRESS_ERROR_ALLOCATION;
        }
        else
        {
            uint8_t* Current = Output;
            memcpy(Current, &Header, sizeof(RadAFileHeader));
            Current += sizeof(RadAFileHeader);

            if (GenerateSeekTable)
            {
                RadASeekTableHeader seek_table_header = seek_table_info.get_header();
                memcpy(Current, &seek_table_header, sizeof(RadASeekTableHeader));
                Current += sizeof(RadASeekTableHeader);
            }

            for (uint8_t stream_index = 0; stream_index < StreamCount; stream_index++)
            {
                memcpy(Current, streams[stream_index].encoder_header_buffer, streams[stream_index].header_size);
                Current += streams[stream_index].header_size;
            }

            memcpy(Current, encoded_residuals.container.data, packed_seek_table_bytes);
            Current += packed_seek_table_bytes;

            Current = MemBufferWriteBuffer(&encoded_stream, Current);

            sanity(Current == Output + Header.file_size);

            *OutData = Output;
            *OutDataLen = Header.file_size;

            Result = RADA_COMPRESS_SUCCESS;
        }
    }

    encoded_residuals.destroy();
    seek_table_info.destroy();
    seek_table_bytes.destroy();
    seek_table_samples.destroy();

    encoded_stream.destroy();
    for (S32 i = 0; i < StreamCount; i++)
    {
        streams[i].destroy();
    }
    MemFree(streams);


    return Result;
}

const char* RadAErrorString(uint8_t InError)
{
    switch (InError)
    {
    case RADA_COMPRESS_SUCCESS: return "No Error";
    case RADA_COMPRESS_ERROR_CHANS: return "Invalid channel count supplied (max 32 channels)";
    case RADA_COMPRESS_ERROR_SAMPLES: return "No samples provided";
    case RADA_COMPRESS_ERROR_RATE: return "Invalid sample rate provided: only 48khz, 44.1khz, 32khz, and 24khz are allowed.";
    case RADA_COMPRESS_ERROR_QUALITY: return "Invalid quality value specified: my be withing 1 and 9, inclusive.";
    case RADA_COMPRESS_ERROR_ALLOCATORS: return "No allocators provided.";
    case RADA_COMPRESS_ERROR_OUTPUT: return "No output provided.";
    case RADA_COMPRESS_ERROR_SEEKTABLE: return "Invalid seek table size requested.";
    case RADA_COMPRESS_ERROR_SIZE: return "Output file is too big to fit in the container.";
    case RADA_COMPRESS_ERROR_ENCODER: return "Internal encoder error (please report!";
    case RADA_COMPRESS_ERROR_ALLOCATION: return "Allocator returned null pointer.";
    }
    return "Invalid RadA Error Code";
}

uint32_t RadAGetBuildVersion()
{
	return RADA_BUILD_VERSION;
}

#if 0

/* to build this fuzzed:
* "C:\Program Files\LLVM\bin\clang" -v -O0 -o rada_fuzz.exe -g rada_encode.cpp -DRADAUDIO_WRAP=UERA 
    -Ipath\to\radrtl -Ipath\to\radaudio_encoder.h 
    -fsanitize=address,fuzzer path/to/radaudio_encoder_win64.lib
*/
#include <malloc.h>

struct fuzzer_input
{
    uint32_t Rate;
    uint8_t Chans;
    uint8_t Quality;
    uint8_t GenTable;
    uint16_t GenTableMaxEntries;  
};

static int counter = 0;
static void* AllocThunk(uintptr_t ByteCount) { counter++;  return  malloc(ByteCount); }
static void FreeThunk(void* Ptr) { if (Ptr) counter--; free(Ptr); }

extern "C" int LLVMFuzzerTestOneInput(const uint8_t *Data, size_t Size) 
{    
    if (Size <= sizeof(fuzzer_input))
        return 0;
    if (Size > 0xFFFFFFFFU)
        return 0;

    fuzzer_input* input = (fuzzer_input*)Data;

    void* CompressedData = 0;
    uint32_t CompressedLen;

    CompressRadAudio((void*)(Data + sizeof(fuzzer_input)), (uint32_t)Size - sizeof(fuzzer_input),
        input->Rate, input->Chans, input->Quality, input->GenTable, input->GenTableMaxEntries, AllocThunk, FreeThunk,
        &CompressedData, &CompressedLen);

    FreeThunk(CompressedData);

    sanity(counter == 0);

    return 0;
}
#endif
