// Copyright Epic Games Tools, LLC. All Rights Reserved.
// This source file is licensed solely to users who have
// accepted a valid Unreal Engine license agreement 
// (see e.g., https://www.unrealengine.com/eula), and use
// of this source file is governed by such agreement.

#include "cpux86.h"

#ifdef __RADX86__

#ifdef _MSC_VER
	#include <intrin.h>

	#if _MSC_VER >= 1500 // VC++2008 or later
	#define HAVE_CPUIDEX

	static inline U64 xgetbv(U32 xcr)
	{
		return _xgetbv(xcr);
	}
	#else
	static inline U64 xgetbv(U32 xcr)
	{
		return 0;
	}
	#endif

#else
	// GCC/Clang
	#ifdef __RADX64__

	// 64-bit: GCC/Clang won't let us use "=b" constraint on Mac64, and we need to preserve RBX
	// (PIC/PIE base)
	#define __cpuidex(out, leaf_id, subleaf_id)\
		asm("xchgq %%rbx,%q1\n" \
			"cpuid\n" \
			"xchgq %%rbx,%q1\n" \
			: "=a" (out[0]), "=&r" (out[1]), "=c" (out[2]), "=d" (out[3]): "0" (leaf_id), "2"(subleaf_id));

	#else
		#error "64 bit only supported"
	#endif // __RADX64__

	#define HAVE_CPUIDEX
	#define __cpuid(out, leaf_id) __cpuidex(out, leaf_id, 0)

	static inline U64 xgetbv(U32 xcr)
	{
		U32 lo, hi;
		__asm__ __volatile__("xgetbv" : "=a"(lo), "=d"(hi) : "c"(xcr));
		return ((U64)hi << 32) | lo;
	}
#endif // _MSC_VER or not


#ifdef RRX86_CPU_DYNAMIC_DETECT
 

// note : g_rrCPUx86_feature_flags is a global atomic shared variable
//   we play lazy & loose with the thread safety here (atomics are in ext)
//	most likely it's fine
extern "C" U32 g_rrCPUx86_feature_flags = 0;

extern "C" void rrCPUx86_detect()
{
	int cpuid_info[4];
	U32 features = 0;
	U32 max_leaf;
	bool is_amd = false;

	// if we already detected, we're good!
	features = g_rrCPUx86_feature_flags; // atomic or volatile load?
	if (features & RRX86_CPU_INITIALIZED)
		return;

	// Basic CPUID information
	__cpuid(cpuid_info, 0);
	max_leaf = cpuid_info[0];

	// Is it AMD?
	if (cpuid_info[1] == 0x68747541 /* "Auth" */ && cpuid_info[3] == 0x69746e65 /* "enti" */ &&
		cpuid_info[2] == 0x444d4163 /* "cAMD" */)
	{
		is_amd = true;
	}

	// Basic feature flags
	__cpuid(cpuid_info, 1);

	if (cpuid_info[3] & (1u<<26))	features |= RRX86_CPU_SSE2;
	if (cpuid_info[2] & (1u<< 9))	features |= RRX86_CPU_SSSE3;
	if (cpuid_info[2] & (1u<<19))	features |= RRX86_CPU_SSE41;
	if (cpuid_info[2] & (1u<<20))	features |= RRX86_CPU_SSE42;

	// Used to compute other feature flags
	bool has_popcnt = (cpuid_info[2] & (1u<<23)) != 0;
	bool has_osxsave = (cpuid_info[2] & (1u<<27)) != 0;
	bool has_cpu_avx = (cpuid_info[2] & (1u<<28)) != 0;
	bool has_cpu_f16c = (cpuid_info[2] & (1u<<29)) != 0;

	if (has_popcnt) features |= RRX86_CPU_POPCNT;

	if (is_amd)
	{
		U32 family = (cpuid_info[0] >> 8) & 0xf;
		U32 ext_family = (cpuid_info[0] >> 20) & 0xff;

		// Zen aka AMD 17h has family=0xf, ext_family=0x08 (Zen and Zen2 both)
		// Zen3 aka AMD 19h has family=0xf, ext_family=0x0a
		// so just testing for this:
		if (family == 0xf && ext_family >= 0x08)
			features |= RRX86_CPU_AMD_ZEN;
	}

	// Get XCR0, if available, and determine context save bits
	U64 xcr0 = 0;
	if (has_osxsave)
	{
		xcr0 = xgetbv(0);
	}

	// YMM register saving and ZMM/opmask register saving support
	bool has_os_avx_support = (xcr0 & 6) == 6;
	bool has_os_avx512_support = (xcr0 & 0xe6) == 0xe6;

	// AVX support requires both CPU and OS support, and gates some other extensions
	if (has_os_avx_support)
	{
		if (has_cpu_avx)	features |= RRX86_CPU_AVX;
		if (has_cpu_f16c)	features |= RRX86_CPU_F16C;
	}

#ifdef HAVE_CPUIDEX
	if (max_leaf >= 7)
	{
		// "Structured extended feature flags enumeration"
		__cpuidex(cpuid_info, 7, 0);

		// Some (Celeron) Skylakes erroneously report BMI1/BMI2 even though they don't have it.
		// These Celerons also don't have AVX.
		//
		// All CPUs that actually have BMI1/BMI2 (as of this writing, 2016-05-11) have AVX.
		// (The ones we care about, anyway.) So only report BMI1/BMI2 if AVX is present.
		// Also only report AVX or the BMIs if POPCNT is present; all processors I know of
		// have either both or neither, and it's convenient for us to be able to assume
		// that either BMI1/BMI2 or AVX2 implies POPCNT.
		if (has_cpu_avx && has_os_avx_support && has_popcnt)
		{
			if (cpuid_info[1] & (1u<<3))	features |= RRX86_CPU_BMI1;
			if (cpuid_info[1] & (1u<<8))	features |= RRX86_CPU_BMI2;

			// In addition to the above, only report AVX2 if BMI1 (and thus LZCNT/TZCNT)
			// are also reported present; finally VC++ with /arch:AVX2 will emit BMI2
			// instructions for things like variable shifts so we require BMI2 for AVX2
			// as well.
			//
			// In practice this is not a limitation, AVX2 and BMI2 are a package deal on
			// all uArchs I'm aware of.
			const U32 avx2_bits = (1u<<3) /* BMI1 */ | (1u<<5) /* AVX2 */ | (1u<<8) /* BMI2 */;
			if ((cpuid_info[1] & avx2_bits) == avx2_bits)
				features |= RRX86_CPU_AVX2;

			if (has_os_avx512_support)
			{
				// For us to report AVX512, we want the Skylake feature set
				const U32 avx512_bits = (1u << 31) /* AVX512VL */ | (1u << 30) /* AVX512BW */ | (1u << 17) /* AVX512DQ */ | (1u << 16) /* AVX512F */;
				if ((cpuid_info[1] & avx512_bits) == avx512_bits)
					features |= RRX86_CPU_AVX512;

				// Use the VBMI2 bit (set on ICL+) to set the PREFER512 flag. This is available
				// on a generation of cores where AVX-512 has no major clock penalty anymore so
				// whether to use AVX-512 or not is a much more straightforward calculation,
				// and not so dependent on what else is running at the same time.
				if (cpuid_info[2] & (1u << 6))
					features |= RRX86_CPU_PREFER512;
			}
		}
	}
#endif

	// Super-paranoia: we use the AMD_ZEN flag to indicate we are free to use Zen-optimized
	// kernels without further CPUID checks. In case some joker monekys around with with CPUID
	// flags in the future, turn it off again if we don't have the CPUID bits we should have
	// on a real Zen.
	if (features & RRX86_CPU_AMD_ZEN)
	{
		const U32 zen_features = RRX86_CPU_SSE2 | RRX86_CPU_SSSE3 | RRX86_CPU_SSE41 | RRX86_CPU_SSE42 | RRX86_CPU_F16C |
			RRX86_CPU_AVX | RRX86_CPU_AVX2 |
			RRX86_CPU_BMI1 | RRX86_CPU_BMI2;

		if ((features & zen_features) != zen_features)
			features &= ~RRX86_CPU_AMD_ZEN;
	}

	// write detected features
	// only write value once at end of the function!
	features |= RRX86_CPU_INITIALIZED;

	g_rrCPUx86_feature_flags = features; // atomic or volatile store
}

#endif // RRX86_CPU_DYNAMIC_DETECT

#endif  // __RADX86__

