// Copyright Epic Games Tools, LLC. All Rights Reserved.
// This source file is licensed solely to users who have
// accepted a valid Unreal Engine license agreement 
// (see e.g., https://www.unrealengine.com/eula), and use
// of this source file is governed by such agreement.

#pragma once
#include "rrCore.h"

#ifdef RADAUDIO_WRAP
#define RADAUDIO_NAME(name) RR_STRING_JOIN(RADAUDIO_WRAP, name##_)
#else
#define RADAUDIO_NAME(name) ( name )
#endif


#define RRX86_CPU_INITIALIZED	(1U<<0)		// bit gets set on CPUx86_detect() call
#define RRX86_CPU_SSE2			(1U<<1)
#define RRX86_CPU_SSSE3			(1U<<2)
#define RRX86_CPU_SSE41			(1U<<3)
#define RRX86_CPU_SSE42			(1U<<4)
#define RRX86_CPU_AVX			(1U<<5)
#define RRX86_CPU_BMI1			(1U<<6)		// bit manipulation instructions 1
#define RRX86_CPU_BMI2			(1U<<7)		// bit manipulation instructions 2
#define RRX86_CPU_AVX2			(1U<<8)
#define RRX86_CPU_AMD_ZEN		(1U<<9)		// not a CPUID bit; checks for AMD Zen uArch chips
#define RRX86_CPU_F16C          (1U<<10)    // float16 conversion instructions
#define RRX86_CPU_AVX512        (1U<<11)    // Skylake feature set: AVXF512{F,VL,BW,DQ}
#define RRX86_CPU_PREFER512		(1U<<12)	// Set when 512-bit vector usage is preferred with no major caveats
#define RRX86_CPU_POPCNT 		(1U<<13)

// HOW TO USE:
// - call rrCPUx86_detect() to init (calling multiple times is perfectly safe)
// - after that, query features with rrCPUx86_feature_present(bit)
// - if you didn't init, you just get 0 on everything

#ifdef __RADX86__

	#if defined(__RADJAGUAR__)

		#define g_rrCPUx86_feature_flags		(RRX86_CPU_INITIALIZED | RRX86_CPU_SSE2 | RRX86_CPU_SSSE3 | RRX86_CPU_SSE41 | RRX86_CPU_SSE42 | RRX86_CPU_AVX | RRX86_CPU_BMI1 | RRX86_CPU_F16C) // these are known on Jaguar
		#define rrCPUx86_detect()		/**/

	#elif defined(__RADZEN2__)

		#define g_rrCPUx86_feature_flags		(RRX86_CPU_INITIALIZED | RRX86_CPU_SSE2 | RRX86_CPU_SSSE3 | RRX86_CPU_SSE41 | RRX86_CPU_SSE42 | RRX86_CPU_AVX | RRX86_CPU_BMI1 | RRX86_CPU_BMI2 | RRX86_CPU_AVX2 | RRX86_CPU_AMD_ZEN | RRX86_CPU_F16C) // Zen 2 has everything
		#define rrCPUx86_detect()		/**/

	#else

		// feel free to add other known targets before here
		// this is the fallback where we CPUID at runtime
		#define RRX86_CPU_DYNAMIC_DETECT

		#define g_rrCPUx86_feature_flags RADAUDIO_NAME(g_rrCPUx86_feature_flags)
		#define rrCPUx86_detect RADAUDIO_NAME(rrCPUx86_detect)

		#ifdef __cplusplus
			extern "C"  U32 g_rrCPUx86_feature_flags;
			extern "C" void rrCPUx86_detect();
		#else
			extern U32 g_rrCPUx86_feature_flags;
			extern void rrCPUx86_detect();
		#endif

	#endif

	//#define rrCPUx86_feature_present(bit)	((g_rrCPUx86_feature_flags & (bit)) != 0)

	static RADINLINE U8 rrCPUx86_feature_present(U32 bit)
	{
		// should have called rrCPUx86_detect :
		U32 flags = g_rrCPUx86_feature_flags;
		RR_ASSERT( flags & RRX86_CPU_INITIALIZED );
		return (flags & bit) ? 1 : 0;
	}

	static RADINLINE U8 rrCPUx86_all_features_present(U32 bits)
	{
		// should have called rrCPUx86_detect :
		U32 flags = g_rrCPUx86_feature_flags;
		RR_ASSERT( flags & RRX86_CPU_INITIALIZED );
		return (flags & bits) == bits;
	}

#else // #ifdef __RADX86__

	#define rrCPUx86_detect()		/**/
	#define rrCPUx86_feature_present(bit) (false)
	#define rrCPUx86_all_features_present(bits) (false)

#endif

