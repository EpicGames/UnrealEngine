// Copyright Epic Games Tools, LLC. All Rights Reserved.
#ifndef RADAUDIO_COMMON_H
#define RADAUDIO_COMMON_H

#define MAX_VALID_VERSION              1

#include "rrCore.h"

#ifdef RADAUDIO_DEVELOPMENT
#include "rrTime.h"
#endif

// sets up what SIMD kernels we build.
#include "radaudio_sse.h"

// determined by the FFT code
#define FFT_ALIGN(type, name) RAD_ALIGN(type, name, 64)

// rather than fight with whether math.h is included we just name our own.
#define RADAUDIO_PI 3.1415926535897932384626433832795028

#ifndef __cplusplus
enum { false, true };
#endif

#define RADAUDIO_LONG_BLOCK_LEN     1024  // number of mdct coefficients for a long block
#define RADAUDIO_SHORT_BLOCK_LEN     128  // number of mdct coefficients for a short block

#define MAX_BANDS                     24
#define MAX_COEFFS                  1024
#define MAX_SUBBANDS                  72
#define MAX_SUBBANDS_QUANTIZED        56
#define MAX_MDCT_SAMPLES            2048
#define MAX_RADAUD_CHANNELS            2
#define MAX_COEFF_PER_BAND           512
#define MAX_COEFF_PER_SUBBAND         32

#define MAX_RUNLEN                   240
#define END_OF_ZERORUN               255  // token indicating end of the zero-runlength data
#define COARSE_RUNLEN_THRESHOLD       60  // runlengths larger than this have bottom two bits sent by varbits
#define PREDICT_FIRST_BAND_EXP         0  // exponent to use as prediction for first band of left channel

#define MAX_FINE_ENERGY_BITS          16  
#define COEFFS_PER_SUBBAND            16  // maximum

#define EXPONENT_NEGATIVE_INFINITY   -17  // encoding of -inf exponent, i.e. 0 amplitude
#define EXPONENT_NEGATIVE_16         -16  // encoding of -16 exponent
#define EXPONENT_ZERO                  0
#define EXPONENT_POSITIVE_14          14
#define BAND_EXPONENT_NONE           -17  // encoding indicating no data in band, i.e. -inf

#define SUBBANDS_SKIP_EMPTY_BANDS     true

#define MACRO_BAND_SIZE                3
#define LARGEST_BIASED_SUBBAND        63
#define SUBBAND_BIAS_CENTER           12

#define NUM_NZ_MODE                    4           // num long block modes
#define NUM_SELECTOR_MODES         (NUM_NZ_MODE+1) // short blocks get separate selectors
#define NUM_NZ_SELECTOR                4           // num selectors per mode
#define NUM_NZ_HUFF                    6           // number of nonzero huffman tables
#define MAX_NZ_BLOCKS                  12          // maximum number of nonzero 8-byte blocks

#define CLAMPED_INTEGER_EXPONENT_TO_CODED_EXPONENT(is_neg_inf, n)     \
                            (is_neg_inf ? EXPONENT_NEGATIVE_INFINITY  \
                                        : (n))

#define CODED_EXPONENT_TO_INTEGER(n)   (n)

#define MAX_ENCODED_BLOCK_BYTES        16383

typedef struct
{
   #ifdef __RADX86__
   U8 has_sse2;
   U8 has_ssse3;
   U8 has_sse4_1;
   U8 has_popcnt;
   U8 has_avx2;
   U8 padding[3];
   #else
   U8 dummy[8];
   #endif
} radaudio_cpu_features;

typedef enum
{
   RADA_48000,
   RADA_44100,
   RADA_32000,
   RADA_24000
} radaudio_samprate_code;
#define RADAUDIO_NUM_RATES    4

// block sizes
enum
{
   RADAUDIO_LONG,
   RADAUDIO_SHORT,
};

// certain values are encoded in the block headers as biased numbers,
// as 1 or 2 bytes depending on if the biased value fits in 1 byte
typedef struct
{
   U16 bytes_bias[2][3]; // bias for blocktype, num_channels
} radaudio_block_header_biases;

typedef struct
{
   U32   version;
   int   num_channels;
   int   sample_rate_mode;

   int   sample_rate;
   S8    mantissa_param[2][MAX_BANDS][2];
   U8    subband_predicted_sum[MAX_BANDS];
   S8    subband_bias[MAX_BANDS];
   U16   bytes_bias;
   U8    nzmode_num64[NUM_NZ_MODE];    // num64[3] is always 0 (not transmitted)
   S8    nzmode_huff[3][12];
   U8    nzmode_selectors[NUM_NZ_SELECTOR][NUM_SELECTOR_MODES];  // other huffman table selectors
} radaudio_stream_header_unpacked;

typedef struct
{
   // first 3-4 members get initialized by data
   U8    num_8byte_chunks;
   U8    huffman_table_for_chunk[MAX_NZ_BLOCKS];
   U8    invert_chunk[MAX_NZ_BLOCKS]; // 0 (don't invert) or 1 (invert)

   // remaining members get initialized by code
   U8    num_chunks_per_huff[NUM_NZ_HUFF];
   U8    source_pos[3][MAX_NZ_BLOCKS]; // for each channel, where its source offset is, in blocks, separately for left & right
} radaudio_nonzero_blockmode_descriptor;

typedef struct
{
   int num_bands;
   int num_quantized_subbands;
   int num_quantized_coeffs;
   int band_exponent[MAX_BANDS];           // MAX_BANDS
   U8  band_mantissa_bitcount[MAX_BANDS];  // MAX_BANDS
   int band_mantissa[MAX_BANDS];           // MAX_BANDS
   U16 quantized_subbands[MAX_SUBBANDS+7]; // excess_read7
   S8  quantized_coeff_decode[MAX_COEFFS];
   union {
      int quantized_coeff_encode[MAX_COEFFS]; // encoder uses this space for full-size quantized coeffs
      FFT_ALIGN(F32, dequantized_coeff_decode[MAX_COEFFS]); // decoder uses it for float dequantized coeffs
   };
} radaudio_block_data;

typedef struct
{
   int rate;
   int num_bands;
   int num_quantized_coeffs;
   int num_coeffs;
   int num_subbands;
   int num_quantized_subbands;
   int num_bands_without_subbands;
   int num_coeffs_for_band    [MAX_BANDS];
   int num_subbands_for_band  [MAX_BANDS];  // only long blocks

   // computed values:
   int first_subband_for_band [MAX_BANDS];
   int first_coeff_for_band   [MAX_BANDS];
   int num_coeffs_for_subband [MAX_SUBBANDS];
   int first_coeff_for_subband[MAX_SUBBANDS];
   float band_scale_encode    [MAX_BANDS];
   float band_scale_decode    [MAX_BANDS];
} radaudio_rate_info;

typedef struct
{
   rrbool final_block;
   rrbool prev_block_short;
   rrbool this_block_short;
   rrbool next_block_short;
   rrbool mid_side_encoded;
   unsigned int num_channels_encoded;
   rrbool disable_final_subband_predict;
   rrbool predict_stereo_subband;
   rrbool predict_stereo_exponent;
   rrbool mid_side_bands;
   unsigned int nonzero_bitarray_mode;

   U32 block_bytes;
   U32 num_runlength_array;
   U32 vbstream0_length;
   U32 final_samples_discard; // number of fully-computed samples to discard from the final block
} radaudio_block_header_unpacked;

typedef struct
{
   U8 length;
   U8 symbol;
} radaudio_huff_symbol;

typedef struct
{
   U16 code;
   U8 length;
} radaudio_huff_code;

typedef struct
{
   #ifdef HUFFMAN_ENCODE
   radaudio_huff_code   encode[256];
   #endif
   #ifdef HUFFMAN_DECODE
   radaudio_huff_symbol decode[2048];
   #endif
   #if !defined(HUFFMAN_ENCODE) && !defined(HUFFMAN_DECODE)
   int dummy; // so struct isn't empty (which is not allowed)
   #endif
} radaudio_huffman;

// BCPL lcg
#define LCG_MUL    2147001325
#define LCG_ADD    715136305

static RADINLINE U32 lcg(U32 value)
{
   return value * LCG_MUL + LCG_ADD;
}
#endif
