// Copyright Epic Games Tools, LLC. All Rights Reserved.
#include "radaudio_common.h"
#include "rrCore.h"
#include <string.h>
#include <stdio.h>

static int sample_rates[4] =
{
   48000,
   44100,
   32000,
   24000,
};

#include "radaudio_data_tables.inl"
#include "radaudio_huffman_tables.inl"

static radaudio_huffman *rada_nonzero_bitflags_huff[6] =
{
   &rada_nonzero_bitflags_huff0,
   &rada_nonzero_bitflags_huff1,
   &rada_nonzero_bitflags_huff2,
   &rada_nonzero_bitflags_huff3,
   &rada_nonzero_bitflags_huff4,
   &rada_nonzero_bitflags_huff5,
};

static radaudio_huffman *rada_nonzero_coefficient_pair_huff[4] =
{
   &rada_nonzero_coefficient_pair_huff0,
   &rada_nonzero_coefficient_pair_huff1,
   &rada_nonzero_coefficient_pair_huff2,
   &rada_nonzero_coefficient_pair_huff3,
};

static radaudio_huffman *rada_nonzero_coefficient_big_huff[4] =
{
   &rada_nonzero_coefficient_big_huff0,
   &rada_nonzero_coefficient_big_huff1,
   &rada_nonzero_coefficient_big_huff2,
   &rada_nonzero_coefficient_big_huff3,
};


// each block picks between 4 modes using two bits in the block header.
// 
// the meaning of the 4 modes is specified in the stream header, which is currently 128 bytes.
// (a 1 second stereo sample at 100 kbps uses 12500 bytes, so stream header is 1% in this extreme case; not worth worrying about)
// 
// for each of the four modes, the header specifies:
// - which of 4 huffman tables to compress pairs of coefficients packed as nibbles (-7..7, plus overflow code)
// - which of 4 huffman tables to compress coefficients whose magnitude exceeds 7
// - how to encode the nonzero/zero information about the coefficients:
//   - when to transition between bit-encoding into 8 zero/non-zero flags per byte and using run length compression
//   - for each group of 8 bytes, which of 6 huffman tables to use to compress those 8 bytes
//   - whether to xor each byte with 255 before encoding
//     - in theory this effectively doubles the available huffman tables
//     - it was very effective in the original design that had one global huffman table
//     - it's used almost never in the current scheme with 6 huffman tables

// slots in huffman selector array 
enum
{
   HS_COEFF_PAIR=0,  // nibble pairs
   HS_COEFF_BIG=1    // values whose magnitude exceeeds 7
};

// compute derived data about the bitflag byte packing to make decoding not require
// any branching. all 8-byte blocks that use the same huffman table will be packed
// together, and a table is built for how to reorder those to original order.
static void compute_packorder(radaudio_nonzero_blockmode_descriptor *nz_desc)
{
   for (int i=0; i < nz_desc->num_8byte_chunks; ++i)
      ++nz_desc->num_chunks_per_huff[nz_desc->huffman_table_for_chunk[i]];

   // current output position for each huff; 0 is a placeholder for numbering convention
   U8 pos[NUM_NZ_HUFF] = { 0 };

   // mono
   pos[0] = 0;
   for (int i=1; i < NUM_NZ_HUFF; ++i) {
      // starting position for each huff output is after all previous output, i.e. previous start + previous len
      pos[i] = pos[i-1] + nz_desc->num_chunks_per_huff[i-1];
   }

   for (int i=0; i < nz_desc->num_8byte_chunks; ++i) {
      int ht = nz_desc->huffman_table_for_chunk[i];
      nz_desc->source_pos[0][i] = pos[ht];
      ++pos[ht];
   }

   // stereo
   pos[0] = 0;
   for (int i=1; i < NUM_NZ_HUFF; ++i) {
      // starting position for each huff output is after all previous output, i.e. previous start + previous len
      pos[i] = pos[i-1] + 2*nz_desc->num_chunks_per_huff[i-1];
   }

   for (int c=0; c < MAX_RADAUD_CHANNELS; ++c) {
      for (int i=0; i < nz_desc->num_8byte_chunks; ++i) {
         int ht = nz_desc->huffman_table_for_chunk[i];
         nz_desc->source_pos[1+c][i] = pos[ht];
         ++pos[ht];
      }
   }
}

static void radaudio_init_nz_desc(radaudio_nonzero_blockmode_descriptor nz_desc[4])
{
   for (int i=0; i < 3; ++i)
      compute_packorder(&nz_desc[i]);
   // slot 3 is always all run-length
}

// mdct windows
static float *radaudio_windows [2] = { window_long  , window_short   };

static radaudio_cpu_features cpu_detect(void)
{
  radaudio_cpu_features f = { 0 };

  #ifdef __RADX86__
  // this function caches results so won't be an issue to call multiple times.
  rrCPUx86_detect();

  f.has_sse2   = rrCPUx86_feature_present(RRX86_CPU_SSE2);
  f.has_ssse3  = rrCPUx86_feature_present(RRX86_CPU_SSSE3);
  f.has_sse4_1 = rrCPUx86_feature_present(RRX86_CPU_SSE41);
  f.has_popcnt = rrCPUx86_feature_present(RRX86_CPU_POPCNT);
  f.has_avx2   = rrCPUx86_feature_present(RRX86_CPU_AVX2);
  #endif

  return f;
}

// values to scale band energy by to "normalize" for sample rate
static int rate_scale[4] = {
   235, // 48,000
   256, // 44,100
   353, // 32,000
   470, // 24,000
};

// this function must compute same data in encoder and decoder so they stay in sync,
// so it's integer/fixed-point only
static void compute_mantissa_bitcount(int rate_mode,
                                      int is_short_block,
                                      S8  mantissa_param[][MAX_BANDS][2],
                                      int *exponents,
                                      U8 *mantissa_bitcount)
{
   radaudio_rate_info *info = &radaudio_rateinfo[is_short_block][rate_mode];
   int j;
   int limit     = 12 * rate_scale[rate_mode];
   if (limit > 16*256) limit = 16*256; // never useful to have more than 16 bits of mantissa

   for (j=0; j < info->num_bands; ++j) {
      int be_log2 = exponents[j];
      int mantissa_bits;
      int base_bits       = mantissa_param[is_short_block][j][0]<<5;
      int bits_for_energy = be_log2 * mantissa_param[is_short_block][j][1];
      if (be_log2 == BAND_EXPONENT_NONE)
         mantissa_bits = 0;
      else {
         mantissa_bits = base_bits + bits_for_energy;
         mantissa_bits = (mantissa_bits * rate_scale[rate_mode]) >> 8;
      }
      if (mantissa_bits < 0) mantissa_bits = 0;
      if (mantissa_bits > limit) mantissa_bits = limit;
      mantissa_bitcount[j] = (U8) (mantissa_bits >> 8);
   }
}

static int radaudio_code_sample_rate(int rate)
{
   int i;
   if (rate > 0)
      for (i=0; i < 4; ++i)
         if (sample_rates[i] == rate)
            return i;
   return -1;
}

static void compute_bias_set(radaudio_block_header_biases *bc, U16 bias)
{
   bc->bytes_bias[0][0] = bc->bytes_bias[0][1] = bc->bytes_bias[0][2] = 0;
   bc->bytes_bias[1][0] = bc->bytes_bias[1][1] = bc->bytes_bias[1][2] = 0;

   // supplied value is stereo long block
   bc->bytes_bias[0][2] = bias;

   // mono long block uses half the bias
   bc->bytes_bias[0][1] = (bias >> 1);
}


// flags at start of block packet

// 0 overhead to use:
#define RADAUDIO_BLOCKFLAG_8BIT_PARITY               (1u <<  0)
#define RADAUDIO_BLOCKFLAG_THIS_SHORT                (1u <<  1)
#define RADAUDIO_BLOCKFLAG_NEXT_SHORT                (1u <<  2)
#define RADAUDIO_BLOCKFLAG_STEREO                    (1u <<  3)
#define RADAUDIO_BLOCKFLAG_MIDSIDE_BANDS             (1u <<  4)
#define RADAUDIO_BLOCKFLAG_PREDICT_SUBBAND_STEREO    (1u <<  5)
#define RADAUDIO_BLOCKFLAG_PREDICT_EXPONENT_STEREO   (1u <<  6)
#define RADAUDIO_BLOCKFLAG_16BIT_FLAGS               (1u <<  7)
// if any of these are set, it costs one more byte:
#define RADAUDIO_BLOCKFLAG_NONZERO_BITARRAY_MASK     (3u <<  8)
#define RADAUDIO_BLOCKFLAG_DISABLE_SUBBAND_PREDICT   (1u << 10)
#define RADAUDIO_BLOCKFLAG_MIDSIDE_ENCODED           (1u << 11)
#define RADAUDIO_BLOCKFLAG_EXCESS_RUNLENGTH_ARRAY    (1u << 12)
#define RADAUDIO_BLOCKFLAG_EXCESS_VBSTREAM0_LENGTH   (1u << 13)
#define RADAUDIO_BLOCKFLAG_EXCESS_BLOCK_BYTES        (1u << 14)
#define RADAUDIO_BLOCKFLAG_FINAL                     (1u << 15)

#define RADAUDIO_BLOCKFLAG_NONZERO_BITARRAY_MODE_GET(x)  (((x) >> 8) & 3u)
#define RADAUDIO_BLOCKFLAG_NONZERO_BITARRAY_MODE_SET(x)   ((x) << 8)

// use invalid flag combination as first byte of stream header, allows us to detect looping automatically
//    mono block, but stereo predict, also incorrect parity
#define RADAUDIO_STREAMHEADER_FLAGS  (                                            \
                                       RADAUDIO_BLOCKFLAG_PREDICT_SUBBAND_STEREO  \
                                     | RADAUDIO_BLOCKFLAG_PREDICT_EXPONENT_STEREO \
                                   ) 

static int countbits8(U32 flags)
{
   flags = ((flags>>1) & 0x5555) + (flags & 0x5555);    // binary: (0q0s0u0w & 01010101) + (0r0t0v0x & 01010101)
   flags = ((flags>>2) & 0x3333) + (flags & 0x3333);    // binary:    foo    & 00110011) + (  bar    & 00110011)
   return  ((flags>>4) & 0x0f0f) + (flags & 0x0f0f);    // binary:    baz    & 00001111) + ( quux    & 00001111)
}

static int put_8bits_or_16bits(U8 buffer[9], int offset, U16 value)
{
   buffer[offset++] = (U8) (value & 255);
   if (value >= 256)
      buffer[offset++] = (U8) (value >> 8);
   return offset;
}

typedef struct
{
   U16 flags, bytes, runlen, vbstream0;
} block_header_info;

enum
{
   COMMON_INCOMPLETE_DATA = -2,
   COMMON_INVALID_DATA = -3,
   COMMON_STREAM_HEADER = -4
};

static int radaudio_encode_block_header(U8 buffer[10], const radaudio_block_header_biases *bc, const radaudio_block_header_unpacked *bh)
{
   int offset=0;
   U32 flags = 0;

   // validate limits of what can be encoded in the header
   if (bh->vbstream0_length > 65535)
      return COMMON_INVALID_DATA;
   if (bh->block_bytes > 65535)
      return COMMON_INVALID_DATA;
   if (bh->num_runlength_array > 65535)
      return COMMON_INVALID_DATA;

   // validate other aspects of configuration
   if (bh->num_channels_encoded != 1 && bh->num_channels_encoded != 2)
      return COMMON_INVALID_DATA;
   if (bh->num_runlength_array > 2u * (bh->this_block_short ? RADAUDIO_SHORT_BLOCK_LEN+1 : RADAUDIO_LONG_BLOCK_LEN+1))
      return COMMON_INVALID_DATA;

   if (bh->final_block) {
      if (bh->this_block_short) {
         if (bh->final_samples_discard > RADAUDIO_SHORT_BLOCK_LEN)
            return COMMON_INVALID_DATA;
      } else {
         // if we would ever discard final samples from a long block,
         // it should have been sent as a series of short blocks instead
         if (bh->final_samples_discard != 0)
            return COMMON_INVALID_DATA;
      }
   }

   U16 bias = bc->bytes_bias[bh->this_block_short][bh->num_channels_encoded];

   U16 block_bytes_biased      = (U16) ((bh->block_bytes         - bias) & 0xffff);
   U16 num_runlength_biased    = (U16) ((bh->num_runlength_array - 0   ) & 0xffff);
   U16 vbstream0_length_biased = (U16) ((bh->vbstream0_length    - 0   ) & 0xffff);

   if ( bh->this_block_short              ) flags |= RADAUDIO_BLOCKFLAG_THIS_SHORT;
   if ( bh->next_block_short              ) flags |= RADAUDIO_BLOCKFLAG_NEXT_SHORT;
   if ( bh->num_channels_encoded == 2     ) flags |= RADAUDIO_BLOCKFLAG_STEREO;
   if ( bh->predict_stereo_subband        ) flags |= RADAUDIO_BLOCKFLAG_PREDICT_SUBBAND_STEREO;
   if ( bh->predict_stereo_exponent       ) flags |= RADAUDIO_BLOCKFLAG_PREDICT_EXPONENT_STEREO;

   flags |= RADAUDIO_BLOCKFLAG_NONZERO_BITARRAY_MODE_SET(bh->nonzero_bitarray_mode);
   if ( bh->final_block                   ) flags |= RADAUDIO_BLOCKFLAG_FINAL;
   if ( bh->disable_final_subband_predict ) flags |= RADAUDIO_BLOCKFLAG_DISABLE_SUBBAND_PREDICT;
   if ( bh->mid_side_encoded              ) flags |= RADAUDIO_BLOCKFLAG_MIDSIDE_ENCODED;
   if ( bh->mid_side_bands                ) flags |= RADAUDIO_BLOCKFLAG_MIDSIDE_BANDS;

   if ( block_bytes_biased >= 256         ) flags |= RADAUDIO_BLOCKFLAG_EXCESS_BLOCK_BYTES;
   if ( num_runlength_biased >= 256       ) flags |= RADAUDIO_BLOCKFLAG_EXCESS_RUNLENGTH_ARRAY;
   if ( vbstream0_length_biased >= 256    ) flags |= RADAUDIO_BLOCKFLAG_EXCESS_VBSTREAM0_LENGTH;

   if ( flags >= 256                      ) flags |= RADAUDIO_BLOCKFLAG_16BIT_FLAGS;

   // force odd parity
   if ( (countbits8(flags&255) & 1) == 0  ) flags |= RADAUDIO_BLOCKFLAG_8BIT_PARITY;

   // write 1 byte if 0..255, otherwise 2 bytes
   offset = put_8bits_or_16bits(buffer, offset, (U16) flags);
   offset = put_8bits_or_16bits(buffer, offset, block_bytes_biased);
   offset = put_8bits_or_16bits(buffer, offset, num_runlength_biased);
   offset = put_8bits_or_16bits(buffer, offset, vbstream0_length_biased);

   if (bh->final_block) {
      RR_PUT16_LE(&buffer[offset], (U16) bh->final_samples_discard);
      offset += 2;
   }

   // maximum written is 5 16-bit values, i.e. 10 bytes

   return offset;
}

// returns length of header in bytes
// returns RADAUDIO_INCOMPLETE_DATA if there's not enough data to decode the header
// returns RADAUDIO_INVALID_DATA if the header fails some simple validity tests
static int radaudio_decode_block_header(const U8 buffer[10], const radaudio_block_header_biases *bc, radaudio_block_header_unpacked *bh, size_t memavail)
{
   int offset = 0;

   // block header is a minimum of 4 bytes
   if (memavail < 4)
      return COMMON_INCOMPLETE_DATA;

   U16 flags = buffer[offset++];

   // if flags value is the magic byte that's the first byte of a stream, assume we're scanning the beginning of the stream
   if (flags == RADAUDIO_STREAMHEADER_FLAGS)
      return COMMON_STREAM_HEADER;

   // check parity of first flag byte to catch corrupt streams as soon as possible
   // can't check before above test because RADAUDIO_STREAMHEADER_FLAGS intentionally has wrong parity
   if ((countbits8(flags) & 1) == 0)
      return COMMON_INVALID_DATA;

   // read second byte of flags if there is one
   if (flags & RADAUDIO_BLOCKFLAG_16BIT_FLAGS)
      flags |= buffer[offset++] << 8;

   // compute header length based on flags
   size_t header_length = offset;

   header_length += (flags & RADAUDIO_BLOCKFLAG_EXCESS_BLOCK_BYTES)      ? 2 : 1;
   header_length += (flags & RADAUDIO_BLOCKFLAG_EXCESS_RUNLENGTH_ARRAY)  ? 2 : 1;
   header_length += (flags & RADAUDIO_BLOCKFLAG_EXCESS_VBSTREAM0_LENGTH) ? 2 : 1;
   header_length += (flags & RADAUDIO_BLOCKFLAG_FINAL)                   ? 2 : 0;

   if ((size_t)header_length > memavail)
      return COMMON_INCOMPLETE_DATA;

   // at this point all data should be available, so no length checking is needed

   bh->this_block_short              = ((flags & RADAUDIO_BLOCKFLAG_THIS_SHORT              )   != 0 );
   bh->next_block_short              = ((flags & RADAUDIO_BLOCKFLAG_NEXT_SHORT              )   != 0 );
   bh->num_channels_encoded          = ((flags & RADAUDIO_BLOCKFLAG_STEREO                  ) ? 2 : 1);
   bh->predict_stereo_subband        = ((flags & RADAUDIO_BLOCKFLAG_PREDICT_SUBBAND_STEREO  )   != 0 );
   bh->predict_stereo_exponent       = ((flags & RADAUDIO_BLOCKFLAG_PREDICT_EXPONENT_STEREO )   != 0 );

   bh->nonzero_bitarray_mode         = RADAUDIO_BLOCKFLAG_NONZERO_BITARRAY_MODE_GET(flags);
   bh->final_block                   = ((flags & RADAUDIO_BLOCKFLAG_FINAL                   )   != 0 );
   bh->disable_final_subband_predict = ((flags & RADAUDIO_BLOCKFLAG_DISABLE_SUBBAND_PREDICT )   != 0 );
   bh->mid_side_encoded              = ((flags & RADAUDIO_BLOCKFLAG_MIDSIDE_ENCODED         )   != 0 );
   bh->mid_side_bands                = ((flags & RADAUDIO_BLOCKFLAG_MIDSIDE_BANDS           )   != 0 );

   U16 block_bytes, num_runlength_array, vbstream0_length;

   // if values are sent as two bytes, verify that they required two bytes.
   // unlikely to catch stream corruption, but it can't hurt.

   if ((flags & RADAUDIO_BLOCKFLAG_EXCESS_BLOCK_BYTES)==0) {
      block_bytes = buffer[offset++];
   } else {
      block_bytes = RR_GET16_LE(&buffer[offset]);
      if (block_bytes < 256)
         return COMMON_INVALID_DATA;
      offset += 2;
   }

   if ((flags & RADAUDIO_BLOCKFLAG_EXCESS_RUNLENGTH_ARRAY) == 0) {
      num_runlength_array = buffer[offset++];
   } else {
      num_runlength_array = RR_GET16_LE(&buffer[offset]);
      if (num_runlength_array < 256)
         return COMMON_INVALID_DATA;
      offset += 2;
   }

   if ((flags & RADAUDIO_BLOCKFLAG_EXCESS_VBSTREAM0_LENGTH) == 0) {
      vbstream0_length = buffer[offset++];
   } else {
      vbstream0_length = RR_GET16_LE(&buffer[offset]);
      if (vbstream0_length < 256)
         return COMMON_INVALID_DATA;
      offset += 2;
   }

   if (bh->final_block) {
      bh->final_samples_discard = RR_GET16_LE(&buffer[offset]);
      offset += 2;
   }

   bh->block_bytes         = (U16) (block_bytes         + bc->bytes_bias[bh->this_block_short][bh->num_channels_encoded]);
   bh->num_runlength_array = (U16) (num_runlength_array + 0             );
   bh->vbstream0_length    = (U16) (vbstream0_length    + 0             );

   // let caller do further sanity checking

   return offset;
}

// all values little-endian
typedef struct
{
   char  magic[8];
   U32   version;
   U16   sample_rate;
   U16   bytes_bias;
   // 16 bytes

   U8    num_channels; // must be 1 or 2
   U8    nzmode_data1[NUM_NZ_MODE/2];                     // 4 modes, 1 nibble each, packed number of 8-byte chunks for 4 modes
   U8    nzmode_data2[NUM_SELECTOR_MODES*1];              // 5 modes, 1 byte each, coding 4 2-bit huffman selectors
   U8    nzmode_data3[(NUM_NZ_MODE-1)*(MAX_NZ_BLOCKS/2)]; // 3 modes, 12 nibbles each, coding 12 4-bit huffman selectors + inversion bit
   // 26 bytes

   U8    subband_predicted_sum[MAX_BANDS];
   S8    mantissa_param[2][MAX_BANDS/4][2];
   // 48 bytes

   U8    padding[128-16-26-48];
} radaudio_stream_header;

RR_COMPILER_ASSERT(sizeof(radaudio_stream_header)==128);

static const char RADAUDIO_MAGIC[8] = {
   RADAUDIO_STREAMHEADER_FLAGS,
   'R', 'a', 'd',

   'A', 'u', 'd',
   '\032', // ^Z
};

// bitflag in packed and unpacked stream header signalling to if an 8-byte block should be xor-with-255
// we don't process it here, we let client process it
#define NZ_MODE_INVERT   (1<<3)

static size_t radaudio_pack_stream_header(U8 *raw_header,
                                    radaudio_stream_header_unpacked *h)
{
   if (h->sample_rate <= 0       || h->sample_rate     > 65535)
      return 0;
   if (h->version < 0            || h->version         > MAX_VALID_VERSION )
      return 0;
   if (h->num_channels <= 0      || h->num_channels    > 2                 )
      return 0;

   if (h->sample_rate != 48000 && h->sample_rate != 44100 && h->sample_rate != 32000 && h->sample_rate != 24000)
      return 0;

   radaudio_stream_header *header = (radaudio_stream_header *) raw_header;
   memset(header, 0, sizeof(*header));

   memcpy(header->magic, RADAUDIO_MAGIC, sizeof(header->magic));
   header->version = h->version;
   header->num_channels = (U8 ) h->num_channels;
   header->sample_rate  = (U16) h->sample_rate;

   memcpy(header->subband_predicted_sum, h->subband_predicted_sum, 24);
   for (int i=0; i < MAX_BANDS/4; ++i) {
      header->mantissa_param[0][i][0] = h->mantissa_param[0][i*4+0][0];
      header->mantissa_param[0][i][1] = h->mantissa_param[0][i*4+0][1];
      header->mantissa_param[1][i][0] = h->mantissa_param[1][i*4+0][0];
      header->mantissa_param[1][i][1] = h->mantissa_param[1][i*4+0][1];
   }
   header->bytes_bias = h->bytes_bias;

   // pack each mode
   for (int i=0; i < NUM_NZ_MODE/2; ++i)
      header->nzmode_data1[i] = h->nzmode_num64[2*i+0] | (h->nzmode_num64[2*i+1] << 4);

   // pack general huff selectors
   for (int i=0; i < NUM_SELECTOR_MODES; ++i) {
      U8 value = 0;
      for (int j=0; j < 4; ++j)
         value |= h->nzmode_selectors[j][i] << (2*j);
      header->nzmode_data2[i] = value;
   }

   // pack nz huff selectors
   for (int i=0; i < NUM_NZ_MODE-1; ++i) {
      for (int j=0; j < MAX_NZ_BLOCKS/2; ++j) {
         U8 value;
         value  = h->nzmode_huff[i][j*2+0]     ;
         value |= h->nzmode_huff[i][j*2+1] << 4;
         header->nzmode_data3[i*MAX_NZ_BLOCKS/2+j] = value;
      }
   }

   return sizeof(*header);
}

static rrbool radaudio_check_stream_header(U8 *raw_header, size_t raw_header_bytes_valid)
{
   radaudio_stream_header *header = (radaudio_stream_header *) raw_header;
   if (raw_header_bytes_valid < 8) // can we check MAGIC?
      return false;
   return (0 == memcmp(header->magic, RADAUDIO_MAGIC, sizeof(header->magic)));
}

static size_t radaudio_unpack_stream_header(U8 *raw_header, size_t raw_header_bytes_valid, radaudio_stream_header_unpacked *h)
{
   radaudio_stream_header *header = (radaudio_stream_header *) raw_header;

   if (!radaudio_check_stream_header(raw_header, raw_header_bytes_valid))
      return 0;

   // we are going to directly address everything up to the padding
   // in the header
   size_t required_header_bytes = sizeof(radaudio_stream_header) - sizeof(header->padding);
   if (raw_header_bytes_valid < required_header_bytes)
      return 0;

   h->sample_rate_mode = radaudio_code_sample_rate(header->sample_rate);
   if (h->sample_rate_mode < 0)
      return 0;

   h->sample_rate  = header->sample_rate;
   h->num_channels = header->num_channels;
   h->version      = header->version;
   h->bytes_bias   = header->bytes_bias;
   memcpy(h->subband_predicted_sum, header->subband_predicted_sum, 24);
   memcpy(h->mantissa_param, header->mantissa_param, sizeof(header->mantissa_param));

   for (int i=0; i < MAX_BANDS/4; ++i) {
      for (int j=0; j < 4; ++j) {
         h->mantissa_param[0][i*4+j][0] = header->mantissa_param[0][i][0];
         h->mantissa_param[0][i*4+j][1] = header->mantissa_param[0][i][1];
         h->mantissa_param[1][i*4+j][0] = header->mantissa_param[1][i][0];
         h->mantissa_param[1][i*4+j][1] = header->mantissa_param[1][i][1];
      }
   }

   // unpack each mode
   for (int i=0; i < NUM_NZ_MODE/2; ++i) {
      h->nzmode_num64[2*i+0] = header->nzmode_data1[i] & 15;
      h->nzmode_num64[2*i+1] = header->nzmode_data1[i] >> 4;
   }

   // unpack general huff selectors
   for (int i=0; i < NUM_SELECTOR_MODES; ++i) {
      U8 value = header->nzmode_data2[i];
      for (int j=0; j < 4; ++j) {
         h->nzmode_selectors[j][i] = (value >> (2*j)) & 3;
      }
   }

   // unpack nz huff selectors
   for (int i=0; i < NUM_NZ_MODE-1; ++i) {
      for (int j=0; j < MAX_NZ_BLOCKS/2; ++j) {
         U8 value = header->nzmode_data3[i*MAX_NZ_BLOCKS/2+j];
         h->nzmode_huff[i][j*2+0] = value & 15;
         h->nzmode_huff[i][j*2+1] = value >> 4;
      }
   }

   radaudio_rate_info *bi = &radaudio_rateinfo[0][h->sample_rate_mode];
   // compute derived bias values
   for (int i=0; i < bi->num_bands; ++i)
      if (bi->num_subbands_for_band[i] != 1) {
         h->subband_bias[i] = (S8) (SUBBAND_BIAS_CENTER - (h->subband_predicted_sum[i] / bi->num_subbands_for_band[i]));
      }
      else
         h->subband_bias[i] = (S8) -1;
   
   if (h->num_channels < 1 || h->num_channels > 2)
      return 0;
   if (h->version > MAX_VALID_VERSION)
      return 0;

   return sizeof(*header);
}
