// Copyright Epic Games Tools, LLC. All Rights Reserved.
static radaudio_rate_info radaudio_rateinfo[2][4] =
{
  {
    { 48000, 20,800, 1024, 54, 42, 12,
      {   8,   8,   8,   8,   8,   8,   8,   8,  16,  16,  16,  16,  32,  32,  48,  48,  48, 144, 160, 160,   0,   0,   0,   0, },
      {   1,   1,   1,   1,   1,   1,   1,   1,   1,   1,   1,   1,   2,   2,   3,   3,   3,   9,  10,  10,   0,   0,   0,   0, },
      {   0,   1,   2,   3,   4,   5,   6,   7,   8,   9,  10,  11,  12,  14,  16,  19,  22,  25,  34,  44,   0,   0,   0,   0, },
      {   0,   8,  16,  24,  32,  40,  48,  56,  64,  80,  96, 112, 128, 160, 192, 240, 288, 336, 480, 640,   0,   0,   0,   0, },
      {   8,   8,   8,   8,   8,   8,   8,   8,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0, },
      {   0,   8,  16,  24,  32,  40,  48,  56,  64,  80,  96, 112, 128, 144, 160, 176, 192, 208, 224, 240, 256, 272, 288, 304, 320, 336, 352, 368, 384, 400, 416, 432, 448, 464, 480, 496, 512, 528, 544, 560, 576, 592, 608, 624, 640, 656, 672, 688, 704, 720, 736, 752, 768, 784,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0, },
      { 0.353553384542f, 0.353553384542f, 0.353553384542f, 0.353553384542f, 0.353553384542f, 0.353553384542f, 0.353553384542f, 0.353553384542f, 0.250000000000f, 0.250000000000f, 0.250000000000f, 0.250000000000f, 0.176776692271f, 0.176776692271f, 0.144337564707f, 0.144337564707f, 0.144337564707f, 0.083333335817f, 0.079056940973f, 0.079056940973f, 0.000000000000f, 0.000000000000f, 0.000000000000f, 0.000000000000f, },
      { 0.000030517576f, 0.000030517576f, 0.000030517576f, 0.000030517576f, 0.000030517576f, 0.000030517576f, 0.000030517576f, 0.000030517576f, 0.000043158372f, 0.000043158372f, 0.000043158372f, 0.000043158372f, 0.000061035153f, 0.000061035153f, 0.000074752490f, 0.000074752490f, 0.000074752490f, 0.000129475113f, 0.000136478760f, 0.000136478760f, 0.000000000000f, 0.000000000000f, 0.000000000000f, 0.000000000000f, },
    },
    { 44100, 21,864, 1024, 58, 46, 12,
      {   8,   8,   8,   8,   8,   8,   8,   8,  16,  16,  16,  16,  32,  32,  32,  32,  48,  64, 144, 160, 192,   0,   0,   0, },
      {   1,   1,   1,   1,   1,   1,   1,   1,   1,   1,   1,   1,   2,   2,   2,   2,   3,   4,   9,  10,  12,   0,   0,   0, },
      {   0,   1,   2,   3,   4,   5,   6,   7,   8,   9,  10,  11,  12,  14,  16,  18,  20,  23,  27,  36,  46,   0,   0,   0, },
      {   0,   8,  16,  24,  32,  40,  48,  56,  64,  80,  96, 112, 128, 160, 192, 224, 256, 304, 368, 512, 672,   0,   0,   0, },
      {   8,   8,   8,   8,   8,   8,   8,   8,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0, },
      {   0,   8,  16,  24,  32,  40,  48,  56,  64,  80,  96, 112, 128, 144, 160, 176, 192, 208, 224, 240, 256, 272, 288, 304, 320, 336, 352, 368, 384, 400, 416, 432, 448, 464, 480, 496, 512, 528, 544, 560, 576, 592, 608, 624, 640, 656, 672, 688, 704, 720, 736, 752, 768, 784, 800, 816, 832, 848,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0, },
      { 0.353553384542f, 0.353553384542f, 0.353553384542f, 0.353553384542f, 0.353553384542f, 0.353553384542f, 0.353553384542f, 0.353553384542f, 0.250000000000f, 0.250000000000f, 0.250000000000f, 0.250000000000f, 0.176776692271f, 0.176776692271f, 0.176776692271f, 0.176776692271f, 0.144337564707f, 0.125000000000f, 0.083333335817f, 0.079056940973f, 0.072168782353f, 0.000000000000f, 0.000000000000f, 0.000000000000f, },
      { 0.000030517576f, 0.000030517576f, 0.000030517576f, 0.000030517576f, 0.000030517576f, 0.000030517576f, 0.000030517576f, 0.000030517576f, 0.000043158372f, 0.000043158372f, 0.000043158372f, 0.000043158372f, 0.000061035153f, 0.000061035153f, 0.000061035153f, 0.000061035153f, 0.000074752490f, 0.000086316744f, 0.000129475113f, 0.000136478760f, 0.000149504980f, 0.000000000000f, 0.000000000000f, 0.000000000000f, },
    },
    { 32000, 23,1024, 1024, 68, 56, 12,
      {   8,   8,   8,   8,   8,   8,   8,   8,  16,  16,  16,  16,  32,  32,  32,  32,  32,  64,  64,  64,  96, 192, 256,   0, },
      {   1,   1,   1,   1,   1,   1,   1,   1,   1,   1,   1,   1,   2,   2,   2,   2,   2,   4,   4,   4,   6,  12,  16,   0, },
      {   0,   1,   2,   3,   4,   5,   6,   7,   8,   9,  10,  11,  12,  14,  16,  18,  20,  22,  26,  30,  34,  40,  52,   0, },
      {   0,   8,  16,  24,  32,  40,  48,  56,  64,  80,  96, 112, 128, 160, 192, 224, 256, 288, 352, 416, 480, 576, 768,   0, },
      {   8,   8,   8,   8,   8,   8,   8,   8,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,   0,   0,   0,   0, },
      {   0,   8,  16,  24,  32,  40,  48,  56,  64,  80,  96, 112, 128, 144, 160, 176, 192, 208, 224, 240, 256, 272, 288, 304, 320, 336, 352, 368, 384, 400, 416, 432, 448, 464, 480, 496, 512, 528, 544, 560, 576, 592, 608, 624, 640, 656, 672, 688, 704, 720, 736, 752, 768, 784, 800, 816, 832, 848, 864, 880, 896, 912, 928, 944, 960, 976, 992, 1008,   0,   0,   0,   0, },
      { 0.353553384542f, 0.353553384542f, 0.353553384542f, 0.353553384542f, 0.353553384542f, 0.353553384542f, 0.353553384542f, 0.353553384542f, 0.250000000000f, 0.250000000000f, 0.250000000000f, 0.250000000000f, 0.176776692271f, 0.176776692271f, 0.176776692271f, 0.176776692271f, 0.176776692271f, 0.125000000000f, 0.125000000000f, 0.125000000000f, 0.102062068880f, 0.072168782353f, 0.062500000000f, 0.000000000000f, },
      { 0.000030517576f, 0.000030517576f, 0.000030517576f, 0.000030517576f, 0.000030517576f, 0.000030517576f, 0.000030517576f, 0.000030517576f, 0.000043158372f, 0.000043158372f, 0.000043158372f, 0.000043158372f, 0.000061035153f, 0.000061035153f, 0.000061035153f, 0.000061035153f, 0.000061035153f, 0.000086316744f, 0.000086316744f, 0.000086316744f, 0.000105715997f, 0.000149504980f, 0.000172633489f, 0.000000000000f, },
    },
    { 24000, 23,1024, 1024, 68, 56, 12,
      {   8,   8,   8,   8,   8,   8,   8,   8,  16,  16,  16,  16,  32,  32,  32,  32,  32,  64,  80,  96, 128, 256, 112,   0, },
      {   1,   1,   1,   1,   1,   1,   1,   1,   1,   1,   1,   1,   2,   2,   2,   2,   2,   4,   5,   6,   8,  16,   7,   0, },
      {   0,   1,   2,   3,   4,   5,   6,   7,   8,   9,  10,  11,  12,  14,  16,  18,  20,  22,  26,  31,  37,  45,  61,   0, },
      {   0,   8,  16,  24,  32,  40,  48,  56,  64,  80,  96, 112, 128, 160, 192, 224, 256, 288, 352, 432, 528, 656, 912,   0, },
      {   8,   8,   8,   8,   8,   8,   8,   8,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,  16,   0,   0,   0,   0, },
      {   0,   8,  16,  24,  32,  40,  48,  56,  64,  80,  96, 112, 128, 144, 160, 176, 192, 208, 224, 240, 256, 272, 288, 304, 320, 336, 352, 368, 384, 400, 416, 432, 448, 464, 480, 496, 512, 528, 544, 560, 576, 592, 608, 624, 640, 656, 672, 688, 704, 720, 736, 752, 768, 784, 800, 816, 832, 848, 864, 880, 896, 912, 928, 944, 960, 976, 992, 1008,   0,   0,   0,   0, },
      { 0.353553384542f, 0.353553384542f, 0.353553384542f, 0.353553384542f, 0.353553384542f, 0.353553384542f, 0.353553384542f, 0.353553384542f, 0.250000000000f, 0.250000000000f, 0.250000000000f, 0.250000000000f, 0.176776692271f, 0.176776692271f, 0.176776692271f, 0.176776692271f, 0.176776692271f, 0.125000000000f, 0.111803397536f, 0.102062068880f, 0.088388346136f, 0.062500000000f, 0.094491124153f, 0.000000000000f, },
      { 0.000030517576f, 0.000030517576f, 0.000030517576f, 0.000030517576f, 0.000030517576f, 0.000030517576f, 0.000030517576f, 0.000030517576f, 0.000043158372f, 0.000043158372f, 0.000043158372f, 0.000043158372f, 0.000061035153f, 0.000061035153f, 0.000061035153f, 0.000061035153f, 0.000061035153f, 0.000086316744f, 0.000096505057f, 0.000105715997f, 0.000122070305f, 0.000172633489f, 0.000114186318f, 0.000000000000f, },
    },
  },
  {
    { 48000, 17,112, 128, 17, 0, 0,
      {   1,   1,   1,   1,   2,   2,   2,   2,   4,   4,   4,   4,   4,  16,  16,  16,  32,   0,   0,   0,   0,   0,   0,   0, },
      {   1,   1,   1,   1,   1,   1,   1,   1,   1,   1,   1,   1,   1,   1,   1,   1,   1,   0,   0,   0,   0,   0,   0,   0, },
      {   0,   1,   2,   3,   4,   5,   6,   7,   8,   9,  10,  11,  12,  13,  14,  15,  16,   0,   0,   0,   0,   0,   0,   0, },
      {   0,   1,   2,   3,   4,   6,   8,  10,  12,  16,  20,  24,  28,  32,  48,  64,  80,   0,   0,   0,   0,   0,   0,   0, },
      {   1,   1,   1,   1,   2,   2,   2,   2,   4,   4,   4,   4,   4,  16,  16,  16,  32,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0, },
      {   0,   1,   2,   3,   4,   6,   8,  10,  12,  16,  20,  24,  28,  32,  48,  64,  80,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0, },
      { 1.000000000000f, 1.000000000000f, 1.000000000000f, 1.000000000000f, 0.707106769085f, 0.707106769085f, 0.707106769085f, 0.707106769085f, 0.500000000000f, 0.500000000000f, 0.500000000000f, 0.500000000000f, 0.500000000000f, 0.250000000000f, 0.250000000000f, 0.250000000000f, 0.176776692271f, 0.000000000000f, 0.000000000000f, 0.000000000000f, 0.000000000000f, 0.000000000000f, 0.000000000000f, 0.000000000000f, },
      { 0.000010789593f, 0.000010789593f, 0.000010789593f, 0.000010789593f, 0.000015258788f, 0.000015258788f, 0.000015258788f, 0.000015258788f, 0.000021579186f, 0.000021579186f, 0.000021579186f, 0.000021579186f, 0.000021579186f, 0.000043158372f, 0.000043158372f, 0.000043158372f, 0.000061035153f, 0.000000000000f, 0.000000000000f, 0.000000000000f, 0.000000000000f, 0.000000000000f, 0.000000000000f, 0.000000000000f, },
    },
    { 44100, 17,112, 128, 17, 0, 0,
      {   1,   1,   1,   1,   2,   2,   2,   2,   4,   4,   4,   4,   4,  16,  16,  16,  32,   0,   0,   0,   0,   0,   0,   0, },
      {   1,   1,   1,   1,   1,   1,   1,   1,   1,   1,   1,   1,   1,   1,   1,   1,   1,   0,   0,   0,   0,   0,   0,   0, },
      {   0,   1,   2,   3,   4,   5,   6,   7,   8,   9,  10,  11,  12,  13,  14,  15,  16,   0,   0,   0,   0,   0,   0,   0, },
      {   0,   1,   2,   3,   4,   6,   8,  10,  12,  16,  20,  24,  28,  32,  48,  64,  80,   0,   0,   0,   0,   0,   0,   0, },
      {   1,   1,   1,   1,   2,   2,   2,   2,   4,   4,   4,   4,   4,  16,  16,  16,  32,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0, },
      {   0,   1,   2,   3,   4,   6,   8,  10,  12,  16,  20,  24,  28,  32,  48,  64,  80,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0, },
      { 1.000000000000f, 1.000000000000f, 1.000000000000f, 1.000000000000f, 0.707106769085f, 0.707106769085f, 0.707106769085f, 0.707106769085f, 0.500000000000f, 0.500000000000f, 0.500000000000f, 0.500000000000f, 0.500000000000f, 0.250000000000f, 0.250000000000f, 0.250000000000f, 0.176776692271f, 0.000000000000f, 0.000000000000f, 0.000000000000f, 0.000000000000f, 0.000000000000f, 0.000000000000f, 0.000000000000f, },
      { 0.000010789593f, 0.000010789593f, 0.000010789593f, 0.000010789593f, 0.000015258788f, 0.000015258788f, 0.000015258788f, 0.000015258788f, 0.000021579186f, 0.000021579186f, 0.000021579186f, 0.000021579186f, 0.000021579186f, 0.000043158372f, 0.000043158372f, 0.000043158372f, 0.000061035153f, 0.000000000000f, 0.000000000000f, 0.000000000000f, 0.000000000000f, 0.000000000000f, 0.000000000000f, 0.000000000000f, },
    },
    { 32000, 17,128, 128, 17, 0, 0,
      {   1,   1,   1,   1,   2,   2,   2,   2,   4,   4,   4,   4,   4,  16,  16,  32,  32,   0,   0,   0,   0,   0,   0,   0, },
      {   1,   1,   1,   1,   1,   1,   1,   1,   1,   1,   1,   1,   1,   1,   1,   1,   1,   0,   0,   0,   0,   0,   0,   0, },
      {   0,   1,   2,   3,   4,   5,   6,   7,   8,   9,  10,  11,  12,  13,  14,  15,  16,   0,   0,   0,   0,   0,   0,   0, },
      {   0,   1,   2,   3,   4,   6,   8,  10,  12,  16,  20,  24,  28,  32,  48,  64,  96,   0,   0,   0,   0,   0,   0,   0, },
      {   1,   1,   1,   1,   2,   2,   2,   2,   4,   4,   4,   4,   4,  16,  16,  32,  32,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0, },
      {   0,   1,   2,   3,   4,   6,   8,  10,  12,  16,  20,  24,  28,  32,  48,  64,  96,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0, },
      { 1.000000000000f, 1.000000000000f, 1.000000000000f, 1.000000000000f, 0.707106769085f, 0.707106769085f, 0.707106769085f, 0.707106769085f, 0.500000000000f, 0.500000000000f, 0.500000000000f, 0.500000000000f, 0.500000000000f, 0.250000000000f, 0.250000000000f, 0.176776692271f, 0.176776692271f, 0.000000000000f, 0.000000000000f, 0.000000000000f, 0.000000000000f, 0.000000000000f, 0.000000000000f, 0.000000000000f, },
      { 0.000010789593f, 0.000010789593f, 0.000010789593f, 0.000010789593f, 0.000015258788f, 0.000015258788f, 0.000015258788f, 0.000015258788f, 0.000021579186f, 0.000021579186f, 0.000021579186f, 0.000021579186f, 0.000021579186f, 0.000043158372f, 0.000043158372f, 0.000061035153f, 0.000061035153f, 0.000000000000f, 0.000000000000f, 0.000000000000f, 0.000000000000f, 0.000000000000f, 0.000000000000f, 0.000000000000f, },
    },
    { 24000, 18,128, 128, 18, 0, 0,
      {   1,   1,   1,   1,   2,   2,   2,   2,   4,   4,   4,   4,   4,  16,  16,  16,  16,  32,   0,   0,   0,   0,   0,   0, },
      {   1,   1,   1,   1,   1,   1,   1,   1,   1,   1,   1,   1,   1,   1,   1,   1,   1,   1,   0,   0,   0,   0,   0,   0, },
      {   0,   1,   2,   3,   4,   5,   6,   7,   8,   9,  10,  11,  12,  13,  14,  15,  16,  17,   0,   0,   0,   0,   0,   0, },
      {   0,   1,   2,   3,   4,   6,   8,  10,  12,  16,  20,  24,  28,  32,  48,  64,  80,  96,   0,   0,   0,   0,   0,   0, },
      {   1,   1,   1,   1,   2,   2,   2,   2,   4,   4,   4,   4,   4,  16,  16,  16,  16,  32,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0, },
      {   0,   1,   2,   3,   4,   6,   8,  10,  12,  16,  20,  24,  28,  32,  48,  64,  80,  96,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0,   0, },
      { 1.000000000000f, 1.000000000000f, 1.000000000000f, 1.000000000000f, 0.707106769085f, 0.707106769085f, 0.707106769085f, 0.707106769085f, 0.500000000000f, 0.500000000000f, 0.500000000000f, 0.500000000000f, 0.500000000000f, 0.250000000000f, 0.250000000000f, 0.250000000000f, 0.250000000000f, 0.176776692271f, 0.000000000000f, 0.000000000000f, 0.000000000000f, 0.000000000000f, 0.000000000000f, 0.000000000000f, },
      { 0.000010789593f, 0.000010789593f, 0.000010789593f, 0.000010789593f, 0.000015258788f, 0.000015258788f, 0.000015258788f, 0.000015258788f, 0.000021579186f, 0.000021579186f, 0.000021579186f, 0.000021579186f, 0.000021579186f, 0.000043158372f, 0.000043158372f, 0.000043158372f, 0.000043158372f, 0.000061035153f, 0.000000000000f, 0.000000000000f, 0.000000000000f, 0.000000000000f, 0.000000000000f, 0.000000000000f, },
    },
  },
};

static float window_long[1024] =
{
  0.000000924059f, 0.000008316517f, 0.000023101364f, 0.000045278459f, 0.000074847594f, 0.000111808491f, 0.000156160808f, 0.000207904115f, 
  0.000267037918f, 0.000333561708f, 0.000407474785f, 0.000488776539f, 0.000577466097f, 0.000673542731f, 0.000777005451f, 0.000887853326f, 
  0.001006085309f, 0.001131700235f, 0.001264696941f, 0.001405074261f, 0.001552830683f, 0.001707964926f, 0.001870475593f, 0.002040361054f, 
  0.002217619680f, 0.002402249724f, 0.002594249556f, 0.002793617314f, 0.003000351135f, 0.003214448923f, 0.003435908817f, 0.003664728720f, 
  0.003900906071f, 0.004144439008f, 0.004395325202f, 0.004653562326f, 0.004919147119f, 0.005192078184f, 0.005472352263f, 0.005759966560f, 
  0.006054918282f, 0.006357205100f, 0.006666823756f, 0.006983771455f, 0.007308044937f, 0.007639640942f, 0.007978556678f, 0.008324787952f, 
  0.008678332902f, 0.009039186873f, 0.009407347068f, 0.009782808833f, 0.010165570304f, 0.010555625893f, 0.010952972807f, 0.011357606389f, 
  0.011769523844f, 0.012188719586f, 0.012615191750f, 0.013048933819f, 0.013489942066f, 0.013938212767f, 0.014393741265f, 0.014856522903f, 
  0.015326553956f, 0.015803826973f, 0.016288341954f, 0.016780089587f, 0.017279066145f, 0.017785267904f, 0.018298687413f, 0.018819322810f, 
  0.019347166643f, 0.019882213324f, 0.020424459130f, 0.020973896608f, 0.021530522034f, 0.022094327956f, 0.022665310651f, 0.023243462667f, 
  0.023828778416f, 0.024421252310f, 0.025020876899f, 0.025627648458f, 0.026241557673f, 0.026862600818f, 0.027490770444f, 0.028126060963f, 
  0.028768463060f, 0.029417973012f, 0.030074583367f, 0.030738286674f, 0.031409073621f, 0.032086942345f, 0.032771881670f, 0.033463887870f, 
  0.034162946045f, 0.034869059920f, 0.035582210869f, 0.036302398890f, 0.037029616535f, 0.037763848901f, 0.038505095989f, 0.039253346622f, 
  0.040008589625f, 0.040770821273f, 0.041540030390f, 0.042316213250f, 0.043099354953f, 0.043889451772f, 0.044686492532f, 0.045490469784f, 
  0.046301372349f, 0.047119196504f, 0.047943927348f, 0.048775561154f, 0.049614083022f, 0.050459489226f, 0.051311764866f, 0.052170902491f, 
  0.053036894649f, 0.053909730166f, 0.054789397866f, 0.055675890297f, 0.056569192559f, 0.057469300926f, 0.058376204222f, 0.059289887547f, 
  0.060210339725f, 0.061137557030f, 0.062071524560f, 0.063012234867f, 0.063959665596f, 0.064913824201f, 0.065874688327f, 0.066842243075f, 
  0.067816480994f, 0.068797402084f, 0.069784976542f, 0.070779196918f, 0.071780063212f, 0.072787553072f, 0.073801659048f, 0.074822366238f, 
  0.075849659741f, 0.076883539557f, 0.077923975885f, 0.078970968723f, 0.080024495721f, 0.081084556878f, 0.082151129842f, 0.083224207163f, 
  0.084303766489f, 0.085389807820f, 0.086482301354f, 0.087581254542f, 0.088686637580f, 0.089798435569f, 0.090916648507f, 0.092041254044f, 
  0.093172237277f, 0.094309583306f, 0.095453277230f, 0.096603311598f, 0.097759671509f, 0.098922334611f, 0.100091286004f, 0.101266525686f, 
  0.102448016405f, 0.103635758162f, 0.104829736054f, 0.106029927731f, 0.107236325741f, 0.108448900282f, 0.109667651355f, 0.110892556608f, 
  0.112123593688f, 0.113360755146f, 0.114604026079f, 0.115853384137f, 0.117108814418f, 0.118370302022f, 0.119637832046f, 0.120911382139f, 
  0.122190937400f, 0.123476482928f, 0.124767996371f, 0.126065462828f, 0.127368867397f, 0.128678202629f, 0.129993423820f, 0.131314530969f, 
  0.132641509175f, 0.133974328637f, 0.135312989354f, 0.136657446623f, 0.138007700443f, 0.139363735914f, 0.140725508332f, 0.142093032598f, 
  0.143466264009f, 0.144845202565f, 0.146229818463f, 0.147620081902f, 0.149015992880f, 0.150417521596f, 0.151824653149f, 0.153237357736f, 
  0.154655620456f, 0.156079426408f, 0.157508760691f, 0.158943593502f, 0.160383895040f, 0.161829650402f, 0.163280844688f, 0.164737462997f, 
  0.166199475527f, 0.167666852474f, 0.169139593840f, 0.170617654920f, 0.172101020813f, 0.173589676619f, 0.175083607435f, 0.176582768559f, 
  0.178087159991f, 0.179596751928f, 0.181111514568f, 0.182631418109f, 0.184156462550f, 0.185686618090f, 0.187221854925f, 0.188762158155f, 
  0.190307497978f, 0.191857844591f, 0.193413197994f, 0.194973513484f, 0.196538761258f, 0.198108941317f, 0.199684008956f, 0.201263949275f, 
  0.202848747373f, 0.204438358545f, 0.206032782793f, 0.207631960511f, 0.209235906601f, 0.210844561458f, 0.212457925081f, 0.214075952768f, 
  0.215698629618f, 0.217325940728f, 0.218957841396f, 0.220594316721f, 0.222235336900f, 0.223880872130f, 0.225530907512f, 0.227185413241f, 
  0.228844344616f, 0.230507701635f, 0.232175439596f, 0.233847543597f, 0.235523983836f, 0.237204730511f, 0.238889753819f, 0.240579038858f, 
  0.242272540927f, 0.243970230222f, 0.245672106743f, 0.247378110886f, 0.249088242650f, 0.250802457333f, 0.252520710230f, 0.254243016243f, 
  0.255969315767f, 0.257699608803f, 0.259433805943f, 0.261171936989f, 0.262913972139f, 0.264659851789f, 0.266409546137f, 0.268163055182f, 
  0.269920319319f, 0.271681338549f, 0.273446083069f, 0.275214463472f, 0.276986539364f, 0.278762221336f, 0.280541479588f, 0.282324314117f, 
  0.284110665321f, 0.285900533199f, 0.287693858147f, 0.289490610361f, 0.291290789843f, 0.293094336987f, 0.294901251793f, 0.296711444855f, 
  0.298524945974f, 0.300341695547f, 0.302161663771f, 0.303984820843f, 0.305811136961f, 0.307640582323f, 0.309473127127f, 0.311308711767f, 
  0.313147366047f, 0.314988970757f, 0.316833585501f, 0.318681091070f, 0.320531517267f, 0.322384834290f, 0.324240952730f, 0.326099902391f, 
  0.327961593866f, 0.329826027155f, 0.331693172455f, 0.333562999964f, 0.335435450077f, 0.337310492992f, 0.339188128710f, 0.341068297625f, 
  0.342950969934f, 0.344836086035f, 0.346723675728f, 0.348613649607f, 0.350505977869f, 0.352400660515f, 0.354297637939f, 0.356196880341f, 
  0.358098357916f, 0.360002011061f, 0.361907839775f, 0.363815814257f, 0.365725874901f, 0.367637962103f, 0.369552105665f, 0.371468245983f, 
  0.373386323452f, 0.375306308270f, 0.377228170633f, 0.379151910543f, 0.381077468395f, 0.383004784584f, 0.384933829308f, 0.386864602566f, 
  0.388797044754f, 0.390731126070f, 0.392666786909f, 0.394604027271f, 0.396542817354f, 0.398483067751f, 0.400424808264f, 0.402367949486f, 
  0.404312461615f, 0.406258344650f, 0.408205538988f, 0.410154014826f, 0.412103712559f, 0.414054632187f, 0.416006714106f, 0.417959928513f, 
  0.419914215803f, 0.421869605780f, 0.423825979233f, 0.425783365965f, 0.427741676569f, 0.429700940847f, 0.431661039591f, 0.433622002602f, 
  0.435583770275f, 0.437546283007f, 0.439509540796f, 0.441473484039f, 0.443438082933f, 0.445403307676f, 0.447369098663f, 0.449335455894f, 
  0.451302319765f, 0.453269630671f, 0.455237388611f, 0.457205563784f, 0.459174066782f, 0.461142897606f, 0.463112026453f, 0.465081423521f, 
  0.467050999403f, 0.469020754099f, 0.470990657806f, 0.472960650921f, 0.474930733442f, 0.476900815964f, 0.478870898485f, 0.480840921402f, 
  0.482810884714f, 0.484780699015f, 0.486750364304f, 0.488719820976f, 0.490689069033f, 0.492658019066f, 0.494626671076f, 0.496594995260f, 
  0.498562902212f, 0.500530421734f, 0.502497494221f, 0.504464030266f, 0.506430029869f, 0.508395493031f, 0.510360360146f, 0.512324571609f, 
  0.514288127422f, 0.516250908375f, 0.518212974072f, 0.520174264908f, 0.522134721279f, 0.524094343185f, 0.526053011417f, 0.528010785580f, 
  0.529967546463f, 0.531923294067f, 0.533878028393f, 0.535831689835f, 0.537784218788f, 0.539735555649f, 0.541685760021f, 0.543634653091f, 
  0.545582354069f, 0.547528743744f, 0.549473762512f, 0.551417410374f, 0.553359687328f, 0.555300474167f, 0.557239830494f, 0.559177637100f, 
  0.561113893986f, 0.563048541546f, 0.564981579781f, 0.566912949085f, 0.568842649460f, 0.570770561695f, 0.572696745396f, 0.574621140957f, 
  0.576543688774f, 0.578464329243f, 0.580383062363f, 0.582299888134f, 0.584214687347f, 0.586127519608f, 0.588038265705f, 0.589946925640f, 
  0.591853499413f, 0.593757927418f, 0.595660090446f, 0.597560107708f, 0.599457859993f, 0.601353287697f, 0.603246390820f, 0.605137169361f, 
  0.607025504112f, 0.608911454678f, 0.610794901848f, 0.612675905228f, 0.614554345608f, 0.616430222988f, 0.618303537369f, 0.620174169540f, 
  0.622042179108f, 0.623907446861f, 0.625770032406f, 0.627629876137f, 0.629486918449f, 0.631341099739f, 0.633192420006f, 0.635040879250f, 
  0.636886417866f, 0.638728976250f, 0.640568554401f, 0.642405092716f, 0.644238650799f, 0.646069049835f, 0.647896409035f, 0.649720549583f, 
  0.651541590691f, 0.653359353542f, 0.655173897743f, 0.656985223293f, 0.658793210983f, 0.660597860813f, 0.662399172783f, 0.664197087288f, 
  0.665991544724f, 0.667782604694f, 0.669570147991f, 0.671354234219f, 0.673134744167f, 0.674911677837f, 0.676685035229f, 0.678454756737f, 
  0.680220842361f, 0.681983232498f, 0.683741927147f, 0.685496866703f, 0.687248051167f, 0.688995420933f, 0.690738976002f, 0.692478656769f, 
  0.694214463234f, 0.695946395397f, 0.697674393654f, 0.699398398399f, 0.701118409634f, 0.702834486961f, 0.704546451569f, 0.706254363060f, 
  0.707958161831f, 0.709657907486f, 0.711353421211f, 0.713044822216f, 0.714732050896f, 0.716414988041f, 0.718093752861f, 0.719768226147f, 
  0.721438348293f, 0.723104178905f, 0.724765717983f, 0.726422846317f, 0.728075563908f, 0.729723930359f, 0.731367826462f, 0.733007252216f, 
  0.734642148018f, 0.736272633076f, 0.737898528576f, 0.739519834518f, 0.741136610508f, 0.742748796940f, 0.744356334209f, 0.745959281921f, 
  0.747557520866f, 0.749151110649f, 0.750739932060f, 0.752324104309f, 0.753903448582f, 0.755478084087f, 0.757047891617f, 0.758612930775f, 
  0.760173141956f, 0.761728465557f, 0.763278961182f, 0.764824569225f, 0.766365230083f, 0.767901003361f, 0.769431829453f, 0.770957648754f, 
  0.772478520870f, 0.773994386196f, 0.775505304337f, 0.777011096478f, 0.778511881828f, 0.780007600784f, 0.781498193741f, 0.782983720303f, 
  0.784464120865f, 0.785939395428f, 0.787409543991f, 0.788874506950f, 0.790334284306f, 0.791788876057f, 0.793238222599f, 0.794682323933f, 
  0.796121239662f, 0.797554910183f, 0.798983275890f, 0.800406336784f, 0.801824152470f, 0.803236603737f, 0.804643750191f, 0.806045532227f, 
  0.807442009449f, 0.808833122253f, 0.810218811035f, 0.811599135399f, 0.812974095345f, 0.814343571663f, 0.815707683563f, 0.817066311836f, 
  0.818419575691f, 0.819767296314f, 0.821109592915f, 0.822446405888f, 0.823777675629f, 0.825103521347f, 0.826423823833f, 0.827738642693f, 
  0.829047858715f, 0.830351591110f, 0.831649780273f, 0.832942426205f, 0.834229469299f, 0.835510969162f, 0.836786866188f, 0.838057219982f, 
  0.839321970940f, 0.840581119061f, 0.841834604740f, 0.843082547188f, 0.844324827194f, 0.845561504364f, 0.846792519093f, 0.848017930984f, 
  0.849237680435f, 0.850451767445f, 0.851660251617f, 0.852863013744f, 0.854060113430f, 0.855251610279f, 0.856437385082f, 0.857617497444f, 
  0.858791887760f, 0.859960615635f, 0.861123681068f, 0.862281024456f, 0.863432705402f, 0.864578664303f, 0.865718960762f, 0.866853535175f, 
  0.867982387543f, 0.869105517864f, 0.870222985744f, 0.871334731579f, 0.872440814972f, 0.873541116714f, 0.874635756016f, 0.875724673271f, 
  0.876807868481f, 0.877885341644f, 0.878957092762f, 0.880023181438f, 0.881083548069f, 0.882138192654f, 0.883187115192f, 0.884230375290f, 
  0.885267913342f, 0.886299729347f, 0.887325882912f, 0.888346314430f, 0.889361023903f, 0.890370070934f, 0.891373455524f, 0.892371118069f, 
  0.893363118172f, 0.894349396229f, 0.895330071449f, 0.896305024624f, 0.897274315357f, 0.898237943649f, 0.899195909500f, 0.900148212910f, 
  0.901094853878f, 0.902035892010f, 0.902971267700f, 0.903901040554f, 0.904825150967f, 0.905743658543f, 0.906656503677f, 0.907563805580f, 
  0.908465445042f, 0.909361541271f, 0.910252034664f, 0.911136925220f, 0.912016272545f, 0.912890076637f, 0.913758277893f, 0.914620935917f, 
  0.915478050709f, 0.916329622269f, 0.917175710201f, 0.918016254902f, 0.918851256371f, 0.919680833817f, 0.920504868031f, 0.921323418617f, 
  0.922136545181f, 0.922944188118f, 0.923746347427f, 0.924543142319f, 0.925334453583f, 0.926120340824f, 0.926900863647f, 0.927675962448f, 
  0.928445637226f, 0.929210007191f, 0.929969012737f, 0.930722594261f, 0.931470930576f, 0.932213902473f, 0.932951509953f, 0.933683872223f, 
  0.934410929680f, 0.935132682323f, 0.935849249363f, 0.936560511589f, 0.937266528606f, 0.937967300415f, 0.938662886620f, 0.939353287220f, 
  0.940038502216f, 0.940718591213f, 0.941393435001f, 0.942063212395f, 0.942727804184f, 0.943387329578f, 0.944041728973f, 0.944691061974f, 
  0.945335328579f, 0.945974528790f, 0.946608722210f, 0.947237849236f, 0.947861969471f, 0.948481142521f, 0.949095308781f, 0.949704527855f, 
  0.950308859348f, 0.950908184052f, 0.951502621174f, 0.952092170715f, 0.952676892281f, 0.953256666660f, 0.953831672668f, 0.954401850700f, 
  0.954967200756f, 0.955527722836f, 0.956083536148f, 0.956634581089f, 0.957180857658f, 0.957722425461f, 0.958259284496f, 0.958791494370f, 
  0.959318995476f, 0.959841907024f, 0.960360169411f, 0.960873782635f, 0.961382865906f, 0.961887359619f, 0.962387263775f, 0.962882637978f, 
  0.963373541832f, 0.963859915733f, 0.964341819286f, 0.964819252491f, 0.965292274952f, 0.965760886669f, 0.966225087643f, 0.966684877872f, 
  0.967140376568f, 0.967591464520f, 0.968038260937f, 0.968480765820f, 0.968919038773f, 0.969352960587f, 0.969782710075f, 0.970208227634f, 
  0.970629572868f, 0.971046686172f, 0.971459686756f, 0.971868515015f, 0.972273290157f, 0.972673952579f, 0.973070502281f, 0.973462998867f, 
  0.973851501942f, 0.974236011505f, 0.974616527557f, 0.974993050098f, 0.975365638733f, 0.975734293461f, 0.976099073887f, 0.976459980011f, 
  0.976817011833f, 0.977170228958f, 0.977519571781f, 0.977865219116f, 0.978206992149f, 0.978545069695f, 0.978879451752f, 0.979210078716f, 
  0.979537069798f, 0.979860365391f, 0.980180025101f, 0.980496048927f, 0.980808496475f, 0.981117367744f, 0.981422722340f, 0.981724560261f, 
  0.982022821903f, 0.982317626476f, 0.982609033585f, 0.982896924019f, 0.983181476593f, 0.983462572098f, 0.983740329742f, 0.984014689922f, 
  0.984285771847f, 0.984553575516f, 0.984818041325f, 0.985079288483f, 0.985337316990f, 0.985592126846f, 0.985843718052f, 0.986092150211f, 
  0.986337423325f, 0.986579656601f, 0.986818730831f, 0.987054705620f, 0.987287640572f, 0.987517595291f, 0.987744510174f, 0.987968444824f, 
  0.988189399242f, 0.988407433033f, 0.988622546196f, 0.988834798336f, 0.989044129848f, 0.989250659943f, 0.989454329014f, 0.989655196667f, 
  0.989853322506f, 0.990048646927f, 0.990241229534f, 0.990431129932f, 0.990618348122f, 0.990802884102f, 0.990984797478f, 0.991164088249f, 
  0.991340756416f, 0.991514861584f, 0.991686403751f, 0.991855442524f, 0.992021918297f, 0.992185950279f, 0.992347478867f, 0.992506623268f, 
  0.992663323879f, 0.992817580700f, 0.992969512939f, 0.993119060993f, 0.993266344070f, 0.993411242962f, 0.993553876877f, 0.993694245815f, 
  0.993832409382f, 0.993968307972f, 0.994102001190f, 0.994233548641f, 0.994362950325f, 0.994490206242f, 0.994615316391f, 0.994738340378f, 
  0.994859337807f, 0.994978249073f, 0.995095133781f, 0.995210051537f, 0.995322942734f, 0.995433926582f, 0.995542943478f, 0.995649993420f, 
  0.995755195618f, 0.995858490467f, 0.995959937572f, 0.996059596539f, 0.996157407761f, 0.996253371239f, 0.996347606182f, 0.996440112591f, 
  0.996530830860f, 0.996619880199f, 0.996707201004f, 0.996792912483f, 0.996876895428f, 0.996959328651f, 0.997040092945f, 0.997119247913f, 
  0.997196853161f, 0.997272968292f, 0.997347474098f, 0.997420489788f, 0.997492015362f, 0.997562050819f, 0.997630655766f, 0.997697830200f, 
  0.997763574123f, 0.997827887535f, 0.997890889645f, 0.997952461243f, 0.998012781143f, 0.998071730137f, 0.998129367828f, 0.998185694218f, 
  0.998240828514f, 0.998294651508f, 0.998347282410f, 0.998398661613f, 0.998448908329f, 0.998497903347f, 0.998545825481f, 0.998592555523f, 
  0.998638153076f, 0.998682677746f, 0.998726129532f, 0.998768448830f, 0.998809754848f, 0.998850047588f, 0.998889267445f, 0.998927533627f, 
  0.998964786530f, 0.999001085758f, 0.999036371708f, 0.999070763588f, 0.999104261398f, 0.999136865139f, 0.999168515205f, 0.999199330807f, 
  0.999229311943f, 0.999258399010f, 0.999286711216f, 0.999314188957f, 0.999340832233f, 0.999366760254f, 0.999391913414f, 0.999416291714f, 
  0.999439954758f, 0.999462842941f, 0.999485075474f, 0.999506592751f, 0.999527454376f, 0.999547660351f, 0.999567210674f, 0.999586105347f, 
  0.999604403973f, 0.999622046947f, 0.999639153481f, 0.999655604362f, 0.999671578407f, 0.999686956406f, 0.999701738358f, 0.999716043472f, 
  0.999729812145f, 0.999743103981f, 0.999755918980f, 0.999768197536f, 0.999779999256f, 0.999791383743f, 0.999802350998f, 0.999812841415f, 
  0.999822914600f, 0.999832570553f, 0.999841809273f, 0.999850690365f, 0.999859213829f, 0.999867320061f, 0.999875128269f, 0.999882519245f, 
  0.999889612198f, 0.999896407127f, 0.999902844429f, 0.999908983707f, 0.999914884567f, 0.999920427799f, 0.999925732613f, 0.999930739403f, 
  0.999935507774f, 0.999940037727f, 0.999944269657f, 0.999948322773f, 0.999952137470f, 0.999955773354f, 0.999959170818f, 0.999962329865f, 
  0.999965369701f, 0.999968171120f, 0.999970793724f, 0.999973297119f, 0.999975621700f, 0.999977767467f, 0.999979794025f, 0.999981641769f, 
  0.999983429909f, 0.999985039234f, 0.999986529350f, 0.999987900257f, 0.999989151955f, 0.999990344048f, 0.999991416931f, 0.999992370605f, 
  0.999993264675f, 0.999994099140f, 0.999994814396f, 0.999995470047f, 0.999996125698f, 0.999996662140f, 0.999997138977f, 0.999997556210f, 
  0.999997913837f, 0.999998271465f, 0.999998569489f, 0.999998807907f, 0.999998986721f, 0.999999225140f, 0.999999344349f, 0.999999523163f, 
  0.999999582767f, 0.999999701977f, 0.999999761581f, 0.999999821186f, 0.999999880791f, 0.999999940395f, 0.999999940395f, 0.999999940395f, 
  1.000000000000f, 1.000000000000f, 1.000000000000f, 1.000000000000f, 1.000000000000f, 1.000000000000f, 1.000000000000f, 1.000000000000f, 
};
static float window_short[128] =
{
  0.000059139038f, 0.000532197882f, 0.001478030114f, 0.002896063495f, 0.004785436206f, 0.007144992705f, 0.009973277338f, 0.013268529437f, 
  0.017028674483f, 0.021251311526f, 0.025933710858f, 0.031072795391f, 0.036665130407f, 0.042706914246f, 0.049193959683f, 0.056121692061f, 
  0.063485108316f, 0.071278803051f, 0.079496912658f, 0.088133141398f, 0.097180701792f, 0.106632351875f, 0.116480343044f, 0.126716434956f, 
  0.137331858277f, 0.148317337036f, 0.159663051367f, 0.171358674765f, 0.183393299580f, 0.195755511522f, 0.208433344960f, 0.221414253116f, 
  0.234685227275f, 0.248232662678f, 0.262042462826f, 0.276100039482f, 0.290390282869f, 0.304897606373f, 0.319605946541f, 0.334498882294f, 
  0.349559515715f, 0.364770591259f, 0.380114465952f, 0.395573228598f, 0.411128699780f, 0.426762402058f, 0.442455708981f, 0.458189755678f, 
  0.473945677280f, 0.489704459906f, 0.505447089672f, 0.521154582500f, 0.536808073521f, 0.552388727665f, 0.567878007889f, 0.583257555962f, 
  0.598509252071f, 0.613615453243f, 0.628558754921f, 0.643322288990f, 0.657889604568f, 0.672244906425f, 0.686372935772f, 0.700258910656f, 
  0.713888943195f, 0.727249741554f, 0.740328788757f, 0.753114342690f, 0.765595495701f, 0.777762115002f, 0.789605021477f, 0.801115870476f, 
  0.812287271023f, 0.823112726212f, 0.833586633205f, 0.843704402447f, 0.853462278843f, 0.862857580185f, 0.871888458729f, 0.880554080009f, 
  0.888854384422f, 0.896790385246f, 0.904363751411f, 0.911577284336f, 0.918434441090f, 0.924939453602f, 0.931097447872f, 0.936914145947f, 
  0.942396163940f, 0.947550535202f, 0.952385127544f, 0.956908285618f, 0.961128890514f, 0.965056359768f, 0.968700468540f, 0.972071409225f, 
  0.975179851055f, 0.978036582470f, 0.980652749538f, 0.983039617538f, 0.985208690166f, 0.987171590328f, 0.988939821720f, 0.990525066853f, 
  0.991939008236f, 0.993192970753f, 0.994298517704f, 0.995266735554f, 0.996108710766f, 0.996835112572f, 0.997456431389f, 0.997982800007f, 
  0.998423933983f, 0.998789250851f, 0.999087631702f, 0.999327600002f, 0.999517142773f, 0.999663650990f, 0.999774158001f, 0.999854981899f, 
  0.999911963940f, 0.999950289726f, 0.999974489212f, 0.999988555908f, 0.999995827675f, 0.999998927116f, 0.999999880791f, 1.000000000000f, 
};
