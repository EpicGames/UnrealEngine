// Copyright Epic Games Tools, LLC. All Rights Reserved.
////////////////////////////////////////////////////////////////////////////
//
// RADaudio is a new audio codec made by Epic Game Tools for use in games,
// optimized for fast SIMD decoding and decent quality (roughly similar to
// Vorbis).
//
// It is a classical MDCT-based codec with two block sizes, and it uses
// the Oodle Data huffman entropy coder to store data.

#include <stdio.h>
#include <stdlib.h>
#include <math.h>

// We want the external defines to be scoped to DECODER, but we don't want to rename everything in here.
#define RADAUDIO_AT_EOF                   RADAUDIO_DECODER_AT_EOF
#define RADAUDIO_INCOMPLETE_DATA          RADAUDIO_DECODER_INCOMPLETE_DATA
#define RADAUDIO_INVALID_DATA             RADAUDIO_DECODER_INVALID_DATA
#define RADAUDIO_START_OF_STREAM          RADAUDIO_DECODER_START_OF_STREAM
#define RADAUDIO_INTERNAL_ERROR           RADAUDIO_DECODER_INTERNAL_ERROR

#define HUFFMAN_DECODE // enable huffman decode tables
#include "radaudio_decoder.h"
#include "radaudio_decoder_sse2.h"
#include "radaudio_decoder_sse4.h"
#include "radaudio_decoder_avx2.h"
#include "radaudio_decoder_neon.h"
#include "radaudio_decoder_internal.h"
#include "radaudio_common.h"
#include "radaudio_mdct.h"
#include "rrCore.h"
#include "rrbits.h"

#include "radaudio_common.inl"

RR_COMPILER_ASSERT(COMMON_INVALID_DATA == RADAUDIO_INVALID_DATA);
RR_COMPILER_ASSERT(COMMON_INCOMPLETE_DATA == RADAUDIO_INCOMPLETE_DATA);
RR_COMPILER_ASSERT(RADAUDIO_STREAM_HEADER_SIZE == sizeof(radaudio_stream_header));

#ifdef RADAUDIO_DEVELOPMENT
#define PROFILE_ZONES()        \
   PROF(decoder_all)           \
   PROF(imdct)                 \
   PROF(window)                \
   PROF(huffman)               \
   PROF(unquantize)            \
   PROF(distribute_rle)        \
   PROF(update_runlength)      \
   PROF(varbits)               \
   PROF(compute_mantissa_len)  \
   PROF(copy)                  \
   PROF(compute_subbands)      \
   PROF(distribute_bitflag)    \
   PROF(unpack)                \
   PROF(randomize)             \
   PROF(compute_subband_energy)\
   PROF(unbias)                \
   PROF(compute_band_energy)   \
   PROF(count_coefficients_huff) \
   PROF(header)                \
   PROF(zero)                  \
   PROF(flagbits)              \
   PROF(overhead)              /* must always be last! */ \
   PROF(total_count)

enum
{
   #define PROF(x) PROF_##x,
   PROFILE_ZONES()
   #undef PROF

   PROF__end
};

#define PROF_BEGIN(var)       if (profile) { profile_times[PROF_##var] -= rrGetTicks(); profile_counts[PROF_##var] += 1; }
#define PROF_END(var)         if (profile) profile_times[PROF_##var] += rrGetTicks()

static rrbool profile;
static U64 profile_times[PROF_total_count];
static S64 profile_counts[PROF_total_count];
#else
#define PROF_BEGIN(var)
#define PROF_END(var)
#define PROF_total_count    1
#endif

#define RANDVAL(r,i)        (r)

// allow breakpointing on any error
static int e(int code)
{
   return code;
}

typedef struct RadAudioDecoder
{
   U32    version;
   rrbool current_block_short;
   rrbool last_block_short;
   rrbool next_block_short; // we don't actually need this
   int    samprate_mode;
   int    num_channels;

   int    skip_bytes;
   int    sample_rate;      // implied by samprate_mode
   U32    block_number;
   int    fully_decoded; // sample offset in stream
   U8     subband_predicted_sum[MAX_BANDS];
   S8     mantissa_param[2][MAX_BANDS][2];
   S8     subband_bias[MAX_BANDS];
   rrbool at_eof;
   rrbool post_seek;

   rrbool bitstream_overshot;
   radaudio_block_header_biases biases;
   radaudio_cpu_features cpu;

   radaudio_rate_info * info[2];  // pre-defined table, indexed by long vs. short block
   radaudio_nonzero_blockmode_descriptor nz_desc[NUM_NZ_MODE];
   U8 nz_correlated_huffman_selectors[NUM_NZ_SELECTOR][NUM_SELECTOR_MODES];

   S16 * prev_block_right_samples[2];
   F32   restore_scale[2]; // how to convert S16s back to floats
} radaudio_decoder_state;

//////////////////////////////////////////////////////////////////////////////
//
//        ENTROPY DECODER
//

typedef struct
{
   U8 *bitstream;
   U8 *end;
   U32 read_pos_in_bits;
   U32 fast_num_bits; // if initial read_pos_in_bits < this, can take fast path (needs to be < not <= so 0 disables fast path)
   U32 total_num_bits;
} rada_bit_decoder;

typedef struct
{
   rada_bit_decoder stream[3];
} huff3_decoder;

static void decode_vbstream_init(rada_bit_decoder *d, U8 *bitstream, U8 *end, int *error)
{
   if (bitstream > end) {
      // this is a backwards stream (for huffman). not allowed to read through
      // decode_vbstream_bits, so make sure we can't by setting total bits count to 0.
      d->bitstream = bitstream;
      d->end = end;
      d->read_pos_in_bits = 0;
      d->fast_num_bits = 0;
      d->total_num_bits = 0;
      return;
   }

   d->bitstream = bitstream;
   d->end = end;
   d->read_pos_in_bits = 0;

   size_t num_bytes = end - bitstream;
   if (num_bytes > MAX_ENCODED_BLOCK_BYTES) {
      // not allowed! set num_bytes to 0 and initialize stream as empty
      num_bytes = 0;
      *error = 1;
   }

   d->total_num_bits = (U32) (num_bytes * 8); // can't overflow: num_bytes checked above
   if (d->total_num_bits >= 32) {
      d->fast_num_bits = d->total_num_bits - 32;
   } else {
      d->fast_num_bits = 0;
   }
}

// bit reading cold path, reads one byte at a time to avoid over-reading
static RADNOINLINE U32 decode_vbstream_bits_cold(rada_bit_decoder *d, int bitlength, int *error)
{
   // check whether actual data required goes off the end
   if (d->read_pos_in_bits + bitlength > d->total_num_bits) {
      *error = 1;
      return 0;
   }

   // can read 0 bits exactly at the end
   if (bitlength == 0)
      return 0;

   // if not, read as many valid bits as exist, then mask
   size_t first_byte = (d->read_pos_in_bits >> 3);
   U32 bits = d->bitstream[first_byte++];
   U32 shift = 8;
   while (d->bitstream+first_byte < d->end) {
      bits = bits + (d->bitstream[first_byte++] << shift);
      shift += 8;
   }
   bits >>= (d->read_pos_in_bits & 7);
   bits &= (1 << bitlength)-1;
   d->read_pos_in_bits += bitlength;
   return bits;
}

static RADFORCEINLINE U32 decode_vbstream_bits(rada_bit_decoder *d, int bitlength, int *error)
{
   // check for reading off the end...
   if (d->read_pos_in_bits < d->fast_num_bits) {
      // simple path
      size_t first_byte = (d->read_pos_in_bits >> 3);
      U32 bits = RR_GET32_LE(d->bitstream + first_byte);
      bits >>= (d->read_pos_in_bits & 7); // discard bits we're pointing past
      bits &= (1 << bitlength)-1;
      d->read_pos_in_bits += bitlength;
      return bits;
   } else {
      return decode_vbstream_bits_cold(d, bitlength, error);
   }
}

typedef struct
{
   U8 *decodeptr;          // Current write cursor for the two stream triples
   U8 *decodeend;          // End of decoded bytes buffer for the two stream triples

   const U8 *bitp[3];      // Next byte to be read for the streams
   U32 bits[3];            // Current contents of bit buffer
   U32 bitc[3];            // Current number of valid bits in bit buffer
} rada_internal_huff_state;

#define NEWLZ_HUFF_CODELEN_LIMIT 11
#define NEWLZ_HUFF_DECODE_TABLE_MASK  2047u

// 32-bit ARM implicitly masks 32-bit shift amounts by 255 (low 8 bits).
// All other current targets implicitly mask by 31 (low 5 bits). Either
// works for us, but we'd prefer not to get an extra AND, so use whatever
// the implicit mask is and rely on the compiler to clean it up.
#if defined(__RADARM__) && !defined(__RAD64__)
#define HUFF32LENMASK 255
#else
#define HUFF32LENMASK 31
#endif

static rrbool huff_decode_precise_finish(rada_internal_huff_state * s, radaudio_huffman *huff)
{
   const U8 * in0 = s->bitp[0];
   const U8 * in1 = s->bitp[1];
   const U8 * in2 = s->bitp[2];

   U32 bits0 = s->bits[0], bitc0 = s->bitc[0];
   U32 bits1 = s->bits[1], bitc1 = s->bitc[1];
   U32 bits2 = s->bits[2], bitc2 = s->bitc[2];

   if (in0 > in2)
      return false;

   U8 *decodeptr = s->decodeptr;
   U8 *decodeend = s->decodeend;

   #define DECONE(strm) \
      peek = bits##strm & NEWLZ_HUFF_DECODE_TABLE_MASK; \
      cl = huff->decode[peek].length; \
      sym = huff->decode[peek].symbol; \
      bits##strm >>= cl & HUFF32LENMASK; bitc##strm -= cl; \
      *decodeptr++ = (U8) sym

   #define DECTHREE() \
      DECONE(0); \
      DECONE(1); \
      DECONE(2)

   RR_COMPILER_ASSERT( NEWLZ_HUFF_CODELEN_LIMIT <= 12 );   
   #define N_DECS_PER_REFILL      2
   #define TRIPLE_DECS_PER_REFILL   (3*N_DECS_PER_REFILL)

   // bulk loop to get within 4B of end
   if (in1 - in2 >= 4 && decodeend - decodeptr >= TRIPLE_DECS_PER_REFILL)
   {
      in1 -= 4;
      decodeend -= TRIPLE_DECS_PER_REFILL-1;

      while (decodeptr < decodeend)
      {
         // non-crossing invariant: in0 <= in2 && in2 <= in1
         if (in0 > in2 || in2 > in1)
            break;

         // non-crossing and 4B access size guarantee that the
         // following reads are safe; the decodeend decrement before the
         // loop guarantees that we don't write out of bounds.

         // refill :
         bits0 |= RR_GET32_LE(in0) << bitc0;
         in0 += (31 - bitc0)>>3; // bytes_consumed
         bitc0 |= 24; // same as += bytes_consumed<<3 here!

         bits1 |= RR_GET32_BE(in1) << bitc1;
         in1 -= (31 - bitc1)>>3; // bytes_consumed
         bitc1 |= 24; // same as += bytes_consumed<<3 here!

         bits2 |= RR_GET32_LE(in2) << bitc2;
         in2 += (31 - bitc2)>>3; // bytes_consumed
         bitc2 |= 24; // same as += bytes_consumed<<3 here!

         U32 peek; int cl; int sym;
         
         RR_COMPILER_ASSERT( N_DECS_PER_REFILL == 2 );
         DECTHREE();
         DECTHREE();
      }

      decodeend += TRIPLE_DECS_PER_REFILL-1;
      in1 += 4;

      // transition to final loop
      in0 -= (bitc0 >> 3); bitc0 &= 7;
      in1 += (bitc1 >> 3); bitc1 &= 7;
      in2 -= (bitc2 >> 3); bitc2 &= 7;
   }

   // Final loop. This is really careful about the bytes it accesses.
   while (decodeptr < decodeend)
   {
      U32 peek, cl, sym;

      // refill to >=16b in bit0 buf
      if (in2 - in0 > 1)
         bits0 |= RR_GET16_LE(in0) << bitc0;
      else if (in2 - in0 == 1)
         bits0 |= in0[0] << bitc0;

      DECONE(0);
      in0 += (7 - bitc0) >> 3;
      bitc0 &= 7;

      if (decodeptr >= decodeend)
         break;

      // refill to >=16b left in bit1, bit2 bufs
      if (in1 - in2 > 1)
      {
         bits1 |= RR_GET16_BE(in1 - 2) << bitc1;
         bits2 |= RR_GET16_LE(in2) << bitc2;
      }
      else if (in1 - in2 == 1)
      {
         // accessing the same byte!
         bits1 |= in2[0] << bitc1;
         bits2 |= in2[0] << bitc2;
      }

      DECONE(1);
      in1 -= (7 - bitc1) >> 3;
      bitc1 &= 7;

      if (decodeptr >= decodeend)
         break;

      DECONE(2);
      in2 += (7 - bitc2) >> 3;
      bitc2 &= 7;

      if (in0 > in2 || in2 > in1) // corruption check
         return false;
   }

   if (decodeptr != decodeend)
      return false;

   #undef DECONE
   #undef DECTHREE
   #undef N_DECS_PER_REFILL
   #undef TRIPLE_DECS_PER_REFILL

   s->bitp[0] = in0; s->bits[0] = bits0; s->bitc[0] = bitc0;
   s->bitp[1] = in1; s->bits[1] = bits1; s->bitc[1] = bitc1;
   s->bitp[2] = in2; s->bits[2] = bits2; s->bitc[2] = bitc2;

   return true;
}

#if defined(__RAD64REGS__)
static rrbool huff_decode_inner64(rada_internal_huff_state * s, radaudio_huffman *huff)
{
   // Layout: strm0-> | strm2-> | <-strm1
   const U8 * in0 = s->bitp[0];
   const U8 * in1 = s->bitp[1];
   const U8 * in2 = s->bitp[2];

   U8 * decodeptr = s->decodeptr;
   U8 * decodeend = s->decodeend;

   // NEWLZ_HUFF_CODELEN_LIMIT == 11 , could actually do 5 per refill = 10 per loop
   #if (56/NEWLZ_HUFF_CODELEN_LIMIT) >= 5
   #define N_DECS_PER_REFILL      5
   #elif (56/NEWLZ_HUFF_CODELEN_LIMIT) >= 4
   #define N_DECS_PER_REFILL      4
   #else
   #define N_DECS_PER_REFILL      3
   #endif
   #define TRIPLE_DECS_PER_REFILL   (3*N_DECS_PER_REFILL)

   // bulk loop
   if (decodeend - decodeptr > TRIPLE_DECS_PER_REFILL-1 && in1 - in2 > 8) // @TODO: maybe test for going outside the buffer instead of this, since this might be true too often
   {
      // offset the end marker so we only run with full groups left
      decodeend -= TRIPLE_DECS_PER_REFILL-1;
      in1 -= 8;

      U64 bits0=s->bits[0], bitcount0 = s->bitc[0];
      U64 bits1=s->bits[1], bitcount1 = s->bitc[1];
      U64 bits2=s->bits[2], bitcount2 = s->bitc[2];
      const U8 *hufftab_base = &huff->decode[0].length;

      #define DECONE(strm) \
         /* NOTE(fg): This address calc is a single UBFIZ */ \
         tabv = (bits##strm & NEWLZ_HUFF_DECODE_TABLE_MASK) * sizeof(radaudio_huff_symbol); \
         tabv = RR_GET16_LE((const U16 *) (hufftab_base + tabv)); \
         bits##strm >>= tabv & 63; bitcount##strm -= tabv; \
         *decodeptr++ = (U8) (tabv >> 8)

      #define DECTHREE() \
         DECONE(0); \
         DECONE(1); \
         DECONE(2)

      while (decodeptr < decodeend)
      {
         // non-crossing invariant: in0 <= in2 && in2 <= in1
         if (in0 > in2) // if_unlikely
            break;
         if (in2 > in1) // if_unlikely
            break;

         // refill :
         U64 next0 = RR_GET64_LE(in0);
         bits0 |= next0 << bitcount0;
         in0 += (63 - bitcount0)>>3; // bytes_consumed
         bitcount0 |= 56; // same as += bytes_consumed<<3 here!

         U64 next1 = RR_GET64_BE(in1);
         bits1 |= next1 << bitcount1;
         in1 -= (63 - bitcount1)>>3; // bytes_consumed
         bitcount1 |= 56; // same as += bytes_consumed<<3 here!

         U64 next2 = RR_GET64_LE(in2);
         bits2 |= next2 << bitcount2;
         in2 += (63 - bitcount2)>>3; // bytes_consumed
         bitcount2 |= 56; // same as += bytes_consumed<<3 here!

         U32 tabv;
         
         RR_COMPILER_ASSERT( N_DECS_PER_REFILL >= 3 && N_DECS_PER_REFILL <= 5 );
         DECTHREE();
         DECTHREE();
         DECTHREE();
         #if N_DECS_PER_REFILL > 3
         DECTHREE();
         #endif
         #if N_DECS_PER_REFILL > 4
         DECTHREE();
         #endif

         // our decode process puts some crap in the top bits; clear them
         bitcount0 &= 63;
         bitcount1 &= 63;
         bitcount2 &= 63;
      }
      #undef DECONE
      #undef DECTHREE

      in1 += 8;

      // transition to careful loop
      s->decodeptr = decodeptr;
      s->bitp[0] = in0 - (bitcount0 >> 3); s->bits[0] = (U32) (bits0 & 0xff); s->bitc[0] = bitcount0 & 7;
      s->bitp[1] = in1 + (bitcount1 >> 3); s->bits[1] = (U32) (bits1 & 0xff); s->bitc[1] = bitcount1 & 7;
      s->bitp[2] = in2 - (bitcount2 >> 3); s->bits[2] = (U32) (bits2 & 0xff); s->bitc[2] = bitcount2 & 7;
   }

   #undef N_DECS_PER_REFILL
   #undef TRIPLE_DECS_PER_REFILL

   return huff_decode_precise_finish(s, huff);
}
#endif

static void decode_huff_array(huff3_decoder *ds, radaudio_huffman *huff, U8 *array, int length, int *error)
{
   rada_internal_huff_state s;

   s.decodeptr = array;
   s.decodeend = array+length;

   // generate Huff3 decoder state from our naive state
   for (int i=0; i <= 2; i += 2) {
      s.bitp[i] = &ds->stream[i].bitstream[ds->stream[i].read_pos_in_bits>>3];
      s.bitc[i] = (0-ds->stream[i].read_pos_in_bits) & 7;  // read pos of 2 => 6 bits left
      if (s.bitc[i] == 0)
         s.bits[i] = 0;
      else {
         s.bits[i] = *(s.bitp[i]) >> (8-s.bitc[i]);
         ++s.bitp[i];
      }
   }

   s.bitp[1] = &ds->stream[1].bitstream[-(int)(ds->stream[1].read_pos_in_bits>>3)];
   s.bitc[1] = (0-ds->stream[1].read_pos_in_bits) & 7;  // read pos of 2 => 6 bits left
   if (s.bitc[1] == 0)
      s.bits[1] = 0;
   else {
      s.bits[1] = *(s.bitp[1]-1) >> (8-s.bitc[1]);
      --s.bitp[1];
   }

   #ifdef __RAD64REGS__
   if (!huff_decode_inner64(&s, huff))
      *error = 1;
   #else
   if (!huff_decode_precise_finish(&s, huff))
      *error = 1;
   #endif

   ds->stream[0].read_pos_in_bits = (int) (8*(s.bitp[0] - ds->stream[0].bitstream) - s.bitc[0]);
   ds->stream[2].read_pos_in_bits = (int) (8*(s.bitp[2] - ds->stream[2].bitstream) - s.bitc[2]);
   ds->stream[1].read_pos_in_bits = (int) (8*(ds->stream[1].bitstream - s.bitp[1]) - s.bitc[1]);
}

////////////////////////////////////////////////////////////////////////////////

static void compute_windowed_sum_multiple64(radaudio_decoder_state *ds, float *output, int n,
                                           const float *fwd_data, S16 *rev_data, int revlen, int revoff, float rev_scale,
                                           const float *window, int block_number, int channel, int stream_offset)
{
   rrAssert(n % 64 == 0);

   // Starting point:
   //    output[0:n] = fwd_c[0:n] .* window[0:n] + rev_scale * rev_c[revoff:revoff+n] .* reverse(window[0:n])
   //
   // let n2 = n/2, then the IMDCT symmetries mean that (when both blocks have same length, I'll account for revoff later)
   //
   //    fwd_c[0:n2] = -reverse(fwd_c[n2:n])
   //    rev_c[n2:n] = reverse(rev_c[0:n2])
   //
   // and therefore we can work with just the middle samples (i.e. the back half of fwd_c and the front
   // half of rev_c). To exploit this systematically, split the loop into two halves at n2:
   //
   //    output[0:n2] = fwd_c[0:n2] .* window[0:n2] + rev_scale * rev_c[revoff+0:revoff+n2] .* reverse(window[n2:n])
   //    output[n2:n] = fwd_c[n2:n] .* window[n2:n] + rev_scale * rev_c[revoff+n2:revoff+n] .* reverse(window[0:n2])
   //
   // note rev_c is symmetric about revoff+n2, so rev_c[revoff+n2:revoff_n] = reverse(rev_c[revoff+0:revoff+n2]).
   // (This is the second symmetry, accounting for potential differences in MDCT size.)
   //
   // Define:
   //    fwd[0:n2] = fwd_c[n2:n]
   //    rev[0:n2] = rev_scale * rev_c[revoff:revoff+n2]
   //
   // and then use the symmetries and algebra to get
   //
   //    output[0:n2] = -reverse(fwd) .* window[0:n2] + rev .* reverse(window[n2:n])
   //                 = rev .* reverse(window[n2:n]) - reverse(fwd) .* window[0:n2]
   //
   //    output[n2:n] = fwd .* window[n2:n] + reverse(rev) .* reverse(window[0:n2])
   //                 = reverse(rev .* window[0:n2]) + fwd .* window[n2:n]
   const float *fwd = fwd_data; // NOTE: second half of the forward data, first half is implied by odd symmetry
   S16 *rev = rev_data + revoff;

   #if defined(DO_BUILD_SSE4) || defined(DO_BUILD_AVX2)
   if (ds->cpu.has_sse2) {
      #ifdef DO_BUILD_AVX2
      if (ds->cpu.has_avx2)
         radaudio_avx2_compute_windowed_sum_multiple16(output, n, fwd, rev, rev_scale, window);
      else
      #endif
         radaudio_sse2_compute_windowed_sum_multiple8(output, n, fwd, rev, rev_scale, window);
      return;
   }
   #endif

   #if defined(DO_BUILD_NEON)

   radaudio_neon_compute_windowed_sum_multiple8(output, n, fwd, rev, rev_scale, window);

   #else

   SINTa N2 = n >> 1;

   for (SINTa j = 0; j < N2; ++j) {
      output[j] = rev_scale * rev[j] * window[n-1-j] - fwd[N2-1-j] * window[j];
   }

   for (SINTa j = 0; j < N2; ++j) {
      output[j+N2] = rev_scale * rev[N2-1-j] * window[N2-1-j] + fwd[j] * window[N2+j];
   }

   #endif
}

static void copy_samples_multiple16(float *output, int n, const float *input)
{
   rrAssert(n % 16 == 0);
   // potentially rely on it being aligned
   memcpy(output, input, 4*n);
}

static void copy_samples_multiple16_scaled(float *output, int n, const S16 *input, float rescale)
{
   rrAssert(n % 16 == 0);
   for (int i=0; i < n; i += 8) {
      output[i+0] = input[i+0] * rescale;
      output[i+1] = input[i+1] * rescale;
      output[i+2] = input[i+2] * rescale;
      output[i+3] = input[i+3] * rescale;
      output[i+4] = input[i+4] * rescale;
      output[i+5] = input[i+5] * rescale;
      output[i+6] = input[i+6] * rescale;
      output[i+7] = input[i+7] * rescale;
   }
}

static void build_rand_state(U32 *rand_state, U32 randval)
{
   U32 r2 = (U32) (((randval + 5000) * (U64) 0xc4ceb9fe1a85ec53ULL) >> 33);
   rand_state[0] = randval;
   rand_state[1] = r2;
   rand_state[2] = randval ^ 0x55555555;
   rand_state[3] = r2 ^ 0x55555555;
}

static void randomize_long_block_8x8_Nx16(radaudio_decoder_state *ds, S8 *quantized_coeff, U32 randval, int num_subbands, int *num_coeffs_for_band)
{
   RAD_ALIGN(U32, rand_state[4], 16);
   build_rand_state(rand_state, randval);

   static S8 random_table[16] = { -1,1, -2,2, -3,3, -4,4, -5,5, -6,6, -7,7, -8,8 };

   // SIMD: compute 4 independent randvals in parallel... the encoder doesn't care what the random
   //       values are, so they should be stable, but don't have to be the same as the current code
   #ifdef DO_BUILD_SSE4
   if (ds->cpu.has_sse4_1) {
      radaudio_sse4_randomize_long_block_8x8_Nx16(quantized_coeff, rand_state, num_subbands);
      return;
   }
   #endif

   int j;

   int cb = 0;
   U32 randval0 = rand_state[0];
   U32 randval1 = rand_state[1];
   for (j=0; num_coeffs_for_band[j] == 4; ++j) {
      if (RR_GET64_NATIVE(&quantized_coeff[cb]) == 0) {
         U32 rbits = randval0 >> 4;
         randval0 = lcg(randval0);
         for (int i=0; i < 4; ++i) {
            quantized_coeff[cb+i] = random_table[rbits & 15];
            rbits >>= 8;
         }
      }
      cb += 4;
   }

   for (; num_coeffs_for_band[j] == 8; ++j) {
      if (RR_GET64_NATIVE(&quantized_coeff[cb]) == 0) {
         U32 rbits = randval0 >> 4;
         randval0 = lcg(randval0);
         for (int i=0; i < 4; ++i) {
            quantized_coeff[cb+i] = random_table[rbits & 15];
            rbits >>= 8;
         }
         rbits = randval1 >> 4;
         randval1 = lcg(randval1);
         for (int i=4; i < 8; ++i) {
            quantized_coeff[cb+i] = random_table[rbits & 15];
            rbits >>= 8;
         }
      }
      cb += 8;
   }
   rand_state[0] = randval0;
   rand_state[1] = randval1;

   for (; j < num_subbands; ++j) {
      if ((RR_GET64_NATIVE(&quantized_coeff[cb+0]) | RR_GET64_NATIVE(&quantized_coeff[cb+8])) == 0) {
         U32 rbits = rand_state[0] >> 4;
         rand_state[0] = lcg(rand_state[0]);
         for (int i=0; i < 4; ++i) {
            quantized_coeff[cb+i] = random_table[rbits & 15];
            rbits >>= 8;
         }
         rbits = rand_state[1] >> 4;
         rand_state[1] = lcg(rand_state[1]);
         for (int i=4; i < 8; ++i) {
            quantized_coeff[cb+i] = random_table[rbits & 15];
            rbits >>= 8;
         }
         rbits = rand_state[2] >> 4;
         rand_state[2] = lcg(rand_state[2]);
         for (int i=8; i < 12; ++i) {
            quantized_coeff[cb+i] = random_table[rbits & 15];
            rbits >>= 8;
         }
         rbits = rand_state[3] >> 4;
         rand_state[3] = lcg(rand_state[3]);
         for (int i=12; i < 16; ++i) {
            quantized_coeff[cb+i] = random_table[rbits & 15];
            rbits >>= 8;
         }
      }
      cb += 16;
   }
}

static void randomize_short_block(S8 quantized_coeff[], U32 randval, int num_bands, int *num_coeffs_for_band)
{
   static S8 random_table[16] = { -1,1, -2,2, -3,3, -4,4, -5,5, -6,6, -7,7, -8,8 };

   int cb = 0;
   U32 rbits = randval;
   randval = lcg(randval);
   rbits >>= 10;

   // Bands 0..3 are 1 coefficient
   for (int j=0; j < 4; ++j) {
      if (quantized_coeff[j] == 0) {
         quantized_coeff[j] = random_table[rbits & 1];
      }
      rbits >>= 4;
   }

   // Bands 4..7 are 2 coefficients each
   for (int j=4; j < 8; ++j) {
      if (RR_GET16_LE_UNALIGNED(&quantized_coeff[j*2-4]) == 0) {
         rbits = randval, randval = lcg(randval);
         rbits >>= 20;
         quantized_coeff[j*2-4] = random_table[rbits & 15];
         quantized_coeff[j*2-3] = random_table[(rbits >> 4) & 15];
      }
   }

   // Bands 8..13 are 4 coefficients each
   for (int j=8; j < 13; ++j) {
      if (RR_GET32_LE_UNALIGNED(&quantized_coeff[j*4-20]) == 0) {
         rbits = randval, randval = lcg(randval);
         quantized_coeff[j*4-20] = random_table[(rbits >> 12) & 15];
         quantized_coeff[j*4-19] = random_table[(rbits >> 16) & 15];
         quantized_coeff[j*4-18] = random_table[(rbits >> 20) & 15];
         quantized_coeff[j*4-17] = random_table[(rbits >> 24) & 15];
      }
   }

   // Remaining bands have 16 or 32 coeffs
   cb = 4*1 + 4*2 + 5*4;
   for (int j=13; j < num_bands; ++j) {
      int i;
      U32 sum1=0, sum2=0;
      int num = num_coeffs_for_band[j];
      for (i=0; i < num; i += 8) { // should be 16 or 32
         sum1 |= RR_GET32_LE_UNALIGNED(&quantized_coeff[cb+i+0]);
         sum2 |= RR_GET32_LE_UNALIGNED(&quantized_coeff[cb+i+4]);
      }
      if ((sum1|sum2) == 0) {
         for (i=0; i+7 < num; i += 8) {
            rbits = randval, randval = lcg(randval);
            for (int k=0; k < 8; ++k) {
               quantized_coeff[cb+i+k] = random_table[rbits & 15];
               rbits >>= 4;
            }
         }
         rbits = randval, randval = lcg(randval);  
         for (; i < num; ++i) {
            quantized_coeff[cb+i] = random_table[rbits & 15];
            rbits >>= 4;
         }
      }
      cb += num_coeffs_for_band[j];
   }
}

static int count_bytes_below_value_sentinel16(radaudio_decoder_state *ds, U8 *data, int num_bytes, U8 threshold)
{
   #ifdef DO_BUILD_SSE4
   if (ds->cpu.has_sse2)
      return radaudio_sse2_count_bytes_below_value_sentinel16(data, num_bytes, threshold);
   #endif

   #if defined(DO_BUILD_NEON)
   return radaudio_neon_count_bytes_below_value_sentinel16(data, num_bytes, threshold);
   #else
   int num=0;
   for (int i=0; i < num_bytes; ++i) {
      num += (data[i] < threshold);
   }
   return num;
   #endif
}

// overwrites up to 7 bytes of space at end of array if not a multiple of 8
static int count_set_bits_multiple8_sentinel8(radaudio_decoder_state *ds, U8 *data, int num_bytes)
{
   #ifdef DO_BUILD_SSE4
   if (ds->cpu.has_sse4_1) // @TODO: ds->cpu.has_popcnt
      return radaudio_intel_popcnt_count_set_bits_read_multiple8_sentinel8(data, num_bytes);
   #endif
   #ifdef DO_BUILD_NEON
   return radaudio_neon_count_set_bits_read_multiple8_sentinel8(data, num_bytes);
   #endif

   #ifndef DO_BUILD_NEON // for unreachable code warnings
   #ifdef __RAD64REGS__
   // 64-bit scalar code
   int num=0;
   int padded_size = (num_bytes + 7) & ~7;
   if (num_bytes != padded_size)
      RR_PUT64_NATIVE(&data[num_bytes], 0);
   for (int i=0; i < padded_size; i += 8) {
      U64 value = RR_GET64_NATIVE(&data[i]);
     value = value - ((value >> 1) & 0x5555555555555555ull); // for pairs of bits: 00->00, 01->01, 10->01, 11->10
     // sums across groups of 2 bits -> sums across groups of 8 bits
     // skipping the groups-of-4 stage to get a wider reduction tree with fewer constants
     U64 threes = 0x0303030303030303ull;
     value = (value & threes) + ((value >> 2) & threes) + ((value >> 4) & threes) + ((value >> 6) & threes);
     // sum the bytes (can't overflow)
     value = (value * 0x0101010101010101ull) >> 56;
     num += (int)value;
   }
   #else
   // 32-bit scalar code
   int num=0;
   int padded_size = (num_bytes + 3) & ~3;
   if (num_bytes != padded_size)
      RR_PUT32_NATIVE(&data[num_bytes], 0);
   for (int i=0; i < padded_size; i += 4) {
      U32 value = RR_GET32_NATIVE(&data[i]);
      value = value - ((value >> 1) & 0x55555555); // for pairs of bits: 00->00, 01->01, 10->01, 11->10
      value = (value & 0x33333333) + ((value>> 2) & 0x33333333);
      value = (value & 0x0f0f0f0f) + ((value>> 4) & 0x0f0f0f0f);
      value = (value * 0x01010101) >> 24;
      num += (int)value;
   }
   #endif
   return num;
   #endif // !DO_BUILD_NEON
}

// guarantees a multiple of 16 bytes is written, with the extra bytes having the value of 1:
//   scalar: reads exact bytes specified, writes an extra 16 bytes of "1"
//   SSE   : writes multiple of 32 bytes with extras equal to "1", also writes 16 bytes starting at &packed[num_packed] 
static void unpack_nibbles_input_excess16_output_excess16_multiple32_default1(radaudio_decoder_state *ds, S8 *unpacked, U8 *packed, int num_packed)
{
   #ifdef DO_BUILD_SSE4
   if (ds->cpu.has_sse2) {
      radaudio_sse2_unpack_nibbles_read_sentinel16_write_multiple32(unpacked, packed, num_packed, 0x1111111111111111ull);
      return;
   }
   #endif
   #ifdef DO_BUILD_NEON
   {
      radaudio_neon_unpack_nibbles_read_sentinel16_write_multiple32(unpacked, packed, num_packed, 0x1111111111111111ull);
      return;
   }
   #endif

   #ifndef DO_BUILD_NEON // for unreachable code warnings
   for (int i=0; i < num_packed; ++i) {
      unpacked[i*2+0] = (S8) (packed[i] & 15);
      unpacked[i*2+1] = (S8) (packed[i] >> 4);
   }

   RR_PUT64_NATIVE(&unpacked[num_packed*2+0], 0x0101010101010101ull);
   RR_PUT64_NATIVE(&unpacked[num_packed*2+8], 0x0101010101010101ull);
   #endif // !DO_BUILD_NEON
}

// if coefficient is 0, then read it from the big coefficient array
// otherwise, remove the +8 bias by subtracting 8
static rrbool expand_nonzero_coefficients(radaudio_decoder_state *ds, S8 *nonzero_coefficients, int num_nonzero, S8 *big_coeff, S8 *big_limit, S8 *safe_read)
{
   if (safe_read - big_limit > 15) {
      #ifdef DO_BUILD_SSE4
      if (ds->cpu.has_sse2) {
         return radaudio_sse2_expand_coefficients_excess_read15(nonzero_coefficients, num_nonzero, big_coeff, big_limit);
      }
      #endif
      #ifdef DO_BUILD_NEON
      return radaudio_neon_expand_coefficients_excess_read15(nonzero_coefficients, num_nonzero, big_coeff, big_limit);
      #endif
   }

   // else fall through to scalar

   for (int i = 0; i < num_nonzero; ++i) {
      if (nonzero_coefficients[i] == 0) {
         if (big_coeff == big_limit)
            return false; // overread error
         nonzero_coefficients[i] = *big_coeff++;
      } else
         nonzero_coefficients[i] -= 8;
   }
   return true;
}

static void compute_band_energy_multiple4(radaudio_decoder_state *ds, F32 *band_energy, int num_bands, int band_exponent[], U16 fine_energy[], F32 band_scale_decode[])
{
   #ifdef DO_BUILD_SSE4
   if (ds->cpu.has_sse2) {
      radaudio_sse2_compute_band_energy_multiple4(band_energy, num_bands, band_exponent, fine_energy, band_scale_decode);
      return;
   }
   #endif

   #ifdef DO_BUILD_NEON
   radaudio_neon_compute_band_energy_multiple4(band_energy, num_bands, band_exponent, fine_energy, band_scale_decode);
   #else
   for (int j=0; j < num_bands; ++j) {  // safe to run 24 times for SIMD
      int qe = fine_energy[j]; // quantized energy, in [0, 1<<MAX_FINE_ENERGY_BITS)
      F32 fe, ce, pe; // fine energy, coarse energy, packed energy

      pe = qe / (float) (1 << MAX_FINE_ENERGY_BITS);    // pe is 0..1
      fe = (0.34375f*pe + 0.65625f)*pe + 1.0f;

      if (band_exponent[j] == BAND_EXPONENT_NONE)
         ce = 0;
      else
         ce = (float) (1 << (band_exponent[j] + 16)); // integer_exponent 0 => (1<<30>>14) => 1<<16

      band_energy[j] = (fe * ce) * band_scale_decode[j];
   }
   #endif
}

static void compute_subband_energy_skip12_excess_read7(radaudio_decoder_state *ds, F32 *subband_energy, const F32 *band_energy, int num_bands, int num_subbands, int *num_subbands_for_band, U16 *quantized_subbands)
{
   #ifdef DO_BUILD_SSE4
   if (ds->cpu.has_sse4_1) {
      radaudio_sse4_compute_subband_energy_skip12_excess_read7(subband_energy, band_energy, num_bands, num_subbands_for_band, quantized_subbands);
      return;
   }
   #endif

   #ifdef DO_BUILD_NEON
   radaudio_neon_compute_subband_energy_skip12_excess_read7(subband_energy, band_energy, num_bands, num_subbands_for_band, quantized_subbands);
   #else
   int start, j;
   for (j=0; num_subbands_for_band[j] == 1; ++j)
      ;
   start = j;
   for (; j < num_bands; ++j) {
      int sum=0;
      int num = num_subbands_for_band[j];
      // these loops are pretty random lengths, for example, at 44.1Khz, they're: 2,2,2,2,3,4,9,10,12 iterations
      for (int i=0; i < num; ++i) {
         sum += (quantized_subbands[start+i] * quantized_subbands[start+i]);
      }

      F32 scale = band_energy[j] / sqrtf((F32) sum);
      rrAssert(!isnan(band_energy[j]));
      rrAssert(sum != 0);
      for (int i=0; i < num; ++i) {
         subband_energy[start+i] = scale * quantized_subbands[start+i];
      }
      start += num;
   }
   rrAssert(start == num_subbands);
   #endif
}

static void distribute_bitflag_coefficients_multiple64(radaudio_decoder_state *ds, 
                                           S8 *quantized_coeff, int num_coeff,
                                           U8 *nonzero_flagbits,
                                           S8 *nonzero_coeffs, int *pcur_nonzero_coeffs)
{
   #ifdef DO_BUILD_SSE4
   if (ds->cpu.has_ssse3) {
      radaudio_ssse3_distribute_bitflag_coefficients_multiple16(
            ds->cpu,
            quantized_coeff, num_coeff,
            nonzero_flagbits,
            nonzero_coeffs, pcur_nonzero_coeffs);
      return;
   }
   #endif
   #ifdef DO_BUILD_NEON
   {
      radaudio_neon_distribute_bitflag_coefficients_multiple16(
            quantized_coeff, num_coeff,
            nonzero_flagbits,
            nonzero_coeffs, pcur_nonzero_coeffs);
      return;
   }
   #endif

   #ifndef DO_BUILD_NEON // for unreachable code warnings
   int cur_nonzero_coeffs = *pcur_nonzero_coeffs;

   memset(quantized_coeff, 0, num_coeff);

   // use a run-length style scheme using bit scans to reduce branch mispredictions
   int pos=0;
   for (int i=0; i < num_coeff; i += 64) {
      U64 flags = RR_GET64_LE(nonzero_flagbits + pos);
      pos += 8;

      // even though the run is never long--we could just use a small lookup table--let's do it right
      int offset = i;
      while (flags) {
         SINTa dist = rrCtz64(flags);
         quantized_coeff[offset+dist] = nonzero_coeffs[cur_nonzero_coeffs++];
         flags = rrClearLowestSetBit64(flags);
      }
   }

   *pcur_nonzero_coeffs = cur_nonzero_coeffs;
   #endif // !DO_BUILD_NEON
}

static rrbool distribute_nonzero_coefficients(radaudio_decoder_state *ds,
                                           S8 *quantized_coeff, int num_coeff32,
                                           U8 *runlength_data, int *pcur_runlength_data, // there's guaranteed sentinels, so don't need length
                                           S8 *nonzero_coeffs, int *pcur_nonzero_coeffs,
                                           U8 *nonzero_flagbits, int num_nonzero_flagbits, int channel)
{
   RR_UNUSED_VARIABLE(channel);
   SINTa num_coeff = num_coeff32;
   SINTa k=0;
   if (num_nonzero_flagbits) {
      PROF_BEGIN(distribute_bitflag);
      distribute_bitflag_coefficients_multiple64(ds, quantized_coeff, num_nonzero_flagbits, nonzero_flagbits, nonzero_coeffs, pcur_nonzero_coeffs);
      PROF_END(distribute_bitflag);
      k = num_nonzero_flagbits;
   }

   const U8 *runlens = runlength_data + *pcur_runlength_data;
   const S8 *nzcoeffs = nonzero_coeffs + *pcur_nonzero_coeffs;

   PROF_BEGIN(distribute_rle);
   memset(quantized_coeff+k, 0, num_coeff-k);
   // tried a branchless version of this using the slot[] logic from above, but saw no gain
   // we put in sentinels that guarantee this loop will see a END_OF_ZERORUN
   for(;;) {
      U8 rl = *runlens++;
      if (rl == END_OF_ZERORUN)
         break;
      k += rl;
      if (rl < MAX_RUNLEN) {
         if (k >= num_coeff)
            return false;
         quantized_coeff[k] = *nzcoeffs++;
         ++k;
      }
   }

   *pcur_runlength_data = (int)(SINTa)(runlens - runlength_data);
   *pcur_nonzero_coeffs = (int)(SINTa)(nzcoeffs - nonzero_coeffs);
   PROF_END(distribute_rle);

   return true;
}

static void dequantize_long_block_8x8_Nx16(radaudio_decoder_state *ds, float *coeffs, S8 *quantized_coeff, float *subband_energy, int num_subbands, int *num_coeffs_for_band)
{
   #ifdef DO_BUILD_SSE4
   if (ds->cpu.has_sse4_1) {
      radaudio_sse4_dequantize_long_block_8x8_Nx16(coeffs, quantized_coeff, subband_energy, num_subbands);
      return;
   }
   #endif

   int cb=0;
   int j=0;

   // first 8 subbands should be 8 coefficients long
   while (num_coeffs_for_band[j] < 16) {
      F32 sum=1.e-20f,scale;
      for (int i=0; i < num_coeffs_for_band[j]; ++i) {
         F32 n = (F32) quantized_coeff[cb+i];
         sum += n*n;
      }
      scale = subband_energy[j] / sqrtf(sum);
      for (int i=0; i < 8; ++i)
         coeffs[cb+i] = quantized_coeff[cb+i] * scale;
      cb += num_coeffs_for_band[j];
      ++j;
   }

   // all remaining subbands are 16 coefficients long, so we don't have to check bands
   for (; j < num_subbands; ++j) {
      F32 sum=1.e-20f,scale;
      for (int i=0; i < 16; ++i) {
         F32 n = (F32) quantized_coeff[cb+i];
         sum += n*n;
      }
      scale = subband_energy[j] / sqrtf(sum);
      for (int i=0; i < 16; ++i)
         coeffs[cb+i] = quantized_coeff[cb+i] * scale;
      cb += 16;
   }
   for (int i=cb; i < RADAUDIO_LONG_BLOCK_LEN; ++i)
      coeffs[i] = 0;
}

static void dequantize_long_block_with_random_8x8_Nx16(radaudio_decoder_state *ds, F32 *coeffs, S8 *quantized_coeff, F32 *subband_energy, int num_subbands, int *num_coeffs_for_band, U32 randval)
{
   #ifdef DO_BUILD_SSE4
   if (ds->cpu.has_sse4_1) {
      RAD_ALIGN(U32, rand_state[4], 16);
      build_rand_state(rand_state, randval);
      radaudio_sse4_dequantize_long_block_replace_0_with_random_8x8_Nx16(coeffs, quantized_coeff, subband_energy, num_subbands, rand_state);
      return;
   }
   #endif
   #ifdef DO_BUILD_NEON
   {
      RAD_ALIGN(U32, rand_state[4], 16);
      build_rand_state(rand_state, randval);
      radaudio_neon_dequantize_long_block_replace_0_with_random_8x8_Nx16(coeffs, quantized_coeff, subband_energy, num_subbands, rand_state);
      return;
   }
   #endif

   #ifndef DO_BUILD_NEON // for unreachable code warnings   
   randomize_long_block_8x8_Nx16(ds, quantized_coeff, randval, num_subbands, num_coeffs_for_band);
   dequantize_long_block_8x8_Nx16(ds, coeffs, quantized_coeff, subband_energy, num_subbands, num_coeffs_for_band);
   #endif // !DO_BUILD_NEON
}

static void scalar_dequantize_short_block(float *coeffs, S8 *quantized_coeff, float *band_energy, int num_bands, int *num_coeffs_for_band)
{
   int i,j;
   int cb=0;
   // first 4 bands are 1 coefficient long, and coefficient is always 1 or -1
   for (j=0; j < 4; ++j) {
      rrAssert(abs(quantized_coeff[j]) == 1);
      coeffs[j] = (F32) quantized_coeff[j] * band_energy[j];
   }

   // next 4 bands are 2 coefficients long
   for (j=0; j < 4; ++j) {
      float x = (F32) quantized_coeff[4+j*2+0];
      float y = (F32) quantized_coeff[4+j*2+1];
      float scale = band_energy[4+j] / sqrtf(x*x+y*y+1.e-20f);
      coeffs[4+j*2+0] = x*scale;
      coeffs[4+j*2+1] = y*scale;
   }

   // next 4 bands are 4 coefficients long
   // (actually next 5 bands are)
   cb = 4*1 + 4*2;
   for (j=8; j < 12; ++j) {
      float sum=1.e-20f, scale;
      for (i=0; i < 4; ++i) {
         float n = (F32) quantized_coeff[cb+i];
         sum += n*n;
      }
      scale = band_energy[j] / sqrtf(sum);
      for (i=0; i < 4; ++i)
         coeffs[cb+i] = (F32) quantized_coeff[cb+i] * scale;
      cb += 4;
   }

   // now we have either [4,16,16,16,32]
   //                 or [4,16,16,32,32] for lower sample rates
   cb = 4*1 + 4*2 + 4*4;
   for (j=12; j < num_bands; ++j) {
      int count = num_coeffs_for_band[j];
      F32 sum=1.e-20f,scale;
      for (i=0; i < count; i += 4) {
         for (int k=0; k < 4; ++k) {
            F32 n = (F32) quantized_coeff[cb+i+k];
            sum += n*n;
         }
      }
      scale = band_energy[j] / sqrtf(sum);
      for (i=0; i < count; i += 4) {
         for (int k=0; k < 4; ++k) {
            coeffs[cb+i+k] = quantized_coeff[cb+i+k] * scale;
         }
      }
      cb += count;
   }

   for (i=cb; i < RADAUDIO_SHORT_BLOCK_LEN; ++i)
      coeffs[i] = 0;

}

static void dequantize_short_block(radaudio_decoder_state *ds, float *coeffs, S8 *quantized_coeff, float *band_energy, int num_bands, int *num_coeffs_for_band)
{
   #ifdef DO_BUILD_SSE4
   if (ds->cpu.has_sse4_1) {
      radaudio_sse4_dequantize_short_block_sse4(coeffs, quantized_coeff, band_energy, num_bands, num_coeffs_for_band);
      return;
   }
   #endif
   #ifdef DO_BUILD_NEON
   {
      radaudio_neon_dequantize_short_block(coeffs, quantized_coeff, band_energy, num_bands, num_coeffs_for_band);
      return;
   }
   #endif

   #ifndef DO_BUILD_NEON // for unreachable code warnings
   scalar_dequantize_short_block(coeffs, quantized_coeff, band_energy, num_bands, num_coeffs_for_band);
   #endif // !DO_BUILD_NEON
}

// we have to store half the MDCT output to overlap in the next block. this is the largest
// per-stream memory cost of the decoder. we used to store floats, but now we convert them
// to S16. This is *pre* windowing, so the quality loss is minimal.
//
// sse2 runs at about half speed of original "store as floats" version, but it's about a 2% slowdown
// overall and we decided it was worth the speed loss in return for halving memory usage
static float save_overlapping_samples(radaudio_decoder_state *ds, S16 *buffer, const float *data, int num)
{
   // the profile wrapper is external to this, under the name "copy"

   rrAssert(num % 64 == 0);

   num /= 2;

   #if defined(DO_BUILD_SSE4) || defined(DO_BUILD_AVX2)
   if (ds->cpu.has_sse2) {
      #ifdef DO_BUILD_AVX2
      if (ds->cpu.has_avx2) {
         return radaudio_avx2_save_samples(buffer, data, num);
      }
      #endif
      return radaudio_sse2_save_samples(buffer, data, num);
   }
   #endif

   #ifdef DO_BUILD_NEON
   return radaudio_neon_save_samples(buffer, data, num);
   #endif

   #ifndef DO_BUILD_NEON // for unreachable code warnings

   #define FAST_FLOAT_TO_INT // best solution i've found for x64

   #if 0
   // naive implementation for reference, but floor() is unacceptably slow
   // doubles total decode time on test platform; round() and rint() were worse
   // also, doesn't round to nearest even like SSE path
   // most files in fnaudio get different results, is it buggy? there's no way this can just be from tie-breaking 0.5?!?
   float largest0 = 1.0f;
   float largest1 = 1.0f;
   float scale = 32767.0f;
   for (int i=0; i < num; i += 2) {
      F32 d0 = data[i+0];
      F32 d1 = data[i+1];
      buffer[i+0] = (S16) floorf(d0 * scale + 0.5f);
      buffer[i+1] = (S16) floorf(d1 * scale + 0.5f);
      F32 a0 = fabsf(d0);
      F32 a1 = fabsf(d1);
      largest0 = RR_MAX(largest0, a0);
      largest1 = RR_MAX(largest1, a1);
   }
   float largest   = RR_MAX(largest0,largest1);
   if (largest > 1.0f) {
      scale = 32767.0f / largest;
      for (int i=0; i < num; i += 2) {
         buffer[i+0] = (S16) floorf(data[i+0] * scale + 0.5f);
         buffer[i+1] = (S16) floorf(data[i+1] * scale + 0.5f);
      }
   }
   return 1.0f / scale;

   #elif defined(FAST_FLOAT_TO_INT)
   // this should round correctly
   // bithack float-to-int

   typedef union {
      F32 f;
      S32 i;
   } float_conv;

   float_conv temp0,temp1,temp2,temp3;
   // add (1<<23) to convert to int, then divide by 2^SHIFT, then add 0.5/2^SHIFT to round
   #define MAGIC(SHIFT) (1.5f * (1 << (23-SHIFT)) + 0.5f/(1 << SHIFT))
   #define ADDEND(SHIFT) (((150-SHIFT) << 23) + (1 << 22))
   #define FAST_SCALED_FLOAT_TO_INT(temp,x,s) (temp.f = (x) + MAGIC(s), temp.i - ADDEND(s))

   float largest0 = 1.0f;
   float largest1 = 1.0f;
   float largest2 = 1.0f;
   float largest3 = 1.0f;
   float scale = 32767.0f;
   for (int i=0; i < num; i += 4) {
      F32 d0 = data[i+0];
      F32 d1 = data[i+1];
      F32 d2 = data[i+2];
      F32 d3 = data[i+3];
      F32 a0 = fabsf(d0);
      F32 a1 = fabsf(d1);
      F32 a2 = fabsf(d2);
      F32 a3 = fabsf(d3);
      buffer[i+0] = (S16) FAST_SCALED_FLOAT_TO_INT(temp0, d0 * scale, 0);
      buffer[i+1] = (S16) FAST_SCALED_FLOAT_TO_INT(temp1, d1 * scale, 0);
      buffer[i+2] = (S16) FAST_SCALED_FLOAT_TO_INT(temp2, d2 * scale, 0);
      buffer[i+3] = (S16) FAST_SCALED_FLOAT_TO_INT(temp3, d3 * scale, 0);
      largest0 = RR_MAX(largest0, a0);
      largest1 = RR_MAX(largest1, a1);
      largest2 = RR_MAX(largest2, a2);
      largest3 = RR_MAX(largest3, a3);
   }
   float largest01 = RR_MAX(largest0,largest1);
   float largest23 = RR_MAX(largest2,largest3);
   float largest   = RR_MAX(largest01,largest23);
   if (largest > 1.0f) {
      scale = 32767.0f / largest;
      for (int i=0; i < num; i += 4) {
         buffer[i+0] = (S16) FAST_SCALED_FLOAT_TO_INT(temp0, data[i+0] * scale, 0);
         buffer[i+1] = (S16) FAST_SCALED_FLOAT_TO_INT(temp0, data[i+1] * scale, 0);
         buffer[i+2] = (S16) FAST_SCALED_FLOAT_TO_INT(temp0, data[i+2] * scale, 0);
         buffer[i+3] = (S16) FAST_SCALED_FLOAT_TO_INT(temp0, data[i+3] * scale, 0);
      }
   }
   return 1.0f / scale;

   #else

   // 30% slower than FAST_FLOAT_TO_INT on x64

   // we want to use the equivalent of floor() so we can round.
   // if we use fixed-point, right-shifting two's complement values is floor.
   // though we might get compiler warnings about signed shifts
   // problem: this doesn't produce the exact same results as other methods
   // most files in fnaudio get different results

   #define TRUNC_SHIFT   15

   float largest0 = 1.0f;
   float largest1 = 1.0f;
   float largest2 = 1.0f;
   float largest3 = 1.0f;
   float scale = 32767.0f;
   for (int i=0; i < num; i += 4) {
      F32 d0 = data[i+0];
      F32 d1 = data[i+1];
      F32 d2 = data[i+2];
      F32 d3 = data[i+3];
      F32 a0 = fabsf(d0);
      F32 a1 = fabsf(d1);
      F32 a2 = fabsf(d2);
      F32 a3 = fabsf(d3);
      S32 i0 = (S32) (d0 * scale * (1 << TRUNC_SHIFT));
      S32 i1 = (S32) (d1 * scale * (1 << TRUNC_SHIFT));
      S32 i2 = (S32) (d2 * scale * (1 << TRUNC_SHIFT));
      S32 i3 = (S32) (d3 * scale * (1 << TRUNC_SHIFT));
      buffer[i+0] = (S16) ((i0 + (1 << (TRUNC_SHIFT-1))) >> TRUNC_SHIFT);
      buffer[i+1] = (S16) ((i1 + (1 << (TRUNC_SHIFT-1))) >> TRUNC_SHIFT);
      buffer[i+2] = (S16) ((i2 + (1 << (TRUNC_SHIFT-1))) >> TRUNC_SHIFT);
      buffer[i+3] = (S16) ((i3 + (1 << (TRUNC_SHIFT-1))) >> TRUNC_SHIFT);
      largest0 = RR_MAX(largest0, a0);
      largest1 = RR_MAX(largest1, a1);
      largest2 = RR_MAX(largest2, a2);
      largest3 = RR_MAX(largest3, a3);
   }
   float largest01 = RR_MAX(largest0,largest1);
   float largest23 = RR_MAX(largest2,largest3);
   float largest   = RR_MAX(largest01,largest23);
   if (largest > 1.0f) {
      scale = 32767.0f / largest;
      for (int i=0; i < num; i += 4) {
         S32 i0 = (S32) (data[i+0] * scale * (1 << TRUNC_SHIFT));
         S32 i1 = (S32) (data[i+1] * scale * (1 << TRUNC_SHIFT));
         S32 i2 = (S32) (data[i+2] * scale * (1 << TRUNC_SHIFT));
         S32 i3 = (S32) (data[i+3] * scale * (1 << TRUNC_SHIFT));
         buffer[i+0] = (S16) ((i0 + (1 << (TRUNC_SHIFT-1))) >> TRUNC_SHIFT);
         buffer[i+1] = (S16) ((i1 + (1 << (TRUNC_SHIFT-1))) >> TRUNC_SHIFT);
         buffer[i+2] = (S16) ((i2 + (1 << (TRUNC_SHIFT-1))) >> TRUNC_SHIFT);
         buffer[i+3] = (S16) ((i3 + (1 << (TRUNC_SHIFT-1))) >> TRUNC_SHIFT);
      }
   }
   return 1.0f / scale;

   #endif

   #endif // !DO_BUILD_NEON
}

/////////////////////////////////////////////////////////////////////////////////

static const size_t decoder_align = 32;

// We fill all-0 subbands with random noise (that's scaled to the
// appropriate subband energy).
//
// We need a mechanism where SSE2 can generate random data very fast,
// but it doesn't hobble the scalar pass. so e.g. SSE2 can generate
// 4 LCG steps in parallel, 16..24 bits of useful data in each one.
// but doing this on scalar might be slow. for a single 16-item subband,
// we need 4*16 = 64 bits of randomness to use our 4-bit random_table[]
// (which we can lookup with pshub).
//
// scalar: old code did 2 32-bit LCGs per subband, but only if the subband was zero.
// Optimized-for-SIMD code might prefer to be branchless and do an LCG on
// every subband even if non-zero, and there are ~80 subbands. But it turns
// out we take a branch anyway in the SSE code, so executive decision to stick
// to a design where we only update LCGs if subband was zero. (Note that if
// we didn't care about identical decoding across platforms, you could use
// whatever random method was optimal for each platform).

static void decode_channel_before_imdct(radaudio_decoder_state *ds,
                          radaudio_block_data   *bd,
                          int channel,
                          U32 rand_seed,
                          U16 fine_energy[])
{
   radaudio_rate_info *info;

   RAD_ALIGN(F32, band_energy[24+16], 16) = { 0 }; // must be a multiple of 4
   RAD_ALIGN(F32, subband_energy[MAX_SUBBANDS+16], 16) = { 0 };

   int is_short_block = ds->current_block_short;
   info = ds->info[is_short_block];

   PROF_BEGIN(compute_band_energy);
   // compute band energy
   compute_band_energy_multiple4(ds, band_energy, info->num_bands, bd->band_exponent, fine_energy, info->band_scale_decode);
   PROF_END(compute_band_energy);

   // compute subband energy
   if (!is_short_block) {
      PROF_BEGIN(compute_subband_energy);
      // first bands are shorter than a full subband, so treat those specially
      int j;
      for (j=0; info->num_subbands_for_band[j] == 1; ++j)
         subband_energy[j] = band_energy[j];

      compute_subband_energy_skip12_excess_read7(ds, subband_energy, band_energy, info->num_bands, info->num_subbands, info->num_subbands_for_band, bd->quantized_subbands);

      for (j=0; j < info->num_subbands; ++j)
         rrAssert(!isnan(subband_energy[j]));

      PROF_END(compute_subband_energy);
   }

   // spread out adjacent blocks to be less similar
   U32 randval = (rand_seed + (rand_seed >> 5)) * 0x27d4eb2d;
   randval = lcg(randval);

   F32 *coeffs = bd->dequantized_coeff_decode;

   if (!is_short_block) {
      for (int j=0; j < info->num_subbands; ++j)
         rrAssert(info->num_coeffs_for_subband[j] == (j < 8 ? 8 : 16));

      PROF_BEGIN(unquantize);
      dequantize_long_block_with_random_8x8_Nx16(ds, coeffs, bd->quantized_coeff_decode, subband_energy, info->num_subbands, info->num_coeffs_for_band, randval);
      PROF_END(unquantize);
   } else {
      PROF_BEGIN(randomize);
      randomize_short_block(bd->quantized_coeff_decode, randval, info->num_bands, info->num_coeffs_for_band);
      PROF_END(randomize);

      PROF_BEGIN(unquantize);
      dequantize_short_block(ds, coeffs, bd->quantized_coeff_decode, band_energy, info->num_bands, info->num_coeffs_for_band);
      PROF_END(unquantize);
   }
}

#if 0
static void decode_channel_before_imdct_reference(radaudio_decoder_state *ds,
                          radaudio_block_data   *bd,
                          int channel,
                          U32 rand_seed,
                          U16 fine_energy[])
{
   radaudio_rate_info *info;

   RAD_ALIGN(F32, band_energy[24+16], 16) = { 0 }; // must be a multiple of 4
   RAD_ALIGN(F32, subband_energy[MAX_SUBBANDS+16], 16) = { 0 };

   int is_short_block = ds->current_block_short;
   info = ds->info[is_short_block];

   PROF_BEGIN(compute_band_energy);
   // compute band energy
   compute_band_energy_multiple4(ds, band_energy, info->num_bands, bd->band_exponent, fine_energy, info->band_scale_decode);
   PROF_END(compute_band_energy);

   // compute subband energy
   if (!is_short_block) {
      PROF_BEGIN(compute_subband_energy);
      // first bands are shorter than a full subband, so treat those specially
      int j;
      for (j=0; info->num_subbands_for_band[j] == 1; ++j)
         subband_energy[j] = band_energy[j];

      compute_subband_energy_skip12_excess_read7(ds, subband_energy, band_energy, info->num_bands, info->num_subbands, info->num_subbands_for_band, bd->quantized_subbands);

      for (j=0; j < info->num_subbands; ++j)
         rrAssert(!isnan(subband_energy[j]));

      PROF_END(compute_subband_energy);
   }

   // spread out adjacent blocks to be less similar
   U32 randval = (rand_seed + (rand_seed >> 5)) * 0x27d4eb2d;
   randval = lcg(randval);

   PROF_BEGIN(randomize);
   if (!is_short_block) {
      // replace all-zero coefficient chunks with noise
      for (int j=0; j < info->num_subbands; ++j)
         rrAssert(info->num_coeffs_for_subband[j] == (j < 8 ? 8 : 16));
      randomize_long_block_8x8_Nx16(bd->quantized_coeff_decode, randval, info->num_subbands, info->num_coeffs_for_band);
   } else {
      // replace all-zero coefficient chunks with noise
      randomize_short_block(bd->quantized_coeff_decode, randval, info->num_bands, info->num_coeffs_for_band);
   }
   PROF_END(randomize);

   F32 *coeffs = bd->dequantized_coeff_decode;

   PROF_BEGIN(unquantize);
   // reference implementation doesn't make any assumptions about distribution of subbands
   {
      int start = 0;

      if (is_short_block) {
         for (int j=0; j < info->num_bands; ++j)
            subband_energy[j] = band_energy[j];
      } else {
         for (int j=0; j < info->num_bands && info->num_subbands_for_band[j] == 1; ++j)
            subband_energy[j] = band_energy[j];
      }

      for (int j=0; j < info->num_subbands; ++j) {
         int n = info->num_coeffs_for_subband[j];
         F32 sum = 1.0e-30f;
         for (int i=0; i < n; ++i) {
            F32 x = (F32) bd->quantized_coeff_decode[start+i];
            sum += x*x;
         }
         F32 scale = subband_energy[j] / sqrtf(sum);
         for (int i=0; i < n; ++i) {
            coeffs[start+i] = (F32) bd->quantized_coeff_decode[start+i] * scale;
         }

         start += n;
      }
      for (int i=info->num_quantized_coeffs; i < info->num_coeffs; ++i)
         coeffs[i] = 0;
   }
   PROF_END(unquantize);
}
#endif

// dequantized_coeff_decode[] in bd is overwritten in the process
static void decode_channel_imdct(radaudio_decoder_state *ds,
                          F32 rawdata[MAX_COEFFS], // only max_coeffs because we expand the symmetries later
                          radaudio_block_data *bd,
                          int channel)
{
   radaudio_rate_info *info;

   int is_short_block = ds->current_block_short;
   info = ds->info[is_short_block];

   F32 *coeffs = bd->dequantized_coeff_decode;
   for (int j=0; j < info->num_coeffs; ++j)
      rrAssert(!isnan(coeffs[j]));

   PROF_BEGIN(imdct);
   radaudio_imdct_fft_only_middle(ds->cpu, rawdata, coeffs, info->num_coeffs);
   PROF_END(imdct);
}

static int decode_channel_after_imdct(radaudio_decoder_state *ds,
                          F32 data1 [MAX_COEFFS],
                          F32 data2 [MAX_COEFFS],
                          radaudio_block_data *bd,
                          int channel,
                          F32 *output)
{
   int result_length;
   radaudio_rate_info *info;

   int is_short_block = ds->current_block_short;
   info = ds->info[is_short_block];

   PROF_BEGIN(window);
   if (ds->post_seek)
      // ignore the left side of the first block
      result_length = 0;
   else {
      // use window for whichever is smaller of current block or previous block
      // apply window to pending data
      if (ds->current_block_short == ds->last_block_short) {
         F32 *window = radaudio_windows[ds->current_block_short];
         int len = ds->current_block_short ? RADAUDIO_SHORT_BLOCK_LEN : RADAUDIO_LONG_BLOCK_LEN;

         compute_windowed_sum_multiple64(ds, output, len,
                                         data1, ds->prev_block_right_samples[channel], len, 0, ds->restore_scale[channel],
                                         window, ds->block_number, channel, ds->fully_decoded);
         result_length = len;
      } else {
         F32 *window = radaudio_windows[RADAUDIO_SHORT];
         const int n = RADAUDIO_SHORT_BLOCK_LEN;
         if (is_short_block) {
            // if previous block was long and this is short
            //
            //                         <-----LONG_BLOCK_LEN---->
            // +-----------+-----------+-----------+-----------+
            // |                     prev                      |
            // +-----------+-----------+-----------+-----------+
            //                         ***************            <- output samples
            //  -----------------------1111111111WWwww000000000   <- window weights
            //                         ----------wwWWW
            //                                  +----+----+
            //                                  |   cur   |
            //                                  +----+----+
            //                                  <---->
            //                                    |
            //                              SHORT_BLOCK_LEN
            // 
            const int len = RADAUDIO_LONG_BLOCK_LEN/2 - RADAUDIO_SHORT_BLOCK_LEN/2;
            copy_samples_multiple16_scaled(output, len, ds->prev_block_right_samples[channel], ds->restore_scale[channel]); // copy samples from previous where the new window is 0 and old window was 1
            compute_windowed_sum_multiple64(ds, output+len, n,
                                            data1, ds->prev_block_right_samples[channel], RADAUDIO_LONG_BLOCK_LEN, len, ds->restore_scale[channel],
                                            window, ds->block_number, channel, ds->fully_decoded+len); // sum the part of the previous block that overlaps the left half of the new block

            result_length = RADAUDIO_LONG_BLOCK_LEN/2 + RADAUDIO_SHORT_BLOCK_LEN/2;  // generated (LONG/2 - SHORT/2) + SHORT
         } else {
            // if previous block was short and this is long
            //
            //         SHORT_BLOCK_LEN
            //              |
            //           <---->
            //      +----+----+
            //      |   prev  |
            //      +----+----+
            //           WWwww---------
            //  000000000wwwWW111111111-----------------------    <- window weights
            //           **************                           <- output samples
            // +-----------+-----------+-----------+-----------+
            // |                      cur                      |
            // +-----------+-----------+-----------+-----------+
            // <-----LONG_BLOCK_LEN---->
            // 

            const int offset = RADAUDIO_LONG_BLOCK_LEN/2 - RADAUDIO_SHORT_BLOCK_LEN/2;
            compute_windowed_sum_multiple64(ds, output, n, data1, ds->prev_block_right_samples[channel], RADAUDIO_SHORT_BLOCK_LEN, 0, ds->restore_scale[channel], window, ds->block_number, channel, ds->fully_decoded);
            copy_samples_multiple16(output+n, RADAUDIO_LONG_BLOCK_LEN - (offset+n), data1 + n/2);

            result_length = RADAUDIO_SHORT_BLOCK_LEN/2 + RADAUDIO_LONG_BLOCK_LEN/2;
         }
      }
   }
   PROF_END(window);

   PROF_BEGIN(copy);
   ds->restore_scale[channel] = save_overlapping_samples(ds, ds->prev_block_right_samples[channel], data2, info->num_coeffs);
   PROF_END(copy);

   return result_length;
}

static U8 *find_next_coarse_run_excess16(radaudio_decoder_state *ds, U8 *cur, U8 *end)
{
   #ifdef DO_BUILD_SSE4
   if (ds->cpu.has_sse2) {
      return radaudio_sse2_find_next_coarse_run_excess16(cur, end);
   }
#endif

#ifdef __RAD64REGS__
   RR_COMPILER_ASSERT(COARSE_RUNLEN_THRESHOLD < 128);
   RR_COMPILER_ASSERT(MAX_RUNLEN >= 128);

   const U64 splat8 = ~(U64)0 / 255; // 0x0101...01
   const U64 msb_mask = 0x80 * splat8;
   const U64 low7_mask = ~msb_mask;
   const U64 bias0 = (128 - COARSE_RUNLEN_THRESHOLD) * splat8;
   const U64 bias1 = (256 - MAX_RUNLEN) * splat8;

   while (cur < end) {
      U64 bytes = RR_GET64_LE(cur);
      cur += 8;

      // check if there are any bytes >=COARSE_RUNLEN_THRESHOLD in those 8 bytes we just read.
      // idea: these are bytes that either
      // 1. have low 7 bits >=COARSE_RUNLEN_THRESHOLD
      // 2. have MSB set (thus >=128)
      // we can check the former by masking with 0x7f7f...7f and then adding (128 - COARSE_RUNLEN_THRESHOLD)
      // to every byte. if the MSB ends up set, they were above COARSE_RUNLEN_THRESHOLD.
      //
      // by the same logic, we have bytes >=MAX_RUNLEN (which is >=128) if and only if both
      // 1. their low 7 bits >= (MAX_RUNLEN - 128)
      // 2. their MSB is set
      U64 low7 = bytes & low7_mask;
      U64 above_coarse_runlen_thresh = (low7 + bias0) | bytes; // MSB in byte set if that byte >=COARSE_RUNLEN_THRESHOLD
      U64 above_max_runlen_thresh = (low7 + bias1) & bytes; // MSB in byte set if that byte >=MAX_RUNLEN
      U64 active = above_coarse_runlen_thresh & ~above_max_runlen_thresh & msb_mask;
      if (active) {
         // found at least one! locate the first occurrence using a trailing
         // zero count.
         return (cur - 8) + rrCtzBytes64(active);
      }
   }

   return cur;
#else
   while (cur < end && (*cur < COARSE_RUNLEN_THRESHOLD || *cur >= MAX_RUNLEN))
      ++cur;

   return cur;
#endif
}

static int decode_block(radaudio_decoder_state *ds, F32 *output[2], void *mem, size_t memavail, size_t *memconsumed)
{
   int len, skip=0;
   radaudio_block_data bd[2];

   *memconsumed = 0;
   U8 *memory = mem;
   //size_t memory_valid = memavail;

   int c;

   huff3_decoder dec;

   // Throw in one empty section first so we know what the overhead of one of these is
   PROF_BEGIN(overhead);
   PROF_END(overhead);

   PROF_BEGIN(header);
   radaudio_block_header_unpacked header;

   int offset = radaudio_decode_block_header(memory, &ds->biases, &header, memavail);

   // check if we're at the stream header, if so skip it; this happens at start,
   // but also if they seek without telling us
   if (offset == COMMON_STREAM_HEADER) {
      // we might be at the start of the stream

      // enough bytes to check for the stream signature?
      if (memavail < 8)
         return e(RADAUDIO_INCOMPLETE_DATA);

      // check the stream signature
      if (!radaudio_check_stream_header(memory, memavail))
         return e(RADAUDIO_INVALID_DATA);

      // enough bytes for a full header?
      if (memavail < sizeof(radaudio_stream_header))
         return e(RADAUDIO_INCOMPLETE_DATA);

      // decode the header, the only way we have to parse it
      radaudio_stream_header_unpacked fh;
      size_t header_size = radaudio_unpack_stream_header(memory, memavail, &fh);

      // was it a valid header?
      if (header_size == 0)
         return e(RADAUDIO_INVALID_DATA);

      // make sure the subtract below can't be negative
      if (header_size > memavail)
         return e(RADAUDIO_INTERNAL_ERROR);

      // behave as if we just did a seek operation
      ds->post_seek = true;

      // we know the block number
      ds->block_number = 0;

      // shrink the input buffer
      memory   += header_size;
      memavail -= header_size;
      skip = (int) header_size;

      // now decode the real block header and go back to the main block decode path with the real block header
      offset = radaudio_decode_block_header(memory, &ds->biases, &header, memavail);

      // if that's ALSO a stream header, it's a corrupt file
      if (offset == COMMON_STREAM_HEADER)
         return e(RADAUDIO_INVALID_DATA);
   }

   if (offset == COMMON_INCOMPLETE_DATA)
      return e(RADAUDIO_INCOMPLETE_DATA);
   else if (offset < 0)
      return e(RADAUDIO_INVALID_DATA);

   U32 block_length_in_bytes = header.block_bytes + offset;

   ///////////////////////////////////////////////////////
   // validate data
   //

   // block length isn't longer than spec max
   if (block_length_in_bytes > MAX_ENCODED_BLOCK_BYTES)
      return RADAUDIO_INVALID_DATA;

   // length of first stream doesn't go off end of block
   U32 mid_side_band_length = header.mid_side_bands ? (24/MACRO_BAND_SIZE+7)/8 : 0;
   if (offset + mid_side_band_length + header.vbstream0_length > block_length_in_bytes)
         return e(RADAUDIO_INVALID_DATA);

   // final
   if (header.final_block)
      if (header.final_samples_discard > RADAUDIO_SHORT_BLOCK_LEN)
         return e(RADAUDIO_INVALID_DATA);

   // can't have more RLE entries than coefficients
   if (header.num_runlength_array > (U32) 2*(header.this_block_short ? RADAUDIO_SHORT_BLOCK_LEN+1 : RADAUDIO_LONG_BLOCK_LEN+1))
      return e(RADAUDIO_INVALID_DATA);

   if (block_length_in_bytes > memavail)
      return e(RADAUDIO_INCOMPLETE_DATA);

   if (header.final_block)
      ds->at_eof = true;

   rrbool is_short_block = header.this_block_short;
   ds->current_block_short = (U8) is_short_block;
   ds->next_block_short = header.next_block_short;

   radaudio_rate_info *bi = ds->info[is_short_block];
   int num_channels = header.num_channels_encoded;

   int nz_mode = header.nonzero_bitarray_mode;

   U8 *mid_side_bands = memory + offset;
   U8 *post_header = mid_side_bands + mid_side_band_length;

   U8 *vbstream2 = post_header + header.vbstream0_length;
   U8 *packet_end = memory + block_length_in_bytes;

   int error=0;

   // we initialize the 'end' pointers for each stream to the end of valid data
   // in that packet, not the end of that stream. So without further tests, they
   // could read the same raw bytes as part of multiple streams; but this is used
   // just to guarantee no memory overreads.
   decode_vbstream_init(&dec.stream[0], post_header, packet_end, &error);
   decode_vbstream_init(&dec.stream[1], packet_end , post_header, &error);
   decode_vbstream_init(&dec.stream[2], vbstream2  , packet_end, &error);

   U32 midside_bands=0;
   if (header.mid_side_encoded)
      midside_bands = 0xffffffff;
   else if (header.mid_side_bands) {
      RR_COMPILER_ASSERT(MACRO_BAND_SIZE == 3);
      U8 midside_band_triples = *mid_side_bands; // read 8 bits
      int k=0;
      for (int j=0; j < bi->num_bands; j += 3, ++k) {
         if (midside_band_triples & (1 << k))
            midside_bands |= (7 << j);
      }
   }

   U8 band_exponents[32*2];
   int cur_band_exponents=0;

   PROF_END(header);

   //
   // decode the band energy first, in case we want to use it to compute/predict other things (we don't anymore)
   //

   // band exponents
   PROF_BEGIN(huffman);
   if (header.predict_stereo_exponent && num_channels == 2) {
      decode_huff_array(&dec, &rada_band_exponent_correct_huff       , band_exponents   , bi->num_bands, &error);
      decode_huff_array(&dec, &rada_band_exponent_stereo_correct_huff, band_exponents+32, bi->num_bands, &error);
   } else {
      decode_huff_array(&dec, &rada_band_exponent_correct_huff, band_exponents, bi->num_bands * num_channels, &error);
   }
   PROF_END(huffman);

   if (error) {
      return e(RADAUDIO_INVALID_DATA);
   }

   PROF_BEGIN(unpack);
   for (c=0; c < (header.predict_stereo_exponent ? 1 : num_channels); ++c) {
      int lastv = PREDICT_FIRST_BAND_EXP;
      for (int j=0; j < bi->num_bands; ++j) {
         int v = (S8) band_exponents[cur_band_exponents++];
         v += lastv;
         lastv = v;
         bd[c].band_exponent[j] = v;
      }
   }

   // decode stereo predicted exponents
   if (header.predict_stereo_exponent && num_channels == 2) {
      for (int j=0; j < bi->num_bands; ++j)
         bd[1].band_exponent[j] = bd[0].band_exponent[j] + (S8) band_exponents[32+j];
   }
   PROF_END(unpack);

   RAD_ALIGN(U16, m_mantissa[MAX_BANDS*2+16], 16);

   PROF_BEGIN(compute_mantissa_len);
   for (c=0; c < num_channels; ++c) {
      // THIS LOGIC MUST BE EXACTLY REPLICATED IN THE COMPRESSOR!!!
      compute_mantissa_bitcount(
                                          ds->samprate_mode,
                                          is_short_block,
                                          ds->mantissa_param,
                                          bd[c].band_exponent,
                                          bd[c].band_mantissa_bitcount);
   }
   PROF_END(compute_mantissa_len);

   if (error)
      return e(RADAUDIO_INVALID_DATA);

   PROF_BEGIN(varbits);
   {
      int slot=0;

      for (c = 0; c < num_channels; ++c) {
         for (int j=0; j < bi->num_bands; ++j) {
            U8 size = bd[c].band_mantissa_bitcount[j];
            U16 mantissa = (U16) decode_vbstream_bits(&dec.stream[2], size, &error);
            m_mantissa[slot] = mantissa << (MAX_FINE_ENERGY_BITS - size);
            ++slot;
         }
      }
   }
   PROF_END(varbits);

   if (error)
      return e(RADAUDIO_INVALID_DATA);

   //
   // now do all the remaining entropy decoding
   //

   #define runlen_value_sentinel_size        2   // room to write two END_OF_ZERORUN markers to preven overread if input doesn't have them
   #define runlen_read_sentinel_size        16   // room to write dummy values for SIMD to run on multiple-of-16-bytes
   #define nonzero_coefficients_padding     32   // room to write dummy values for SIMD overwrite/overread, both of which are at most 16
   #define coeff_pair_padding               16   // room to write dummy data when unpacking

   #define runlen_pad  (runlen_value_sentinel_size + runlen_read_sentinel_size)

   #define max_runlength_data             1025  // 1024 empty runs per channel, plus two end-of-run markers

   RAD_ALIGN(U8, subband_value         [2* MAX_SUBBANDS                      ], 16);
   RAD_ALIGN(U8, subband_correction    [2* MAX_BANDS                         ], 16);
   RAD_ALIGN(U8, subband_stereo_correct[   MAX_SUBBANDS                      ], 16);

   RAD_ALIGN(S8, nonzero_coefficients  [2* 1024+nonzero_coefficients_padding ], 16);

   RAD_ALIGN(U8, runlength_data        [2* max_runlength_data + runlen_pad   ], 16);
   RAD_ALIGN(U8, nonzero_flagbits      [2* (1024/8) + 16                     ], 16);

   int num_subband_values0=0;
   int num_subband_corrections=0, num_subband_stereo_correct=0;
   int num_runlength_data=header.num_runlength_array;
   int num_coeff_pairs;

   for (int j=0; j < bi->num_bands; ++j) {
      int numsub = bi->num_subbands_for_band[j];
      if (numsub > 1) {
         for (c=0; c < num_channels; ++c) {
            if (bd[c].band_exponent[j] == BAND_EXPONENT_NONE && SUBBANDS_SKIP_EMPTY_BANDS)
               continue;
            int n = numsub;
            if (c == 1 && header.predict_stereo_subband) {
                num_subband_stereo_correct += n;
            } else {
               if (!header.disable_final_subband_predict) {
                  --n;
                  ++num_subband_corrections;
               }
               num_subband_values0 += n;
            }
         }
      }
   }

   // subband values
   PROF_BEGIN(huffman);
   if (!is_short_block) {
      decode_huff_array(&dec, &rada_subband_value_huff                        , subband_value  , num_subband_values0    , &error);
      if (!header.disable_final_subband_predict)
         decode_huff_array(&dec, &rada_subband_value_last_in_band_correct_huff, subband_correction, num_subband_corrections, &error);
      if (header.predict_stereo_subband)
         decode_huff_array(&dec, &rada_subband_value_stereo_correct_huff, subband_stereo_correct, num_subband_stereo_correct, &error);
   }
   PROF_END(huffman);

   if (error)
      return e(RADAUDIO_INVALID_DATA);
                               // coefficient zero-runlength data
   if (num_runlength_data > 1025*2)
      return e(RADAUDIO_INVALID_DATA);

   if (!is_short_block && nz_mode != 3) {
      U8 huffbits[2048];
      radaudio_nonzero_blockmode_descriptor *bdesc = &ds->nz_desc[nz_mode];

      PROF_BEGIN(huffman);
      {
         int p=0;
         for (int i=0; i < NUM_NZ_HUFF; ++i) {
            int q = bdesc->num_chunks_per_huff[i];
            if (q) {
               decode_huff_array(&dec, rada_nonzero_bitflags_huff[i], huffbits+p, q*8*num_channels, &error);
               p += q*8*num_channels;
            }
         }
      }
      PROF_END(huffman);

      PROF_BEGIN(huffman);
      int j=0, s=num_channels-1;
      for (c=0; c < num_channels; ++c, ++s) {
         for (int i=0; i < bdesc->num_8byte_chunks; ++i, ++j) {
            U8 p = bdesc->source_pos[s][i];
            U64 xor = (U64)0 - bdesc->invert_chunk[i]; // if invert_chunk=1, this gives ~0 (invert), else 0.
            RR_PUT64_NATIVE(&nonzero_flagbits[j*8], xor ^ RR_GET64_NATIVE(huffbits+p*8));
         }
      }
      PROF_END(huffman);
   }

   if (error)
      return e(RADAUDIO_INVALID_DATA);

   PROF_BEGIN(huffman);
   decode_huff_array(&dec, &rada_zero_runlength_huff, runlength_data, num_runlength_data, &error);

   // add sentinel so we don't need to length-check loop
   runlength_data[num_runlength_data+0] = END_OF_ZERORUN;

   // add extra sentinel in case the data is invalid and doesn't have the stereo separator, so we don't need to length-check loop
   runlength_data[num_runlength_data+1] = END_OF_ZERORUN;
   PROF_END(huffman);

   if (error)
      return e(RADAUDIO_INVALID_DATA);

   PROF_BEGIN(count_coefficients_huff);
   // values of MAX_RUNLEN don't indicate coefficients, because they have a following real runlength
   int num_nonzero_coefficients = count_bytes_below_value_sentinel16(ds, runlength_data, num_runlength_data, MAX_RUNLEN);

   if (!is_short_block) {
      int num_flagbit_bytes = ds->nz_desc[nz_mode].num_8byte_chunks * 8;
      if (num_flagbit_bytes != 0)
         num_nonzero_coefficients += count_set_bits_multiple8_sentinel8(ds, nonzero_flagbits, num_flagbit_bytes*num_channels);
   }
   PROF_END(count_coefficients_huff);

   // runlength data + flagbits combined could be too many coefficients

   if (num_nonzero_coefficients > num_channels*1024)
      return e(RADAUDIO_INVALID_DATA);

   int nz_selector = is_short_block ? 4 : nz_mode;

   // coefficients -- need to have decoded the runlength data to know how many coefficients
   {
      // transient temp mem
      RAD_ALIGN(U8, coefficient_pairs[2* 1024/2 * 2 + coeff_pair_padding], 16);

      PROF_BEGIN(huffman);
      num_coeff_pairs = (num_nonzero_coefficients+1)/2;
      int tp = ds->nz_correlated_huffman_selectors[HS_COEFF_PAIR][nz_selector];
      decode_huff_array(&dec, rada_nonzero_coefficient_pair_huff[tp], coefficient_pairs, num_coeff_pairs, &error);
      PROF_END(huffman);

      // convert coefficient pairs to coefficients
      PROF_BEGIN(unpack);
      unpack_nibbles_input_excess16_output_excess16_multiple32_default1(ds, nonzero_coefficients, coefficient_pairs, num_coeff_pairs);
      PROF_END(unpack);
   }

   // read and apply bottom bits of run length data
   // we have 2*1024 coeffs, COARSE_RUNLEN_THRESHOLD=60 and such runs are followed by a
   // nonzero coefficient, so per 1024 coeffs we can have at most floor(1024/61)=16 of these
   // (32 total between the total channels). in practice, the typical counts are 0-4.
   PROF_BEGIN(update_runlength);
   {
      U8 *cur = runlength_data;
      U8 *end = runlength_data + num_runlength_data; // we have runlen_read_sentinel_size of padding, so can be sloppy
      while (cur < end) {
         cur = find_next_coarse_run_excess16(ds, cur, end);
         if (cur >= end)
            break;

         rrAssert(*cur >= COARSE_RUNLEN_THRESHOLD && *cur < MAX_RUNLEN);

         // process this run and advance
         U8 extra = (U8) decode_vbstream_bits(&dec.stream[2], 2, &error); 
         *cur += extra;
         ++cur;
      }
   }
   PROF_END(update_runlength);

   // big coefficients are coded as value 0 in the coefficient pairs

   {
      // transient temp mem, only used right here
      RAD_ALIGN(S8, big_coefficients[2* 1024 + 16], 16);

      // count zero bytes
      int num_big_coefficients = count_bytes_below_value_sentinel16(ds, (U8*) nonzero_coefficients, num_nonzero_coefficients, 1);

      PROF_BEGIN(huffman);
      int tb = ds->nz_correlated_huffman_selectors[HS_COEFF_BIG][nz_selector];
      decode_huff_array(&dec, rada_nonzero_coefficient_big_huff[tb], (U8*) big_coefficients, num_big_coefficients, &error);
      PROF_END(huffman);

      PROF_BEGIN(unbias);
      // big coefficients are byte-sized, so stored aligned in stream[2]
      //decode_stream_align_to_byte(&dec.stream[2]);
      //U8 *bytestream = &dec.stream[2].bitstream[ dec.stream[2].read_pos_in_bits>>3 ];
      // bytestream ends at current position of reverse-read stream 1
      //decode_stream_align_to_byte(&dec.stream[1]);
      //U8 *bytestream_end = &dec.stream[1].bitstream[-(int)(dec.stream[1].read_pos_in_bits>>3)];

      // expand used to decode directly from the stream and hence needed a safety range
      if (!expand_nonzero_coefficients(ds, nonzero_coefficients, num_nonzero_coefficients,
                     big_coefficients, (big_coefficients+num_nonzero_coefficients), (big_coefficients+sizeof(big_coefficients))))
         return e(RADAUDIO_INVALID_DATA);
      PROF_END(unbias);
   }

   if (error)
      return e(RADAUDIO_INVALID_DATA);

   int cur_subband_values0=0;
   int cur_subband_corrections=0;
   int cur_subband_stereo_correct=0;
   int cur_nonzero_coefficients=0;
   int cur_runlength_data=0;

   PROF_BEGIN(compute_subbands);
   for (c=0; c < num_channels; ++c)
      memset(bd[c].quantized_subbands, 0, bi->num_subbands * 2);

   if (!is_short_block) {
      // subbands
      for (c=0; c < num_channels; ++c) {
         for (int j=0; j < bi->num_bands; ++j) {
            if (bi->num_subbands_for_band[j] == 1)
               continue;

            int start = bi->first_subband_for_band[j];
            int num_coded_subbands = bi->num_subbands_for_band[j];

            if (bd[c].band_exponent[j] == BAND_EXPONENT_NONE && SUBBANDS_SKIP_EMPTY_BANDS) {
               for (int i=0; i < num_coded_subbands; ++i)
                  bd[c].quantized_subbands[start+i] = (U16) (ds->subband_predicted_sum[j] / num_coded_subbands); // this value is predicted from in stero
            } else if (header.predict_stereo_subband && c == 1) {
               for (int i=0; i < num_coded_subbands; ++i) {
                  int predict = bd[0].quantized_subbands[start+i];
                  int correct = (S8) subband_stereo_correct[cur_subband_stereo_correct++];
                  bd[c].quantized_subbands[start+i] = (U16) (predict + correct);
               }
            } else {
               int predicted_sum = ds->subband_predicted_sum[j];
               int bias = ds->subband_bias[j];
               int partial_sum = 0;

               if (!header.disable_final_subband_predict)
                  --num_coded_subbands;

               for (int i=0; i < num_coded_subbands; ++i) {
                  int v = subband_value[cur_subband_values0++];
                  v -= bias; // remove bias
                  v = (v & 63);
                  bd[c].quantized_subbands[start+i] = (U16) v;
                  partial_sum += v;
               }

               if (!header.disable_final_subband_predict) {
                  int actual_sum = predicted_sum + (S8) subband_correction[cur_subband_corrections++];
                  int v = actual_sum - partial_sum;

                  if (v < 0) // @TODO investigate this case closely, why can't it be negative, should there be an upper bound?
                     return e(RADAUDIO_INVALID_DATA);

                  bd[c].quantized_subbands[start+num_coded_subbands] = (U16) v;
               }
            }
         }
      }
   }
   PROF_END(compute_subbands);

   if (error)
      return e(RADAUDIO_INVALID_DATA);

   for (c=0; c < num_channels; ++c) {
      int num_nonzero_bitarray_bytes = ds->nz_desc[nz_mode].num_8byte_chunks * 8;
      rrbool result = distribute_nonzero_coefficients(ds, bd[c].quantized_coeff_decode, bi->num_quantized_coeffs,
                                                      runlength_data, &cur_runlength_data,
                                                      nonzero_coefficients, &cur_nonzero_coefficients,
                                                      nonzero_flagbits + c*num_nonzero_bitarray_bytes, is_short_block ? 0 : num_nonzero_bitarray_bytes*8, c);
      if (!result)
         return e(RADAUDIO_INVALID_DATA);
   }

   // we expect to read the first sentinel; if we read the second, it's a bug
   if (cur_runlength_data > num_runlength_data+1)
      return e(RADAUDIO_INVALID_DATA);

   FFT_ALIGN(F32, rawdata[MAX_COEFFS]);
   F32 *data1 = rawdata, *data2 = rawdata + (bi->num_coeffs >> 1);

   if (ds->num_channels==1) {
      // mono stream
      (void) decode_channel_before_imdct(ds,               &bd[0], 0, ds->block_number, m_mantissa);
      (void) decode_channel_imdct       (ds, rawdata     , &bd[0], 0);
      len =  decode_channel_after_imdct (ds, data1, data2, &bd[0], 0, output[0]);
   } else if (ds->num_channels==2 && num_channels==1) {
      // stereo stream with mono block
      (void) decode_channel_before_imdct(ds,               &bd[0], 0, ds->block_number, m_mantissa);
      (void) decode_channel_imdct       (ds, rawdata     , &bd[0], 0);
      (void) decode_channel_after_imdct (ds, data1, data2, &bd[0], 0, output[0]);
      len =  decode_channel_after_imdct (ds, data1, data2, &bd[0], 1, output[1]);
   } else {
      // stereo stream with stereo block
      (void) decode_channel_before_imdct(ds, &bd[0], 0, ds->block_number           , m_mantissa);
      (void) decode_channel_before_imdct(ds, &bd[1], 1, ds->block_number^0x55555555, m_mantissa+ds->info[is_short_block]->num_bands);

      // midside decode
      for (int j=0; j < bi->num_bands; ++j) {
         if (midside_bands & (1 << j)) {
            F32 *coeffs1 = bd[0].dequantized_coeff_decode;
            F32 *coeffs2 = bd[1].dequantized_coeff_decode;
            int start = bi->first_coeff_for_band[j];
            int end = start + bi->num_coeffs_for_band[j];
            for (int i=start; i < end; ++i) {
               float x = coeffs1[i];
               float y = coeffs2[i]*0.5f;
               coeffs1[i] = x+y;
               coeffs2[i] = x-y;
            }
         }
      }
      (void) decode_channel_imdct       (ds, rawdata     , &bd[0], 0);
      (void) decode_channel_after_imdct (ds, data1, data2, &bd[0], 0, output[0]);
      (void) decode_channel_imdct       (ds, rawdata     , &bd[1], 1);
      len =  decode_channel_after_imdct (ds, data1, data2, &bd[1], 1, output[1]);
   }

   if (error)
      return e(RADAUDIO_INVALID_DATA);

   *memconsumed = block_length_in_bytes + skip;

   ++ds->block_number;
   ds->last_block_short = ds->current_block_short;

   if (header.final_block) {
      if (header.final_samples_discard > (U32) len)
         return e(RADAUDIO_INVALID_DATA);
      else {
         int total = len - header.final_samples_discard;
         return total;
      }
   } else
      return len;
}

size_t RadAudioDecoderMemoryRequired(U8 *raw_header, size_t raw_header_size)
{
   size_t size = sizeof(radaudio_decoder_state);
   size += decoder_align-1; // allow room to align

   int num_channels;
   if (raw_header == NULL)
      num_channels = 2;
   else {
      radaudio_stream_header_unpacked header;
      if (radaudio_unpack_stream_header(raw_header, raw_header_size, &header) == 0)
         return 0;
      num_channels = header.num_channels;
   }

   // room for buffered samples from previous block
   size += RADAUDIO_LONG_BLOCK_LEN/2 * sizeof(S16) * num_channels;
   return size;
}

static radaudio_decoder_state * radaudio_decompressor_memalloc(radaudio_stream_header_unpacked *header, void *vmem, size_t memsize)
{
   int i;
   union {
      UINTa   addr;
      U8    * ptr;
   } convert;

   if (memsize < sizeof(radaudio_decoder_state))
      return 0;

   U8 *mem = vmem;
   radaudio_decoder_state *ds;
   ds = (void*) mem; mem += sizeof(*ds);
   memset(ds, 0, sizeof(*ds));

   // align data after struct
   convert.ptr = mem;
   convert.addr = (convert.addr + decoder_align-1) & ~(decoder_align-1);
   mem = convert.ptr;
   for (i=0; i < header->num_channels; ++i) {
      ds->prev_block_right_samples[i] = (void *) mem;
      mem += RADAUDIO_LONG_BLOCK_LEN/2 * sizeof(S16);
   }

   size_t memneeded = mem - (U8*)vmem;
   if (memneeded > memsize)
      return 0;

   ds->last_block_short = 1; // shouldn't matter
   return ds;
}

RadAudioDecoder *RadAudioDecoderOpen(U8 *raw_header, size_t raw_header_size, void *vmem, size_t memsize, size_t *header_read)
{
   radaudio_decoder_state *ds;
   radaudio_stream_header_unpacked header;
   if (raw_header_size < sizeof(radaudio_stream_header))
      return NULL;
   size_t header_size = radaudio_unpack_stream_header(raw_header, raw_header_size, &header);
   if (header_size == 0)
      return NULL;

   // unpack_stream_header does some sanity checking, here's the rest:

   for (int i=0; i < NUM_NZ_MODE; ++i)
      if (header.nzmode_num64[i] > MAX_NZ_BLOCKS)
         return 0;

   ds = radaudio_decompressor_memalloc(&header, vmem, memsize);
   if (ds == NULL)
      return 0;

   ds->version = header.version;
   ds->num_channels = header.num_channels;
   ds->skip_bytes = 0;//(U8) header_size;
   ds->cpu = cpu_detect();
   ds->post_seek = true; // very first block decoded discards input

   ds->sample_rate   = header.sample_rate;
   ds->samprate_mode = header.sample_rate_mode;
   memcpy(ds->subband_bias, header.subband_bias, sizeof(ds->subband_bias));

   ds->info[0] = &radaudio_rateinfo[0][ds->samprate_mode];
   ds->info[1] = &radaudio_rateinfo[1][ds->samprate_mode];

   memcpy(ds->subband_predicted_sum, header.subband_predicted_sum, 24);
   memcpy(ds->mantissa_param , header.mantissa_param, sizeof(header.mantissa_param));
   compute_bias_set(&ds->biases, header.bytes_bias);

   for (int i=0; i < NUM_NZ_MODE; ++i) {
      ds->nz_desc[i].num_8byte_chunks = header.nzmode_num64[i];
      if (ds->nz_desc[i].num_8byte_chunks > MAX_NZ_BLOCKS)
         return 0;
      for (int j=0; j < MAX_NZ_BLOCKS; ++j) {
         ds->nz_desc[i].huffman_table_for_chunk[j] = (header.nzmode_huff[i][j] & ~NZ_MODE_INVERT);
         ds->nz_desc[i].invert_chunk[j]            = (header.nzmode_huff[i][j] & NZ_MODE_INVERT) != 0;
         if (ds->nz_desc[i].huffman_table_for_chunk[j] >= NUM_NZ_HUFF)
            return 0;
      }
   }
   for (int j=0; j < NUM_NZ_SELECTOR; ++j)
      for (int i=0; i < NUM_SELECTOR_MODES; ++i)
         ds->nz_correlated_huffman_selectors[j][i] = header.nzmode_selectors[j][i];

   radaudio_init_nz_desc(ds->nz_desc);

   if (header_read)
      *header_read = header_size;

   return ds;
}

#ifdef RADAUDIO_DEVELOPMENT
void RadAudioDecoderForceIntelCPU(RadAudioDecoder *hradaud, rrbool has_sse2, rrbool has_ssse3, rrbool has_sse4_1, rrbool has_popcnt, rrbool has_avx2)
{
   radaudio_decoder_state *ds = (radaudio_decoder_state *) hradaud;
   RR_UNUSED_VARIABLE(ds);
   #ifdef __RADX86__
   ds->cpu.has_sse2   = (U8) has_sse2;
   ds->cpu.has_ssse3  = (U8) has_ssse3;
   ds->cpu.has_sse4_1 = (U8) has_sse4_1;
   ds->cpu.has_popcnt = (U8) has_popcnt;
   ds->cpu.has_avx2   = (U8) has_avx2;
   #endif   
}
#endif

static void decode_version(RadAudioInfo *info, U32 version)
{
   info->major_version      = (U8 ) ((version & 0xff000000) >> 24);
   info->minor_version      = (U8 ) ((version & 0x00ff0000) >> 16);
   info->sequential_version = (U16) ((version & 0x0000ffff) >>  0);
}

void RadAudioDecoderGetInfo(const RadAudioDecoder *hradaud, RadAudioInfo *out_info)
{
   radaudio_decoder_state *ds = (radaudio_decoder_state *) hradaud;
   out_info->sample_rate = ds->sample_rate;
   out_info->num_channels = ds->num_channels;
   decode_version(out_info, ds->version);
}

size_t RadAudioDecoderGetInfoHeader(U8* raw_header, size_t raw_header_size, RadAudioInfo *out_info)
{
   radaudio_stream_header_unpacked header;
   size_t header_size = radaudio_unpack_stream_header(raw_header, raw_header_size, &header);
   if (header_size == 0)
      return 0;
   out_info->sample_rate = header.sample_rate;
   out_info->num_channels = header.num_channels;   
   decode_version(out_info, header.version);
   return header_size;
}

RADDEFFUNC void RadAudioDecoderDidSeek(RadAudioDecoder *radaudio_decomp)
{
   radaudio_decoder_state *ds = (radaudio_decoder_state *) radaudio_decomp;
   ds->post_seek = true;
   ds->at_eof    = false;
}

int RadAudioDecoderGetChunkLength(RadAudioDecoder *radaudio_decomp, const U8 *data, size_t data_avail)
{
   radaudio_decoder_state *ds = (radaudio_decoder_state *) radaudio_decomp;
   if (ds->at_eof)
      return RADAUDIO_AT_EOF;

   if (data_avail < 4)
      return RADAUDIO_INCOMPLETE_DATA;

   radaudio_block_header_unpacked header;
   int offset = radaudio_decode_block_header(data, &ds->biases, &header, data_avail);

   if (offset == COMMON_STREAM_HEADER)
      return RADAUDIO_START_OF_STREAM;
   if (offset == COMMON_INCOMPLETE_DATA)
      return RADAUDIO_INCOMPLETE_DATA;
   if (offset == COMMON_INVALID_DATA)
      return RADAUDIO_INVALID_DATA;

   U32 block_length_in_bytes = header.block_bytes + offset;

   // validate data

   if (block_length_in_bytes > MAX_ENCODED_BLOCK_BYTES)
      return RADAUDIO_INVALID_DATA;

   if (offset + header.vbstream0_length > block_length_in_bytes)
      return RADAUDIO_INVALID_DATA;

   if (header.final_block)
      if (header.final_samples_discard > RADAUDIO_SHORT_BLOCK_LEN)
         return RADAUDIO_INVALID_DATA;

   if (header.num_runlength_array > (U32) 2*(header.this_block_short ? RADAUDIO_SHORT_BLOCK_LEN+1 : RADAUDIO_LONG_BLOCK_LEN+1))
      return RADAUDIO_INVALID_DATA;

   return header.block_bytes + offset;
}

// returns the number of samples output per channel, and update 'memconsumed'
// with the amount of memory consumed.
//
// return values:
//     n       number of samples decoded (for one channel, e.g. n=1024 means 1024 stereo pairs)
//     0       can decode 0 samples legitimately, e.g. first block or after seeking
//    -1       at end-of-stream
//    -2       not enough input data to decode a frame, always consumes 0
//    -3       error (e.g. corrupt stream)
int RadAudioDecoderDecodeChunk(
                              RadAudioDecoder *radaudio_decomp,
                              const U8 *mem         ,
                              size_t memavail       ,
                              size_t *memconsumed   ,
                              F32 *output_samples[2],
                              size_t max_samples_per_channel
                             )
{
   *memconsumed = 0;

   radaudio_decoder_state *ds = (radaudio_decoder_state *) radaudio_decomp;
   if (ds->at_eof)
      return RADAUDIO_AT_EOF;

   if (memavail < 7)
      return RADAUDIO_INCOMPLETE_DATA;

   size_t used;

   if (!ds)
      return -2;

   PROF_BEGIN(decoder_all);

   size_t skip = ds->skip_bytes;
   int len = decode_block(ds, output_samples, (U8*)mem+skip, memavail-skip, &used);
   ds->post_seek = false;

   if (len >= 0) {
      *memconsumed = used + skip;
      ds->skip_bytes = 0;
      ds->fully_decoded += len;
   }

   PROF_END(decoder_all);
   return len;
}

#ifdef RADAUDIO_DEVELOPMENT
// internal use
int RadAudioDecoderGetProfileData(RadAudioDecoder *hradaud, radaudio_profile_value *aprofile, int num_profile)
{
   RR_UNUSED_VARIABLE(hradaud);
   int n = RR_MIN(num_profile, PROF_total_count - 1);
   static const char *names[] = {
      #define PROF(x) #x,
      PROFILE_ZONES()
      #undef PROF
   };

   if (aprofile) {
      double overhead_time = 0.0;
      // we have an empty profiling region to estimate overhead of tracking a region to begin with
      if (profile_counts[PROF_overhead]) {
         overhead_time = rrTicksToSeconds(profile_times[PROF_overhead]) / profile_counts[PROF_overhead];
      }
      for (int i=0; i < n; ++i) {
         aprofile[i].name = names[i];
         // subtract out estimated overhead
         aprofile[i].time = rrTicksToSeconds(profile_times[i]) - overhead_time * profile_counts[i];
      }
   } else {
      profile = num_profile;
   }

   for (int i=0; i < PROF_total_count; ++i) {
      profile_times[i] = 0;
      profile_counts[i] = 0;
   }
   return n;
}
#else
int RadAudioDecoderGetProfileData(RadAudioDecoder *hradaud, radaudio_profile_value *profile, int num_profile)
{
   RR_UNUSED_VARIABLE(hradaud); RR_UNUSED_VARIABLE(profile); RR_UNUSED_VARIABLE(num_profile);
   return 0;
}
#endif
