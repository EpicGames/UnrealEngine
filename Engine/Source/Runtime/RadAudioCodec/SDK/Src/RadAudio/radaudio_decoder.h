// Copyright Epic Games Tools, LLC. All Rights Reserved.
#ifndef INCLUDE_RADAUDIO_DECODER_H
#define INCLUDE_RADAUDIO_DECODER_H

#include "egttypes.h"
#include <stddef.h>

typedef struct RadAudioDecoder RadAudioDecoder;

typedef struct
{
   U16 sequential_version;
   U8  minor_version;
   U8  major_version;
   int sample_rate;
   int num_channels;
} RadAudioInfo;

#define RADAUDIO_DECODER_AT_EOF            -1
#define RADAUDIO_DECODER_INCOMPLETE_DATA   -2
#define RADAUDIO_DECODER_INVALID_DATA      -3
#define RADAUDIO_DECODER_START_OF_STREAM   -4
#define RADAUDIO_DECODER_INTERNAL_ERROR    -5
#define RADAUDIO_DECODER_MAX_OUTPUT_SAMPLES_PER_CHANNEL_PER_CHUNK    1024

RADDEFSTART

// RADAUDIO compatibility version - if these are the same the exports are named the same so we expect
// that if the linker selects a different copy they all work.
#define RADAUDIO_DECODER_LIBRARY_VERSION    1

#define RADAUDIO_DECODER_HAS_INTERLEAVING

// The max size of a stream header. It could be smaller but this is a safe size
// for buffers.
#define RADAUDIO_STREAM_HEADER_SIZE   128

#ifndef RR_STRING_JOIN3
#define RR_STRING_JOIN3(arg1, arg2, arg3)            RR_STRING_JOIN_DELAY3(arg1, arg2, arg3)
#define RR_STRING_JOIN_DELAY3(arg1, arg2, arg3)      RR_STRING_JOIN_IMMEDIATE3(arg1, arg2, arg3)
#define RR_STRING_JOIN_IMMEDIATE3(arg1, arg2, arg3)  arg1 ## arg2 ## arg3
#endif

#ifdef RADAUDIO_WRAP
#define RADAUDIO_DECODER_NAME(name) RR_STRING_JOIN3(RADAUDIO_WRAP, name##_, RADAUDIO_DECODER_LIBRARY_VERSION )
#else
#define RADAUDIO_DECODER_NAME(name) RR_STRING_JOIN( name##_, RADAUDIO_DECODER_LIBRARY_VERSION )
#endif

#define RadAudioDecoderMemoryRequired        RADAUDIO_DECODER_NAME(RadAudioDecoderMemoryRequired)
#define RadAudioDecoderOpen                  RADAUDIO_DECODER_NAME(RadAudioDecoderOpen)
#define RadAudioDecoderGetInfo               RADAUDIO_DECODER_NAME(RadAudioDecoderGetInfo)
#define RadAudioDecoderGetInfoHeader         RADAUDIO_DECODER_NAME(RadAudioDecoderGetInfoHeader)
#define RadAudioDecoderDecodeChunk           RADAUDIO_DECODER_NAME(RadAudioDecoderDecodeChunk)
#define RadAudioDecoderGetProfileData        RADAUDIO_DECODER_NAME(RadAudioDecoderGetProfileData)
#define RadAudioDecoderDidSeek               RADAUDIO_DECODER_NAME(RadAudioDecoderDidSeek)
#define RadAudioDecoderGetChunkLength        RADAUDIO_DECODER_NAME(RadAudioDecoderGetChunkLength)

#define RadAudioInterleave                   RADAUDIO_DECODER_NAME(RadAudioInterleave)
#define RadAudio1ChToS16                     RADAUDIO_DECODER_NAME(RadAudio1ChToS16)
#define RadAudioInterleave2ChToS16           RADAUDIO_DECODER_NAME(RadAudioInterleave2ChToS16)
#define RadAudioInterleave4ChToS16           RADAUDIO_DECODER_NAME(RadAudioInterleave4ChToS16)

// RadAudioDecoderMemoryRequired()
//
// returns the amount of memory that needs to be passed to Open().
// stereo requires more than mono. Returns 0 if the header is invalid.
//
// Pass in NULL to get the maximum size required for any stream.
RADDEFFUNC size_t RadAudioDecoderMemoryRequired(U8* stream_header, size_t stream_header_bytes_valid);

// RadAudioDecoderOpen()
//
// returns a pointer to a decoder structure that is allocated in 'mem'.
// you don't need to close it, just dispose of the memory yourself when done
//
// returns NULL if header is invalid or if memsize is smaller than RadAudioDecoderMemoryRequired()
// or if stream_header_bytes_valid is too small (needs to be RADAUDIO_STREAM_HEADER_SIZE bytes)
//
// if the header is large enough to be read, the size of the header will be written
// to *header_size
//
// there is no RadAudioDecoderClose() function as the structure holds no resources;
// just free the memory you passed in as necessary
RADDEFFUNC RadAudioDecoder* RadAudioDecoderOpen(U8 *stream_header, size_t stream_header_bytes_valid, void *mem, 
   size_t memsize, size_t *out_header_size);

// RadAudioDecoderGetInfo*
//
// return info about the decoder either from an unopened stream header or an open
// decoder.
//
// 
RADDEFFUNC void   RadAudioDecoderGetInfo(const RadAudioDecoder *radaudio_decoder, RadAudioInfo *out_info);

// returns size of the header on success, false if not a valid stream header
RADDEFFUNC size_t RadAudioDecoderGetInfoHeader(U8* stream_header, size_t stream_header_bytes_valid, RadAudioInfo *out_info);


// RadAudioDecoderChunk()
//
// - `radaudio_decoder`: a pointer to a decoder created with `RadAudioDecoderOpen()`
// - `data`            : pointer to the next chunk of compressed data to decode
// - `data_avail`      : the length of the data pointed to by `data`
// - `data_consumed`   : output value, the amount of data consumed decoding this chunk
// - `output_samples`  : a pointer to an array of pointers where to write the output, one pointer per channel
// - `max_samples`     : the maximum number of samples that can be written to each of the `output_samples` pointers
//
// For the first call, `data` should either point to the full stream (that is,
// it points to the stream header), or it can point to the first block output by
// radaudio_encode_block (e.g. if you store the stream header separately from
// the sequence of blocks in the stream).
//
// returns the number of samples output per channel, and sets `data_consumed`
// to the number of bytes of input data consumed... you should advance `data`
// by this much. data_consumed can be 0 if there wasn't enough data.
//
// Note that 0 is a valid "number of samples" decoded, it means it decoded a
// block but there were no samples fully decoded due to the way the codec works. (This
// should only happen at the start of the stream and after seeks.)
//
// returns RADAUDIO_DECODER_AT_EOF if at the "end of stream", i.e. there's no more data in the audio stream
// returns RADAUDIO_DECODER_INCOMPLETE_DATA if there's not enough data to decode a block
// returns RADAUDIO_DECODER_INVALID_DATA if the block is invalid
RADDEFFUNC int RadAudioDecoderDecodeChunk(RadAudioDecoder *radaudio_decoder, const U8 *data, size_t data_avail,
          size_t *data_consumed, F32 *(output_samples[2]), size_t max_samples);

// Returns the length of the next block, given the block header (needs as much as 6 bytes).
//
// returns RADAUDIO_DECODER_AT_START_OF_STREAM if you pass in the stream header instead of the block header
// returns RADAUDIO_DECODER_AT_EOF if at the "end of stream", i.e. there's no more data expected in the audio stream (regardless of the block you pass in)
// returns RADAUDIO_DECODER_INCOMPLETE_DATA if there's not enough data to decode the block header
// returns RADAUDIO_DECODER_INVALID_DATA if the block header is invalid
RADDEFFUNC int RadAudioDecoderGetChunkLength(RadAudioDecoder *radaudio_decoder, const U8 *data, size_t data_avail);

// inform the decoder that you performed a seek operation on the input stream,
// such that the next decoder call is not continuous with the previous one.
RADDEFFUNC void RadAudioDecoderDidSeek(RadAudioDecoder *radaudio_decoder);



// utility function to interleave N buffers of float data into one output buffer of S16 (scales and clamps from -1.0 to 1 into -32768 to 32767)
RADDEFFUNC void  RadAudioInterleave( S16 * output, float const ** inputs, U32 input_count, U32 samples );

// low level direct routines, if these return 0, then they converted NOTHING (the sample count is too small for even one SIMD round).
//    if they return 1, then the ENTIRE buffer was converted.
RADDEFFUNC int   RadAudio1ChToS16( S16 * output, float const * input, U32 samples );
RADDEFFUNC int   RadAudioInterleave2ChToS16( S16 * output, float const * input_left, float const * input_right, U32 samples );
RADDEFFUNC int   RadAudioInterleave4ChToS16( S16 * output, float const * input_left, float const * input_right, float const * input_back_left, float const * input_back_right, U32 samples );

RADDEFEND

#endif
