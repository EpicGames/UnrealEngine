// Copyright Epic Games Tools, LLC. All Rights Reserved.
/////////////////////////////////////////////////////////////////////
//
//   AVX2 required
//

/* @cdep pre
   $when($NeedsClangMachineSwitches,
      $addlocalcswitches(-mavx2 -mavx)
   )
   $when($NeedsVCMachineSwitches,
      $addlocalcswitches(/arch:AVX2)
   )
*/

// Disable FP contraction, which allows compilers to
// automatically turn multiplies followed by additions into fused
// multiply-adds where available. This results in different results when
// FMAs are available (e.g. when compiling on x86 with AVX2 support) than
// when they are not, which breaks our compatibility guarantees.
//
// If you want FMAs you need to write them directly. (And be very
// careful to not introduce mismatches between build targets with
// and without FMA support.)

#if defined(_MSC_VER) && !defined(__clang__)
#pragma fp_contract(off)
#else
#pragma STDC FP_CONTRACT OFF
#endif

#include "radaudio_common.h"
#include "radaudio_decoder_avx2.h"
#include <stdio.h>

#ifdef RADAUDIO_WRAP
#define WRAPPED_NAME(name) RR_STRING_JOIN(RADAUDIO_WRAP, name##_)
#define radaudio_dummy_decode_avx2 WRAPPED_NAME(radaudio_dummy_decode_avx2)
#endif

int radaudio_dummy_decode_avx2(void) { return 0; } // avoid "empty translation unit" warning

#ifdef DO_BUILD_AVX2

#include <immintrin.h> // AVX

#define ps(x)  _mm_castsi128_ps(x)
#define pi(x)  _mm_castps_si128(x)
#define ps8(x)  _mm256_castsi256_ps(x)
#define pi8(x)  _mm256_castps_si256(x)

typedef __m128i int32x4;
typedef __m128i int16x8;
typedef __m128i int8x16;
typedef __m128  float4;

typedef __m256i int32x8;
typedef __m256i int16x16;
typedef __m256i int8x32;
typedef __m256  float8;

#define _mm_shuffle_ps_mine(a,imm) _mm_permute_ps((a), (imm))
#define _mm256_shuffle_ps_mine(a,imm) _mm256_permute_ps((a), (imm))

#define MY_SHUFFLE(w,x,y,z) _MM_SHUFFLE(z,y,x,w)

static RADFORCEINLINE float8 float8_reverse(float8 v)
{
   return _mm256_permutevar8x32_ps(v, _mm256_setr_epi32(7,6,5,4,3,2,1,0));
}

static RADFORCEINLINE float8 float8_load_rev(const float *data)
{
   return float8_reverse(_mm256_loadu_ps(data));
}

static RADFORCEINLINE void load_16int16_scaled(float8 *data_r0, float8 *data_r1, const S16 *data, float8 scale)
{
   // VPMOVSXWD from memory is a single instruction
   int32x8 data_r0i = _mm256_cvtepi16_epi32(_mm_loadu_si128((int16x8 *) data));
   int32x8 data_r1i = _mm256_cvtepi16_epi32(_mm_loadu_si128((int16x8 *) (data + 8)));
   // convert to int32 and scale
   *data_r0 = _mm256_mul_ps(scale, _mm256_cvtepi32_ps(data_r0i));
   *data_r1 = _mm256_mul_ps(scale, _mm256_cvtepi32_ps(data_r1i));
}

// rev_data is read forwards, but uses a reversed window
void radaudio_avx2_compute_windowed_sum_multiple16(float *output, int n, const float *fwd_data, const S16 *rev_data, float rev_scale, const float *window)
{
   float8 scale = _mm256_set1_ps(rev_scale);
   SINTa N2 = n >> 1;

   // front half
   for (SINTa i = 0; i < N2; i += 16) {
      // read rev_data and scale
      float8 data_r0, data_r1;
      load_16int16_scaled(&data_r0, &data_r1, rev_data + i, scale);

      // fwd_data is read reversed (confusing naming, I know)
      float8 data_f0  = float8_load_rev(fwd_data+N2 - i - 8);
      float8 data_f1  = float8_load_rev(fwd_data+N2 - i - 16);

      // windows
      float8 win_r0   = float8_load_rev(window+n - i - 8);
      float8 win_r1   = float8_load_rev(window+n - i - 16);
      float8 win_f0   = _mm256_loadu_ps(window + i);
      float8 win_f1   = _mm256_loadu_ps(window + i + 8);

      // weighted sum
      float8 sum0     = _mm256_sub_ps(_mm256_mul_ps(data_r0, win_r0), _mm256_mul_ps(data_f0, win_f0));
      float8 sum1     = _mm256_sub_ps(_mm256_mul_ps(data_r1, win_r1), _mm256_mul_ps(data_f1, win_f1));
      _mm256_storeu_ps(output+i  , sum0);
      _mm256_storeu_ps(output+i+8, sum1);
   }

   // back half
   for (SINTa i = 0; i < N2; i += 16) {
      // read rev_data and scale
      // (this one is actually reversed, hence r1 and r0 being swapped)
      float8 data_r0, data_r1;
      load_16int16_scaled(&data_r1, &data_r0, rev_data+N2 - i - 16, scale);

      // fwd_data
      float8 data_f0  = _mm256_loadu_ps(fwd_data + i);
      float8 data_f1  = _mm256_loadu_ps(fwd_data + i + 8);

      // windows
      float8 win_r0   = _mm256_loadu_ps(window+N2 - i - 8);
      float8 win_r1   = _mm256_loadu_ps(window+N2 - i - 16);
      float8 win_f0   = _mm256_loadu_ps(window+N2 + i);
      float8 win_f1   = _mm256_loadu_ps(window+N2 + i + 8);

      // weighted sum
      float8 sum0     = _mm256_add_ps(float8_reverse(_mm256_mul_ps(data_r0, win_r0)), _mm256_mul_ps(data_f0, win_f0));
      float8 sum1     = _mm256_add_ps(float8_reverse(_mm256_mul_ps(data_r1, win_r1)), _mm256_mul_ps(data_f1, win_f1));
      _mm256_storeu_ps(output+N2+i  , sum0);
      _mm256_storeu_ps(output+N2+i+8, sum1);
   }
}

// this is about 20% faster than SSE2
float radaudio_avx2_save_samples(S16 *buffer, const float *data, int num)
{
   float8 scale = _mm256_set1_ps(32767.0f);
   float8 maxabs8 = _mm256_set1_ps(1.0f);
   float8 absmask = ps8(_mm256_set1_epi32(0x7fffffff));

   // scale the data by 'scale' and simultaneously find the max absolute value
   for (int i=0; i < num; i += 32) {
      // unroll to 16-at-a-time to write 32 bytes per iteration
      // unroll once more to reduce dependency chains in min/max calculation
      float8 v0 = _mm256_loadu_ps(data+i+ 0);
      float8 v1 = _mm256_loadu_ps(data+i+ 8);
      float8 v2 = _mm256_loadu_ps(data+i+16);
      float8 v3 = _mm256_loadu_ps(data+i+24);

      float8 v0abs = _mm256_and_ps(v0, absmask);
      float8 v1abs = _mm256_and_ps(v1, absmask);
      float8 v2abs = _mm256_and_ps(v2, absmask);
      float8 v3abs = _mm256_and_ps(v3, absmask);
      float8 maxa01 = _mm256_max_ps(v0abs, v1abs);
      float8 maxa23 = _mm256_max_ps(v2abs, v3abs);

      float8 scaled0 = _mm256_mul_ps(v0, scale);
      float8 scaled1 = _mm256_mul_ps(v1, scale);
      float8 scaled2 = _mm256_mul_ps(v2, scale);
      float8 scaled3 = _mm256_mul_ps(v3, scale);
      float8 maxa0123 = _mm256_max_ps(maxa01, maxa23);

      // we want to round() to integer:
      int32x8 int0   = _mm256_cvtps_epi32(scaled0);
      int32x8 int1   = _mm256_cvtps_epi32(scaled1);
      int32x8 int2   = _mm256_cvtps_epi32(scaled2);
      int32x8 int3   = _mm256_cvtps_epi32(scaled3);

      // these are AVX2 only:
      int16x16 pack0 = _mm256_packs_epi32(int0,int1);
      int16x16 pack1 = _mm256_packs_epi32(int2,int3);

      // the above produce the wrong order, because they pack in lanes, so fix that
      // go from <p0lo, p1lo, p0hi, p1hi> to <p0lo, p0hi, p1lo, p1hi>
      int16x16 repack0 = _mm256_permute4x64_epi64(pack0, MY_SHUFFLE(0,2,1,3));
      int16x16 repack1 = _mm256_permute4x64_epi64(pack1, MY_SHUFFLE(0,2,1,3));

      maxabs8 = _mm256_max_ps(maxabs8, maxa0123);
      _mm256_storeu_si256((int16x16 *) (buffer+i+ 0), repack0);
      _mm256_storeu_si256((int16x16 *) (buffer+i+16), repack1);
   }

   // if the largest value saturated, redo with a smaller scale

   float8 maxabs4 = _mm256_max_ps(maxabs8, _mm256_shuffle_ps_mine(maxabs8, MY_SHUFFLE(2,3,0,1)));
   float8 maxabs2 = _mm256_max_ps(maxabs4, _mm256_shuffle_ps_mine(maxabs4, MY_SHUFFLE(1,0,3,2)));
   // at this point, all lo entries are identical, and all hi entries are identical
   float8 repack2 = ps8(_mm256_permute4x64_epi64(pi8(maxabs2), MY_SHUFFLE(2,3,0,1)));
   float8 maxabs  = _mm256_max_ps(maxabs2, repack2);
   float max1 = _mm_cvtss_f32(_mm256_castps256_ps128(maxabs));

   if (max1 > 1.0f) {
      // rarely taken

      #if 0
      // this doesn't work, but all values in maxabs should be identical!
      scale = _mm256_div_ps(scale, maxabs);

      // this never trips, so apparently they are identical?!? mystery
      float temp[8];
      _mm256_storeu_ps(temp, maxabs);
      for (int i=0; i < 8; ++i) if (temp[i] != max1) __debugbreak();
      #else
      scale = _mm256_div_ps(scale, _mm256_set1_ps(max1)); // inefficient but who cares
      #endif

      for (int i=0; i < num; i += 16) {
         float8 v0      = _mm256_loadu_ps(data+i+0);
         float8 v1      = _mm256_loadu_ps(data+i+8);
         float8 scaled0 = _mm256_mul_ps(v0, scale);
         float8 scaled1 = _mm256_mul_ps(v1, scale);
         int32x8 int0   = _mm256_cvtps_epi32(scaled0);
         int32x8 int1   = _mm256_cvtps_epi32(scaled1);
         int16x16 pack0 = _mm256_packs_epi32(int0,int1);
         int16x16 repack0 = _mm256_permute4x64_epi64(pack0, MY_SHUFFLE(0,2,1,3));
         _mm256_storeu_si256((int16x16 *) (buffer+i+0), repack0);
      }
   }
   return 1.0f / _mm_cvtss_f32(_mm256_castps256_ps128(scale));
}

#endif // #if DO_BUILD_AVX2
