// Copyright Epic Games Tools, LLC. All Rights Reserved.
#ifndef RADAUDIO_AVX2_DECODE_H
#define RADAUDIO_AVX2_DECODE_H

#include "radaudio_common.h"
#include "radaudio_sse.h"

#ifdef __cplusplus
extern "C" {
#endif

#ifdef RADAUDIO_WRAP
#define WRAPPED_NAME(name) RR_STRING_JOIN(RADAUDIO_WRAP, name##_)

#define radaudio_avx2_save_samples          					WRAPPED_NAME(radaudio_avx2_save_samples)
#define radaudio_avx2_compute_windowed_sum_multiple16           WRAPPED_NAME(radaudio_avx2_compute_windowed_sum_multiple16)
#endif


extern float  radaudio_avx2_save_samples(S16 *buffer, const float *data, int num);
extern void   radaudio_avx2_compute_windowed_sum_multiple16(float *output, int n, const float *fwd_data, const S16 *rev_data, float rev_scale, const float *window);

#ifdef __cplusplus
}
#endif

#endif
