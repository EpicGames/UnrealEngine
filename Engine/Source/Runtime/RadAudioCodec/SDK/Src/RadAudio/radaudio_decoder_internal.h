// Copyright Epic Games Tools, LLC. All Rights Reserved.
#ifndef INCLUDE_RADAUDIO_DECODER_INTERNAL_H
#define INCLUDE_RADAUDIO_DECODER_INTERNAL_H

#include "radaudio_common.h"
#include "radaudio_decoder.h"

// internal profiling info
typedef struct
{
   const char *name;
   double time;
} radaudio_profile_value;


// also call GetProfileData to enable profiling, profile=NULL, num_prof=1 is normal profiling, num_prof=2 is full profile
extern int  RadAudioDecoderGetProfileData(RadAudioDecoder *hradaud, radaudio_profile_value *profile, int num_profile);

#ifdef RADAUDIO_DEVELOPMENT
extern void RadAudioDecoderForceIntelCPU (RadAudioDecoder *hradaud, rrbool has_sse2, rrbool has_ssse3, rrbool has_sse4_1, rrbool has_popcnt, rrbool has_avx2);
#endif

#endif//INCLUDE_RADAUDIO_DECODER_INTERNAL_H
