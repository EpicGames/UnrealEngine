// Copyright Epic Games Tools, LLC. All Rights Reserved.
#include "radaudio_common.h"
#include "radaudio_decoder_neon.h"
#include "radaudio_decoder_simd_tables.h"
#include <string.h>
#include <math.h>

#ifdef RADAUDIO_WRAP
#define WRAPPED_NAME(name) RR_STRING_JOIN(RADAUDIO_WRAP, name##_)
#define radaudio_dummy_decode_neon WRAPPED_NAME(radaudio_dummy_decode_neon)
#endif

int radaudio_dummy_decode_neon(void) { return 0; } // avoid "empty translation unit" warning

#ifdef DO_BUILD_NEON

#include <arm_neon.h>

rrbool radaudio_neon_expand_coefficients_excess_read15(S8 *nonzero_coefficients, int num_nonzero, S8 *big_coeff, S8 *big_limit)
{
   static const RAD_ALIGN(U8, mask_const[8], 8) = { 1,2,4,8, 16,32,64,128 };
   uint8x8_t mask = vld1_u8(mask_const);

   // nonzero_coefficients[] has been padded with dummy data to a multiple of 16, but we only work 8 at a time here
   int padded_size = (num_nonzero + 7) & ~7;
   for (int i=0; i < padded_size; i += 8) {
      // load the coefficients and check if any are 0
      int8x8_t coeff = vld1_s8(&nonzero_coefficients[i]);
      uint8x8_t is_zero = vceqz_u8(vreinterpret_u8_s8(coeff));

      // mask of where there are zero values (=big coeffs)
      U32 result = vaddv_u8(vand_u8(is_zero, mask));
      
      // regular coeffs have 8 subtracted from them
      coeff = vsub_s8(coeff, vdup_n_s8(8));

      if (result != 0) {
         // distribute up to 8 big coefficients according to result
         int8x8_t next8big = vld1_s8(big_coeff);
         int8x8_t shuffle = vld1_s8(s_shuffle_table[result]);
         int8x8_t big_distributed = vtbl1_s8(next8big, shuffle);

         // splice them in
         coeff = vbsl_s8(is_zero, big_distributed, coeff);

         // advance coeff ptr
         big_coeff += s_popcnt_table[result];
         if (big_coeff > big_limit)
            return 0;
      }

      vst1_s8(&nonzero_coefficients[i], coeff);
   }
   return 1;
}

void radaudio_neon_compute_band_energy_multiple4(F32 *band_energy, int num_bands, int band_exponent[], U16 fine_energy[], F32 band_scale_decode[])
{
   // num_bands may not be a multiple of 4, but all arrays are defined so we can run to a multiple of 4
   for (int j=0; j < num_bands; j += 4) {  // all arrays are safe to run extra
      uint16x4_t qe_raw = vld1_u16(&fine_energy[j]);

      // quantized fine energy
      uint32x4_t qe = vmovl_u16(qe_raw);

      // packed fine_energy (still in a pseudo-logarithmic encoding)
      float32x4_t pe = vmulq_f32(vcvtq_f32_u32(qe), vdupq_n_f32(1.0f / (1 << MAX_FINE_ENERGY_BITS)));

      // approximate fe = pow(2,pe) with a quadratic: fe = 0.34375f*pe*pe + 0.65625f*pe + 1;
      float32x4_t fe_1 = vaddq_f32(vmulq_f32(pe  , vdupq_n_f32(0.34375f)), vdupq_n_f32(0.65625f));
      float32x4_t fe   = vaddq_f32(vmulq_f32(pe  , fe_1                 ), vdupq_n_f32(1.0f    ));

      int32x4_t bandexp = vld1q_s32(&band_exponent[j]);

      // +16 to make it non-negative (C code uses shift, so doesn't handle negative, and the +16 is compensated in the other multiplier
      // which we're also going to use), then +127 to convert to an IEEE biased expnonent
      int32x4_t biased = vaddq_s32(bandexp, vdupq_n_s32(16+127));

      // shift to the right place
      int32x4_t exp_float = vshlq_n_s32(biased, 23); // shift past 23 bits of mantissa

      // zero it if bandexp is BAND_EXPONENT_NONE
      int32x4_t is_none = vceqq_s32(bandexp, vdupq_n_s32(BAND_EXPONENT_NONE));

      // if is_none is true in a lane, then mask that lane to 0, which is floating point 0.0
      float32x4_t ce = vreinterpretq_f32_s32(vbicq_s32(exp_float, is_none));

      // multiply the exponent, the fine energy, and the compensatory scaler
      float32x4_t band_energy_decoded = vmulq_f32(vmulq_f32(fe, ce), vld1q_f32(&band_scale_decode[j]));

      vst1q_f32(&band_energy[j], band_energy_decoded);
   }
}

static RADFORCEINLINE float32x4_t float32x4_reverse(float32x4_t v)
{
   float32x4_t pairs_swapped = vrev64q_f32(v);
   return vextq_f32(pairs_swapped, pairs_swapped, 2);
}

static RADFORCEINLINE float32x4_t float32x4_load_rev(const float *data)
{
   return float32x4_reverse(vld1q_f32(data));
}

static RADFORCEINLINE void load_8int16_scaled(float32x4_t *data_r0, float32x4_t *data_r1, const S16 *data, float32x4_t scale)
{
   int16x8_t data_ri = vld1q_s16(data);
   // unpack s16 to s32
   int32x4_t data_r0i = vmovl_s16(vget_low_s16(data_ri));
   int32x4_t data_r1i = vmovl_s16(vget_high_s16(data_ri));
   // convert to float and scale
   *data_r0 = vmulq_f32(scale, vcvtq_f32_s32(data_r0i));
   *data_r1 = vmulq_f32(scale, vcvtq_f32_s32(data_r1i));
}

void radaudio_neon_compute_subband_energy_skip12_excess_read7(F32 *subband_energy, const F32 *band_energy, int num_bands, int *num_subbands_for_band, U16 *quantized_subbands)
{
   static U16 subband_mask[16] = // load at 8-count to get mask for count active subbands
   {
      0xffff, 0xffff, 0xffff, 0xffff, 0xffff, 0xffff, 0xffff, 0xffff, 0, 0, 0, 0, 0, 0, 0, 0
   };

   int start = 12;
   int j = 12;

   // we're going to add squares of quantized subband values.

   // this loop processes 4 bands with 2 subbands each
   for (; j+3 < num_bands && num_subbands_for_band[j+3] == 2; j += 4) {
      rrAssert(num_subbands_for_band[j] == 2 && num_subbands_for_band[j+1] == 2 && num_subbands_for_band[j+2] == 2);

      // load the eight subbands
      uint16x8_t qsub = vld1q_u16(&quantized_subbands[start]);

      uint16x4_t qsub_lo = vget_low_u16(qsub);
      uint16x4_t qsub_hi = vget_high_u16(qsub);

      // square and add together
      uint32x4_t squares_lo = vmull_u16(qsub_lo, qsub_lo);
      uint32x4_t squares_hi = vmull_u16(qsub_hi, qsub_hi);
      uint32x4_t total = vpaddq_u32(squares_lo, squares_hi);

      // compute the normalizing factor, 1/sqrt(sum_of_squares) and multiply by the band energy now so we don't have to do it later
      float32x4_t band_scale = vdivq_f32(vld1q_f32(&band_energy[j]), vsqrtq_f32(vcvtq_f32_u32(total)));
      float32x4x2_t band_scale_x2 = vzipq_f32(band_scale, band_scale);

      float32x4_t subbandq_lo = vcvtq_f32_u32(vmovl_u16(qsub_lo));
      float32x4_t subbandq_hi = vcvtq_f32_u32(vmovl_u16(qsub_hi));

      float32x4x2_t values =
      {
         vmulq_f32(subbandq_lo, band_scale_x2.val[0]),
         vmulq_f32(subbandq_hi, band_scale_x2.val[1]),
      };
      vst1q_f32_x2(&subband_energy[start], values);
      start += 8;
   }

   // this loop processes 2 bands with 2 subbands each
   for (; j+1 < num_bands && num_subbands_for_band[j+1] == 2; j += 2) {
      rrAssert(num_subbands_for_band[j] == 2);

      // load the four subbands
      uint16x4_t qsub = vld1_u16(&quantized_subbands[start]);

      // square the subbands and add pairs
      uint32x4_t squared = vmull_u16(qsub, qsub);
      uint32x2_t total = vget_low_u32(vpaddq_u32(squared, squared));

      // compute the normalizing factor, 1/sqrt(sum_of_squares) and multiply by the band energy now so we don't have to do it later
      float32x2_t band_scale = vdiv_f32(vld1_f32(&band_energy[j]), vsqrt_f32(vcvt_f32_u32(total)));
      float32x2x2_t replicated = vzip_f32(band_scale, band_scale);
      float32x4_t band_scalev = vcombine_f32(replicated.val[0], replicated.val[1]);

      float32x4_t subbandq_lo = vcvtq_f32_u32(vmovl_u16(qsub));
      float32x4_t subband_lo = vmulq_f32(subbandq_lo, band_scalev);

      // store all 4, as we'll overwrite that with next band
      vst1q_f32(&subband_energy[start], subband_lo);

      start += 4;
   }

   // this loop process bands with 2, 3, or 4 subbands, one band at a time
   for (; j < num_bands && num_subbands_for_band[j] <= 4; ++j) {
      int num = num_subbands_for_band[j];

      uint16x4_t mask = vld1_u16(&subband_mask[8 - num]);

      // load four subbands, and 0 the unused ones
      uint16x4_t qsub = vand_u16(mask, vld1_u16(&quantized_subbands[start]));

      // square the subbands and reduce
      uint32x4_t squared = vmull_u16(qsub, qsub);
      uint32_t total = vaddvq_u32(squared);

      // compute the normalizing factor, 1/sqrt(sum_of_squares) and multiply by the band energy now so we don't have to do it later
      float band_scale = band_energy[j] / sqrtf((float) total);
      float32x4_t band_scalev = vdupq_n_f32(band_scale);

      float32x4_t subbandq_lo = vcvtq_f32_u32(vmovl_u16(qsub));
      float32x4_t subband_lo = vmulq_f32(subbandq_lo, band_scalev);

      // store all 4, as we'll overwrite that with next band
      vst1q_f32(&subband_energy[start], subband_lo);

      start += num;
   }

   // this loop processes bands with up to 8 subbands, one band at a time
   for (; j < num_bands && num_subbands_for_band[j] <= 8; ++j) {
      int num = num_subbands_for_band[j];

      uint16x8_t mask = vld1q_u16(&subband_mask[8 - num]);

      // load eight subbands, and 0 the unused ones
      uint16x8_t qsub = vandq_u16(mask, vld1q_u16(&quantized_subbands[start]));

      // square the subbands and add pairs together
      uint16x4_t qsub_lo = vget_low_u16(qsub);
      uint16x4_t qsub_hi = vget_high_u16(qsub);

      uint32x4_t sum0 = vmull_u16(qsub_lo, qsub_lo);
      uint32x4_t sum1 = vmlal_u16(sum0, qsub_hi, qsub_hi);
      
      // finish the reduction
      uint32_t total = vaddvq_u32(sum1);

      // compute the normalizing factor, 1/sqrt(sum_of_squares) and multiply by the band energy now so we don't have to do it later
      float band_scale = band_energy[j] / sqrtf((float) total);
      float32x4_t band_scalev = vdupq_n_f32(band_scale);

      // scale the coeffs
      float32x4_t subbandq_lo = vcvtq_f32_u32(vmovl_u16(qsub_lo));
      float32x4_t subbandq_hi = vcvtq_f32_u32(vmovl_u16(qsub_hi));

      float32x4x2_t values =
      {
         vmulq_f32(subbandq_lo, band_scalev),
         vmulq_f32(subbandq_hi, band_scalev),
      };
      vst1q_f32_x2(&subband_energy[start], values);

      start += num;
   }

   // this loop processes bands with 9..16 subbands
   for (; j < num_bands && num_subbands_for_band[j] >= 9 && num_subbands_for_band[j] <= 16; ++j) {
      int num = num_subbands_for_band[j];

      uint16x8_t mask = vld1q_u16(&subband_mask[16 - num]);

      // load sixteen subbands, and 0 the unused ones
      uint16x8_t qsub0 =                 vld1q_u16(&quantized_subbands[start+0]);
      uint16x8_t qsub1 = vandq_u16(mask, vld1q_u16(&quantized_subbands[start+8]));

      // square the subbands and add pairs together
      uint16x4_t qsub0_lo = vget_low_u16(qsub0);
      uint16x4_t qsub0_hi = vget_high_u16(qsub0);
      uint16x4_t qsub1_lo = vget_low_u16(qsub1);
      uint16x4_t qsub1_hi = vget_high_u16(qsub1);

      uint32x4_t sum0 = vmull_u16(qsub0_lo, qsub0_lo);
      uint32x4_t sum1 = vmlal_u16(sum0, qsub0_hi, qsub0_hi);
      uint32x4_t sum2 = vmlal_u16(sum1, qsub1_lo, qsub1_lo);
      uint32x4_t sum3 = vmlal_u16(sum2, qsub1_hi, qsub1_hi);
      
      // finish the reduction
      uint32_t total = vaddvq_u32(sum3);

      // compute the normalizing factor, 1/sqrt(sum_of_squares) and multiply by the band energy now so we don't have to do it later
      float band_scale = band_energy[j] / sqrtf((float) total);
      float32x4_t band_scalev = vdupq_n_f32(band_scale);

      // scale the coeffs
      float32x4_t subbandq0_lo = vcvtq_f32_u32(vmovl_u16(qsub0_lo));
      float32x4_t subbandq0_hi = vcvtq_f32_u32(vmovl_u16(qsub0_hi));
      float32x4_t subbandq1_lo = vcvtq_f32_u32(vmovl_u16(qsub1_lo));
      float32x4_t subbandq1_hi = vcvtq_f32_u32(vmovl_u16(qsub1_hi));

      float32x4x4_t values =
      {
         vmulq_f32(subbandq0_lo, band_scalev),
         vmulq_f32(subbandq0_hi, band_scalev),
         vmulq_f32(subbandq1_lo, band_scalev),
         vmulq_f32(subbandq1_hi, band_scalev),
      };
      vst1q_f32_x4(&subband_energy[start], values);

      start += num;
   }

   // bands with more than 16 subbands are processed with scalar logic, also a final smaller band for 24khz
   for (; j < num_bands; ++j) {
      int sum=0;
      int num = num_subbands_for_band[j];
      for (int i=0; i < num; ++i)
         sum += (quantized_subbands[start+i] * quantized_subbands[start+i]);

      F32 scale = band_energy[j] / sqrtf((F32) sum);
      for (int i=0; i < num; ++i) {
         subband_energy[start+i] = scale * quantized_subbands[start+i];
      }
      start += num;
   }
}

// rev_data is read forwards, but uses a reversed window
void radaudio_neon_compute_windowed_sum_multiple8(float *output, int n, const float *fwd_data, const S16 *rev_data, float rev_scale, const float *window)
{
   float32x4_t scale = vdupq_n_f32(rev_scale);
   SINTa N2 = n >> 1;

   // front half
   for (SINTa i = 0; i < N2; i += 8) {
      // read rev_data and scale
      float32x4_t data_r0, data_r1;
      load_8int16_scaled(&data_r0, &data_r1, rev_data + i, scale);

      // fwd_data is read reversed (confusing naming, I know)
      float32x4_t data_f0 = float32x4_load_rev(fwd_data+N2 - i - 4);
      float32x4_t data_f1 = float32x4_load_rev(fwd_data+N2 - i - 8);

      // windows
      float32x4_t win_r0  = float32x4_load_rev(window+n - i - 4);
      float32x4_t win_r1  = float32x4_load_rev(window+n - i - 8);
      float32x4_t win_f0  = vld1q_f32(window + i);
      float32x4_t win_f1  = vld1q_f32(window + i + 4);

      // weighted sum
      float32x4_t sum0 = vsubq_f32(vmulq_f32(data_r0, win_r0), vmulq_f32(data_f0, win_f0));
      float32x4_t sum1 = vsubq_f32(vmulq_f32(data_r1, win_r1), vmulq_f32(data_f1, win_f1));
      vst1q_f32(output+i  , sum0);
      vst1q_f32(output+i+4, sum1);
   }

   // back half
   for (SINTa i = 0; i < N2; i += 8) {
      // read rev_data and scale
      // (this one is actually reversed, hence r1 and r0 being swapped)
      float32x4_t data_r0, data_r1;
      load_8int16_scaled(&data_r1, &data_r0, rev_data+N2 - i - 8, scale);

      // fwd_data
      float32x4_t data_f0 = vld1q_f32(fwd_data + i);
      float32x4_t data_f1 = vld1q_f32(fwd_data + i + 4);

      // windows
      float32x4_t win_r0  = vld1q_f32(window+N2 - i - 4);
      float32x4_t win_r1  = vld1q_f32(window+N2 - i - 8);
      float32x4_t win_f0  = vld1q_f32(window+N2 + i);
      float32x4_t win_f1  = vld1q_f32(window+N2 + i + 4);

      // weighted sum
      float32x4_t sum0 = vaddq_f32(float32x4_reverse(vmulq_f32(data_r0, win_r0)), vmulq_f32(data_f0, win_f0));
      float32x4_t sum1 = vaddq_f32(float32x4_reverse(vmulq_f32(data_r1, win_r1)), vmulq_f32(data_f1, win_f1));
      vst1q_f32(output+N2+i  , sum0);
      vst1q_f32(output+N2+i+4, sum1);
   }
}

void radaudio_neon_unpack_nibbles_read_sentinel16_write_multiple32(S8 *unpacked, U8 *packed, int num_packed, U64 default_nibbles)
{
   const int8x16_t nibble_mask = vdupq_n_s8(0x0f);
   int padded_size = (num_packed + 15) & ~15;
   // put default values in the padding
   RR_PUT64_NATIVE(&packed[num_packed+0], default_nibbles);
   RR_PUT64_NATIVE(&packed[num_packed+8], default_nibbles);

   for (int i=0; i < padded_size; i += 16) {
      // process 16 bytes at a time
      uint16x8_t value = vreinterpretq_u16_u8(vld1q_u8(&packed[i]));

      int8x16_t lo_nibble = vandq_s8(vreinterpretq_s8_u16(value), nibble_mask);
      int8x16_t hi_nibble = vandq_s8(vreinterpretq_s8_u16(vshrq_n_s16(value, 4)), nibble_mask);

      vst1q_s8_x2(&unpacked[i*2], vzipq_s8(lo_nibble, hi_nibble));
   }
}

// num_bytes is rounded up to the nearest multiple of 8
int radaudio_neon_count_set_bits_read_multiple8_sentinel8(U8 *data, int num_bytes)
{
   int padded_size = (num_bytes+7) & ~7;
   if (padded_size != num_bytes)
      RR_PUT64_NATIVE(&data[num_bytes], 0);

   uint16x8_t num = vdupq_n_u16(0);
   for (int i=0; i < padded_size; i += 8)
   {
      num = vaddw_u8(num, vcnt_u8(vld1_u8(&data[i])));
   }
   return (int)vaddvq_u16(num);
}

float radaudio_neon_save_samples(S16 *buffer, const float *data, int num)
{
   float scale = 32767.0f;
   float32x4_t maxabs4 = vdupq_n_f32(1.0f);

   // scale the data by 'scale' and simultaneously find the max absolute value
   for (int i=0; i < num; i += 16) {
      float32x4x4_t v = vld1q_f32_x4(data+i);
      float32x4_t v0abs = vabsq_f32(v.val[0]);
      float32x4_t v1abs = vabsq_f32(v.val[1]);
      float32x4_t v2abs = vabsq_f32(v.val[2]);
      float32x4_t v3abs = vabsq_f32(v.val[3]);
      float32x4_t maxa01 = vmaxq_f32(v0abs, v1abs);
      float32x4_t maxa23 = vmaxq_f32(v2abs, v3abs);
      float32x4_t maxa0123 = vmaxq_f32(maxa01, maxa23);
      float32x4_t scaled0 = vmulq_n_f32(v.val[0], scale);
      float32x4_t scaled1 = vmulq_n_f32(v.val[1], scale);
      float32x4_t scaled2 = vmulq_n_f32(v.val[2], scale);
      float32x4_t scaled3 = vmulq_n_f32(v.val[3], scale);
      // we want to round() to integer:
      int32x4_t int0 = vcvtnq_s32_f32(scaled0);
      int32x4_t int1 = vcvtnq_s32_f32(scaled1);
      int32x4_t int2 = vcvtnq_s32_f32(scaled2);
      int32x4_t int3 = vcvtnq_s32_f32(scaled3);
      int16x8x2_t pack =
      {
         vcombine_s16(vmovn_s32(int0), vmovn_s32(int1)),
         vcombine_s16(vmovn_s32(int2), vmovn_s32(int3)),
      };
      vst1q_s16_x2(buffer+i, pack);
      maxabs4 = vmaxq_f32(maxabs4, maxa0123);
   }

   float maxabs = vmaxvq_f32(maxabs4);
   if (maxabs > 1.0f)
   {
      // rarely taken
      scale = scale / maxabs;
      for (int i=0; i < num; i += 8) {
         float32x4x2_t v = vld1q_f32_x2(data+i);
         float32x4_t scaled0 = vmulq_n_f32(v.val[0], scale);
         float32x4_t scaled1 = vmulq_n_f32(v.val[1], scale);
         int32x4_t int0 = vcvtnq_s32_f32(scaled0);
         int32x4_t int1 = vcvtnq_s32_f32(scaled1);
         int16x8_t pack = vcombine_s16(vmovn_s32(int0), vmovn_s32(int1));
         vst1q_s16(buffer+i, pack);
      }
   }
   return 1.0f / scale;
}

// sum of squares across 8 coeffs in [-127,127]
static RADFORCEINLINE int subband8_sum_squares(int8x8_t coeff_8)
{
   int16x8_t squares = vmull_s8(coeff_8, coeff_8); // each <=127*127
   int full_sum = vaddlvq_s16(squares);

   return full_sum;
}

// sum of squares across 16 coeffs in [-127,127]
static RADFORCEINLINE int subband16_sum_squares(int8x16_t coeff_8)
{
   int8x8_t coeff8_lo = vget_low_s8(coeff_8);
   int8x8_t coeff8_hi = vget_high_s8(coeff_8);

   int16x8_t squares_lo = vmull_s8(coeff8_lo, coeff8_lo); // each <=127*127<16384
   int16x8_t pairs = vmlal_s8(squares_lo, coeff8_hi, coeff8_hi); // each <32768
   int full_sum = vaddlvq_s16(pairs);

   return full_sum;
}

// compute coefficient normalization factor for subband from sum-of-squares in float.
static RADFORCEINLINE float subband_normalization_factor(float* energy_ptr, float sum_of_squares)
{
   // the coefficient normalization factor is the reciprocal of the square-root of the squared sum.
   // in other words, we are going to compute:
   //          coefficient * (1/sqrt(sum))
   //
   // but then after normalizing that, we then want to multiply it by the subband energy:
   //          coefficient * (1/sqrt(sum)) * energy
   //
   // so we simplify this to:
   //          coefficient * (energy/sqrt(sum))
   //
   // use scalar math here since everything is the same in all 4 lanes and this helps on smaller machines.
   return *energy_ptr / sqrtf(sum_of_squares);
}

static RADFORCEINLINE void subband8_output(F32 *coeffs, int8x8_t coeff_8, float scale)
{
   float32x4_t band_scale = vdupq_n_f32(scale);

   int16x8_t coeff_16 = vmovl_s8(coeff_8);
   int32x4_t unpacked_set0_i = vmovl_s16(vget_low_s16(coeff_16));
   int32x4_t unpacked_set1_i = vmovl_s16(vget_high_s16(coeff_16));
   float32x4_t unpacked_set0 = vcvtq_f32_s32(unpacked_set0_i);
   float32x4_t unpacked_set1 = vcvtq_f32_s32(unpacked_set1_i);
   float32x4x2_t unquantized =
   {
      vmulq_f32(unpacked_set0, band_scale),
      vmulq_f32(unpacked_set1, band_scale),
   };
   vst1q_f32_x2(coeffs, unquantized);
}

static RADFORCEINLINE void subband16_output(F32 *coeffs, int8x16_t coeff_8, float scale)
{
   subband8_output(coeffs+0, vget_low_s8(coeff_8), scale);
   subband8_output(coeffs+8, vget_high_s8(coeff_8), scale);
}

void radaudio_neon_dequantize_long_block_replace_0_with_random_8x8_Nx16(F32 *coeffs, S8 *quantized_coeff, F32 *subband_energy, int num_subbands, U32 rand_state_array[4])
{
   int j,cb;

   static const int8_t randtab_source[16] = { -1,1, -2,2, -3,3, -4,4, -5,5, -6,6, -7,7, -8,8 };
   const uint8x16_t randtab = vreinterpretq_u8_s8(vld1q_s8(randtab_source));

   uint32x4_t rand_state = vld1q_u32(rand_state_array);

   // setup constants that update the first two lcg states, but not the second two, for consistency with scalar impl
   const uint32x4_t half_lcg_mul = vcombine_u32(vdup_n_u32(LCG_MUL), vdup_n_u32(1));
   const uint32x4_t half_lcg_add = vcombine_u32(vdup_n_u32(LCG_ADD), vdup_n_u32(0));
   const uint16x8_t nibble_mask  = vdupq_n_u16(0x0f0f);

   // first 8 subbands are always 8 coefficients long
   j = 0;
   cb = 0;
   for (; j < 8; ++j) {

      // load 8 signed-byte coefficients
      int8x8_t coeff_8 = vld1_s8(&quantized_coeff[cb]);

      // randomize all-zero coefficient sets
      if (RR_GET64_NATIVE(&quantized_coeff[cb+0]) == 0) {
         // get 4 bits of random state for each of 8 lanes (other 8 are "don't care")
         uint16x8_t rbits = vandq_u16(vshrq_n_u16(vreinterpretq_u16_u32(rand_state), 4), nibble_mask);

         // load the coefficients from the random table
         coeff_8 = vget_low_u8(vqtbl1q_u8(randtab, vreinterpretq_u8_u16(rbits)));

         // update the first two LCG streams
         rand_state = vmlaq_u32(half_lcg_add, rand_state, half_lcg_mul);
      }

      //
      // compute the squared sum of all 8 coefficients in the subband so they can be normalized to 1
      //

      int coeff_sum = subband8_sum_squares(coeff_8);
      float subband_scale = subband_normalization_factor(&subband_energy[j], (float)coeff_sum);

      subband8_output(coeffs+cb, coeff_8, subband_scale);

      cb += 8;
   }

   // all remaining subbands are 16 coefficients long
   const uint32x4_t lcg_mul = vdupq_n_u32(LCG_MUL);
   const uint32x4_t lcg_add = vdupq_n_u32(LCG_ADD);

   for (; j < num_subbands; ++j) {

      // load 16 signed-byte coefficients
      int8x16_t coeff_8 = vld1q_s8(&quantized_coeff[cb]);

      //
      // if coefficients are all 0, replace with random values from -8..8 (but not 0,
      // because if we got unlucky and they ALL were 0, we'd divide by 0 later)
      //

      if ((RR_GET64_NATIVE(&quantized_coeff[cb+0]) | RR_GET64_NATIVE(&quantized_coeff[cb+8])) == 0)
      {
         // get the top 4 bits from each byte of random state
         uint16x8_t rbits = vandq_u16(vshrq_n_u16(vreinterpretq_u16_u32(rand_state), 4), nibble_mask);

         // lookup in random_table[].
         int8x16_t values = vqtbl1q_s8(randtab, vreinterpretq_u8_u16(rbits));

         // those are the coefficients now
         coeff_8 = values;

         // update the LCG streams
         rand_state = vmlaq_u32(lcg_add, rand_state, lcg_mul);
      }

      //
      // compute the squared sum of all 16 coefficients in the subband so they can be normalized to 1
      //
      int sum_squares = subband16_sum_squares(coeff_8);
      float subband_scale = subband_normalization_factor(&subband_energy[j], (float)sum_squares);

      subband16_output(coeffs+cb, coeff_8, subband_scale);

      cb += 16;
   }

   memset(&coeffs[cb], 0, 4*(RADAUDIO_LONG_BLOCK_LEN-cb));
}

void radaudio_neon_dequantize_short_block(float *coeffs, S8 *quantized_coeff, float *band_energy, int num_bands, int *num_coeffs_for_band)
{
   int cb=0;
   int band=0;

   // in the following code, each chunk is independent, so big out-of-order CPUs
   // should crunch through it, so we just want to minimize uops, so minimize instructions

   ////////////////////////////////////////////////////////////////////////////////
   //
   // Process bands 0..3
   //
   // first 4 bands are 1 coefficient long, and coefficient is always 1 or -1
   //

   // for (j=0; j < 4; ++j) {
   //   coeffs[j] = (F32) quantized_coeff[j] * band_energy[j];
   // }

   {
      int16x8_t coeff8 = vmovl_s8(vld1_s8(&quantized_coeff[cb]));
      int32x4_t coeff16 = vmovl_s16(vget_low_s16(coeff8));
      float32x4_t coeff4 = vcvtq_f32_s32(coeff16);
      float32x4_t unquantized_band4 = vmulq_f32(coeff4, vld1q_f32(&band_energy[band]));
      vst1q_f32(&coeffs[cb], unquantized_band4);
      cb += 4;
      band += 4;
   }

   ////////////////////////////////////////////////////////////////////////////////
   //
   // Process bands 4..7
   //
   // next 4 bands are 2 coefficients each
   //

   // for (j=0; j < 4; ++j) {
   //   float x = (F32) quantized_coeff[4+j*2+0];
   //   float y = (F32) quantized_coeff[4+j*2+1];
   //   float scale = band_energy[4+j] / (F32) sqrt(x*x+y*y+1.e-20f);
   //   coeffs[4+j*2+0] = x*scale;
   //   coeffs[4+j*2+1] = y*scale;
   // }

   const float epsilon = 1e-20f;
   const float32x4_t epsilon_vec = vdupq_n_f32(epsilon);

   {
      // load 8 coefficients
      int8x8_t coeff8_8 = vld1_s8(&quantized_coeff[cb]);

      // square to 16-bit
      int16x8_t squared = vmull_s8(coeff8_8, coeff8_8);

      // add pairs to 32-bit
      int32x4_t pairs = vpaddlq_s16(squared);

      // sum as 32-bit float, add epsilon
      float32x4_t sum = vaddq_f32(vcvtq_f32_s32(pairs), epsilon_vec);

      float32x4_t band_scale = vdivq_f32(vld1q_f32(&band_energy[band]), vsqrtq_f32(sum));
      float32x4x2_t scale = vzipq_f32(band_scale, band_scale);

      int16x8_t coeff = vmovl_s8(coeff8_8);
      float32x4_t quantized_lo = vcvtq_f32_s32(vmovl_s16(vget_low_s16(coeff)));
      float32x4_t quantized_hi = vcvtq_f32_s32(vmovl_s16(vget_high_s16(coeff)));

      float32x4x2_t unquantized =
      {
         vmulq_f32(quantized_lo, scale.val[0]),
         vmulq_f32(quantized_hi, scale.val[1]),
      };
      vst1q_f32_x2(&coeffs[cb], unquantized);

      cb += 8;
      band += 4;
   }

   ////////////////////////////////////////////////////////////////////////////////
   //
   // Process bands 8..12
   //
   // next 5 bands are 4 coefficients each
   //

   // for (j=8; j < 13; ++j) {
   //   float sum=1.e-20f, scale;
   //   for (i=0; i < 4; ++i) {
   //      float n = (F32) quantized_coeff[cb+i];
   //      sum += n*n;
   //   }
   //   scale = band_energy[j] / (F32) sqrt(sum);
   //   for (i=0; i < 4; ++i)
   //      coeffs[cb+i] = (F32) quantized_coeff[cb+i] * scale;
   //   cb += 4;
   // }

   {
      // load 16 unique coefficients
      int8x16_t coeff16 = vld1q_s8(&quantized_coeff[cb]);

      int8x8_t coeff8_lo = vget_low_s8(coeff16);
      int8x8_t coeff8_hi = vget_high_s8(coeff16);

      // square to 16-bit
      int16x8_t squared_lo = vmull_s8(coeff8_lo, coeff8_lo); // each <=127*127<16384
      int16x8_t squared_hi = vmull_s8(coeff8_hi, coeff8_hi); // each <=127*127<16384

      // add pairs together, giving us lanes <32768
      int16x8_t pairs = vpaddq_s16(squared_lo, squared_hi);

      // final sum goes to 32-bit
      int32x4_t quads = vpaddlq_s16(pairs);

      // sum as 32-bit float, add epsilon
      float32x4_t sum = vaddq_f32(vcvtq_f32_s32(quads), epsilon_vec);

      float32x4_t band_scale = vdivq_f32(vld1q_f32(&band_energy[band]), vsqrtq_f32(sum));

      int16x8_t coeff_lo = vmovl_s8(coeff8_lo);
      int16x8_t coeff_hi = vmovl_s8(coeff8_hi);

      float32x4_t coeff4_0 = vcvtq_f32_s32(vmovl_s16(vget_low_s16(coeff_lo)));
      float32x4_t coeff4_1 = vcvtq_f32_s32(vmovl_s16(vget_high_s16(coeff_lo)));
      float32x4_t coeff4_2 = vcvtq_f32_s32(vmovl_s16(vget_low_s16(coeff_hi)));
      float32x4_t coeff4_3 = vcvtq_f32_s32(vmovl_s16(vget_high_s16(coeff_hi)));

      float32x4x4_t unquantized =
      {
         vmulq_laneq_f32(coeff4_0, band_scale, 0),
         vmulq_laneq_f32(coeff4_1, band_scale, 1),
         vmulq_laneq_f32(coeff4_2, band_scale, 2),
         vmulq_laneq_f32(coeff4_3, band_scale, 3),
      };
      vst1q_f32_x4(&coeffs[cb], unquantized);

      // at most one more band that is 4 long -- currently all of them are, but low-sample-rate probably shouldn't
      if (num_coeffs_for_band[band] == 4) {
         cb += 16;
         band += 4;
         F32 inner_sum = 0;
         for (int k=0; k < 4; ++k) {
            F32 n = (F32) quantized_coeff[cb+k];
            inner_sum += n*n;
         }
         F32 scale = band_energy[band] / sqrtf(inner_sum);
         coeffs[cb+0] = quantized_coeff[cb+0] * scale;
         coeffs[cb+1] = quantized_coeff[cb+1] * scale;
         coeffs[cb+2] = quantized_coeff[cb+2] * scale;
         coeffs[cb+3] = quantized_coeff[cb+3] * scale;

         cb += 4;
         band += 1;
      }
      rrAssert(num_coeffs_for_band[band] > 4);
   }

   // now we have either [16,16,16,32]
   //                 or [16,16,32,32] for lower sample rates, or maybe even
   //                    [16,16,16,16,32]

   while (num_coeffs_for_band[band] == 16) {
      //
      // compute the squared sum of all 16 coefficients in the band so they can be normalized to 1
      //

      // load 16 signed-byte coefficients
      int8x16_t coeff8 = vld1q_s8(&quantized_coeff[cb]);

      int full_sum = subband16_sum_squares(coeff8);

      // sum as 32-bit float, add epsilon
      float full_sum_float = (float)full_sum + epsilon;
      float band_scale = subband_normalization_factor(&band_energy[band], full_sum_float);

      subband16_output(coeffs+cb, coeff8, band_scale);

      cb += 16;
      band += 1;
   }

   // @OPTIMIZE - but these are probably rare
   for (; band < num_bands; ++band) {
      int count = num_coeffs_for_band[band];
      F32 sum=epsilon,scale;
      for (int i=0; i < count; i += 4) {
         for (int k=0; k < 4; ++k) {
            F32 n = (F32) quantized_coeff[cb+i+k];
            sum += n*n;
         }
      }
      scale = band_energy[band] / sqrtf(sum);
      for (int i=0; i < count; i += 4) {
         for (int k=0; k < 4; ++k) {
            coeffs[cb+i+k] = quantized_coeff[cb+i+k] * scale;
         }
      }
      cb += count;
   }

   rrAssert(band == num_bands);

   memset(&coeffs[cb], 0, 4*(RADAUDIO_SHORT_BLOCK_LEN-cb));
}

void radaudio_neon_distribute_bitflag_coefficients_multiple16(S8 *quantized_coeff, int num_coeff, U8 *nonzero_flagbits, S8 *nonzero_coeffs, int *pcur_nonzero_coeffs)
{
   SINTa cur_nonzero_coeffs = *pcur_nonzero_coeffs;

   for (int i=0; i < num_coeff; i += 8) {
      // load 8 bitflags. 0 = set coefficient to 0, 1 = decode from array
      U8 flags = *nonzero_flagbits++;

      // load next batch of nonzero coefficients and shuffles from a table
      SINTa advance = s_popcnt_table[flags];
      int8x8_t shuffles = vld1_s8(s_shuffle_table[flags]);
      int8x8_t nzcoeffs = vld1_s8(&nonzero_coeffs[cur_nonzero_coeffs]);

      // distribute 8 coefficients
      int8x8_t distributed_coeffs = vtbl1_s8(nzcoeffs, shuffles);

      // write them out
      vst1_s8(&quantized_coeff[i], distributed_coeffs);
      cur_nonzero_coeffs += advance;
   }
   *pcur_nonzero_coeffs = (int)cur_nonzero_coeffs;
}

int radaudio_neon_count_bytes_below_value_sentinel16(U8 *data, int num_bytes, U8 threshold)
{
   int num=0;
   int padded_size = (num_bytes+15) & ~15;
   RR_PUT64_NATIVE(&data[num_bytes+0], 0xffffffffffffffffull);
   RR_PUT64_NATIVE(&data[num_bytes+8], 0xffffffffffffffffull);

   uint8x16_t threshold_vec = vdupq_n_u8(threshold);
   int max_in_group = 255 * 16;
   for (int base=0; base < padded_size; base += max_in_group) {
      int group_end = RR_MIN(base + max_in_group, padded_size);
      uint8x16_t accum = vdupq_n_u8(0);
      // count number of bytes below threshold inside the group
      // each byte lane keeps its own total (can go up to 255)
      for (int i=base; i < group_end; i += 16) {
         uint8x16_t values = vld1q_u8(&data[i]);
         uint8x16_t is_below = vcltq_u8(values, threshold_vec); // either 0 (when >=) or -1 (when <)
         accum = vsubq_u8(accum, is_below); // adds 1 if <
      }
      // sum subtotals of lanes before the next group
      num += vaddlvq_u8(accum);
   }
   return num;
}
#endif // #if DO_BUILD_NEON
