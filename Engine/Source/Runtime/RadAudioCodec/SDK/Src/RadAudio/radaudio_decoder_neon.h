// Copyright Epic Games Tools, LLC. All Rights Reserved.
#ifndef RADAUDIO_NEON_DECODE_H
#define RADAUDIO_NEON_DECODE_H

#include "radaudio_common.h"

#ifdef __cplusplus
extern "C" {
#endif

#ifdef RADAUDIO_WRAP
#define WRAPPED_NAME(name) RR_STRING_JOIN(RADAUDIO_WRAP, name##_)

#define radaudio_neon_expand_coefficients_excess_read15 WRAPPED_NAME(radaudio_neon_expand_coefficients_excess_read15)
#define radaudio_neon_compute_band_energy_multiple4 WRAPPED_NAME(radaudio_neon_compute_band_energy_multiple4)
#define radaudio_neon_compute_subband_energy_skip12_excess_read7 WRAPPED_NAME(radaudio_neon_compute_subband_energy_skip12_excess_read7)
#define radaudio_neon_compute_windowed_sum_multiple8 WRAPPED_NAME(radaudio_neon_compute_windowed_sum_multiple8)
#define radaudio_neon_unpack_nibbles_read_sentinel16_write_multiple32 WRAPPED_NAME(radaudio_neon_unpack_nibbles_read_sentinel16_write_multiple32)
#define radaudio_neon_count_set_bits_read_multiple8_sentinel8 WRAPPED_NAME(radaudio_neon_count_set_bits_read_multiple8_sentinel8)
#define radaudio_neon_save_samples WRAPPED_NAME(radaudio_neon_save_samples)
#define radaudio_neon_dequantize_long_block_replace_0_with_random_8x8_Nx16 WRAPPED_NAME(radaudio_neon_dequantize_long_block_replace_0_with_random_8x8_Nx16)
#define radaudio_neon_dequantize_short_block WRAPPED_NAME(radaudio_neon_dequantize_short_block)
#define radaudio_neon_distribute_bitflag_coefficients_multiple16 WRAPPED_NAME(radaudio_neon_distribute_bitflag_coefficients_multiple16)
#define radaudio_neon_count_bytes_below_value_sentinel16 WRAPPED_NAME(radaudio_neon_count_bytes_below_value_sentinel16)
#endif

extern rrbool radaudio_neon_expand_coefficients_excess_read15(S8 *nonzero_coefficients, int num_nonzero, S8 *big_coeff, S8 *big_limit);
extern void   radaudio_neon_compute_band_energy_multiple4(F32 *band_energy, int num_bands, int band_exponent[], U16 fine_energy[], F32 band_scale_decode[]);
extern void   radaudio_neon_compute_subband_energy_skip12_excess_read7(F32 *subband_energy, const F32 *band_energy, int num_bands, int *num_subbands_for_band, U16 *quantized_subbands);
extern void   radaudio_neon_compute_windowed_sum_multiple8(float *output, int n, const float *fwd_data, const S16 *rev_data, float rev_scale, const float *window);
extern void   radaudio_neon_unpack_nibbles_read_sentinel16_write_multiple32(S8 *unpacked, U8 *packed, int num_packed, U64 default_nibbles);
extern int    radaudio_neon_count_set_bits_read_multiple8_sentinel8(U8 *data, int num_bytes);
extern float  radaudio_neon_save_samples(S16 *buffer, const float *data, int num);
extern void   radaudio_neon_dequantize_long_block_replace_0_with_random_8x8_Nx16(F32 *coeffs, S8 *quantized_coeff, F32 *subband_energy, int num_subbands, U32 rand_state[4]);
extern void   radaudio_neon_dequantize_short_block(float *coeffs, S8 *quantized_coeff, float *band_energy, int num_bands, int *num_coeffs_for_band);
extern void   radaudio_neon_distribute_bitflag_coefficients_multiple16(S8 *quantized_coeff, int num_coeff, U8 *nonzero_flagbits, S8 *nonzero_coeffs, int *pcur_nonzero_coeffs);
extern int    radaudio_neon_count_bytes_below_value_sentinel16(U8 *data, int num_bytes, U8 threshold);

#ifdef __cplusplus
}
#endif

#endif
