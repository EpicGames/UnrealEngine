// Copyright Epic Games Tools, LLC. All Rights Reserved.
#ifndef RADAUDIO_DECODER_SIMD_TABLES_H
#define RADAUDIO_DECODER_SIMD_TABLES_H

#include "rrCore.h"

/*

#!/usr/bin/python3
print('static const U8 s_popcnt_table[256] = {')
for x in range(256):
   if (x & 15) == 0:
      print('   ', end='')
   npop = sum([(x >> k) & 1 for k in range(8)])
   print('{:2},'.format(npop), end='')
   if (x & 15) == 15:
      print('')
print('};')
print('')
print('static const S8 s_shuffle_table[256][8] = {')
for x in range(256):
   if (x & 3) == 0:
      print('   ', end='')
   print('{ ', end='')
   k = 0
   for j in range(8):
      if (x & (1 << j)) == 0:
         print('-1,', end='')
      else:
         print(f'{k:2},', end='')
         k += 1
   print(' },', end='')
   if (x & 3) == 3:
      print('')
print('};')

*/

static const U8 s_popcnt_table[256] = {
    0, 1, 1, 2, 1, 2, 2, 3, 1, 2, 2, 3, 2, 3, 3, 4,
    1, 2, 2, 3, 2, 3, 3, 4, 2, 3, 3, 4, 3, 4, 4, 5,
    1, 2, 2, 3, 2, 3, 3, 4, 2, 3, 3, 4, 3, 4, 4, 5,
    2, 3, 3, 4, 3, 4, 4, 5, 3, 4, 4, 5, 4, 5, 5, 6,
    1, 2, 2, 3, 2, 3, 3, 4, 2, 3, 3, 4, 3, 4, 4, 5,
    2, 3, 3, 4, 3, 4, 4, 5, 3, 4, 4, 5, 4, 5, 5, 6,
    2, 3, 3, 4, 3, 4, 4, 5, 3, 4, 4, 5, 4, 5, 5, 6,
    3, 4, 4, 5, 4, 5, 5, 6, 4, 5, 5, 6, 5, 6, 6, 7,
    1, 2, 2, 3, 2, 3, 3, 4, 2, 3, 3, 4, 3, 4, 4, 5,
    2, 3, 3, 4, 3, 4, 4, 5, 3, 4, 4, 5, 4, 5, 5, 6,
    2, 3, 3, 4, 3, 4, 4, 5, 3, 4, 4, 5, 4, 5, 5, 6,
    3, 4, 4, 5, 4, 5, 5, 6, 4, 5, 5, 6, 5, 6, 6, 7,
    2, 3, 3, 4, 3, 4, 4, 5, 3, 4, 4, 5, 4, 5, 5, 6,
    3, 4, 4, 5, 4, 5, 5, 6, 4, 5, 5, 6, 5, 6, 6, 7,
    3, 4, 4, 5, 4, 5, 5, 6, 4, 5, 5, 6, 5, 6, 6, 7,
    4, 5, 5, 6, 5, 6, 6, 7, 5, 6, 6, 7, 6, 7, 7, 8,
};

static const S8 s_shuffle_table[256][8] = {
   { -1,-1,-1,-1,-1,-1,-1,-1, },{  0,-1,-1,-1,-1,-1,-1,-1, },{ -1, 0,-1,-1,-1,-1,-1,-1, },{  0, 1,-1,-1,-1,-1,-1,-1, },
   { -1,-1, 0,-1,-1,-1,-1,-1, },{  0,-1, 1,-1,-1,-1,-1,-1, },{ -1, 0, 1,-1,-1,-1,-1,-1, },{  0, 1, 2,-1,-1,-1,-1,-1, },
   { -1,-1,-1, 0,-1,-1,-1,-1, },{  0,-1,-1, 1,-1,-1,-1,-1, },{ -1, 0,-1, 1,-1,-1,-1,-1, },{  0, 1,-1, 2,-1,-1,-1,-1, },
   { -1,-1, 0, 1,-1,-1,-1,-1, },{  0,-1, 1, 2,-1,-1,-1,-1, },{ -1, 0, 1, 2,-1,-1,-1,-1, },{  0, 1, 2, 3,-1,-1,-1,-1, },
   { -1,-1,-1,-1, 0,-1,-1,-1, },{  0,-1,-1,-1, 1,-1,-1,-1, },{ -1, 0,-1,-1, 1,-1,-1,-1, },{  0, 1,-1,-1, 2,-1,-1,-1, },
   { -1,-1, 0,-1, 1,-1,-1,-1, },{  0,-1, 1,-1, 2,-1,-1,-1, },{ -1, 0, 1,-1, 2,-1,-1,-1, },{  0, 1, 2,-1, 3,-1,-1,-1, },
   { -1,-1,-1, 0, 1,-1,-1,-1, },{  0,-1,-1, 1, 2,-1,-1,-1, },{ -1, 0,-1, 1, 2,-1,-1,-1, },{  0, 1,-1, 2, 3,-1,-1,-1, },
   { -1,-1, 0, 1, 2,-1,-1,-1, },{  0,-1, 1, 2, 3,-1,-1,-1, },{ -1, 0, 1, 2, 3,-1,-1,-1, },{  0, 1, 2, 3, 4,-1,-1,-1, },
   { -1,-1,-1,-1,-1, 0,-1,-1, },{  0,-1,-1,-1,-1, 1,-1,-1, },{ -1, 0,-1,-1,-1, 1,-1,-1, },{  0, 1,-1,-1,-1, 2,-1,-1, },
   { -1,-1, 0,-1,-1, 1,-1,-1, },{  0,-1, 1,-1,-1, 2,-1,-1, },{ -1, 0, 1,-1,-1, 2,-1,-1, },{  0, 1, 2,-1,-1, 3,-1,-1, },
   { -1,-1,-1, 0,-1, 1,-1,-1, },{  0,-1,-1, 1,-1, 2,-1,-1, },{ -1, 0,-1, 1,-1, 2,-1,-1, },{  0, 1,-1, 2,-1, 3,-1,-1, },
   { -1,-1, 0, 1,-1, 2,-1,-1, },{  0,-1, 1, 2,-1, 3,-1,-1, },{ -1, 0, 1, 2,-1, 3,-1,-1, },{  0, 1, 2, 3,-1, 4,-1,-1, },
   { -1,-1,-1,-1, 0, 1,-1,-1, },{  0,-1,-1,-1, 1, 2,-1,-1, },{ -1, 0,-1,-1, 1, 2,-1,-1, },{  0, 1,-1,-1, 2, 3,-1,-1, },
   { -1,-1, 0,-1, 1, 2,-1,-1, },{  0,-1, 1,-1, 2, 3,-1,-1, },{ -1, 0, 1,-1, 2, 3,-1,-1, },{  0, 1, 2,-1, 3, 4,-1,-1, },
   { -1,-1,-1, 0, 1, 2,-1,-1, },{  0,-1,-1, 1, 2, 3,-1,-1, },{ -1, 0,-1, 1, 2, 3,-1,-1, },{  0, 1,-1, 2, 3, 4,-1,-1, },
   { -1,-1, 0, 1, 2, 3,-1,-1, },{  0,-1, 1, 2, 3, 4,-1,-1, },{ -1, 0, 1, 2, 3, 4,-1,-1, },{  0, 1, 2, 3, 4, 5,-1,-1, },
   { -1,-1,-1,-1,-1,-1, 0,-1, },{  0,-1,-1,-1,-1,-1, 1,-1, },{ -1, 0,-1,-1,-1,-1, 1,-1, },{  0, 1,-1,-1,-1,-1, 2,-1, },
   { -1,-1, 0,-1,-1,-1, 1,-1, },{  0,-1, 1,-1,-1,-1, 2,-1, },{ -1, 0, 1,-1,-1,-1, 2,-1, },{  0, 1, 2,-1,-1,-1, 3,-1, },
   { -1,-1,-1, 0,-1,-1, 1,-1, },{  0,-1,-1, 1,-1,-1, 2,-1, },{ -1, 0,-1, 1,-1,-1, 2,-1, },{  0, 1,-1, 2,-1,-1, 3,-1, },
   { -1,-1, 0, 1,-1,-1, 2,-1, },{  0,-1, 1, 2,-1,-1, 3,-1, },{ -1, 0, 1, 2,-1,-1, 3,-1, },{  0, 1, 2, 3,-1,-1, 4,-1, },
   { -1,-1,-1,-1, 0,-1, 1,-1, },{  0,-1,-1,-1, 1,-1, 2,-1, },{ -1, 0,-1,-1, 1,-1, 2,-1, },{  0, 1,-1,-1, 2,-1, 3,-1, },
   { -1,-1, 0,-1, 1,-1, 2,-1, },{  0,-1, 1,-1, 2,-1, 3,-1, },{ -1, 0, 1,-1, 2,-1, 3,-1, },{  0, 1, 2,-1, 3,-1, 4,-1, },
   { -1,-1,-1, 0, 1,-1, 2,-1, },{  0,-1,-1, 1, 2,-1, 3,-1, },{ -1, 0,-1, 1, 2,-1, 3,-1, },{  0, 1,-1, 2, 3,-1, 4,-1, },
   { -1,-1, 0, 1, 2,-1, 3,-1, },{  0,-1, 1, 2, 3,-1, 4,-1, },{ -1, 0, 1, 2, 3,-1, 4,-1, },{  0, 1, 2, 3, 4,-1, 5,-1, },
   { -1,-1,-1,-1,-1, 0, 1,-1, },{  0,-1,-1,-1,-1, 1, 2,-1, },{ -1, 0,-1,-1,-1, 1, 2,-1, },{  0, 1,-1,-1,-1, 2, 3,-1, },
   { -1,-1, 0,-1,-1, 1, 2,-1, },{  0,-1, 1,-1,-1, 2, 3,-1, },{ -1, 0, 1,-1,-1, 2, 3,-1, },{  0, 1, 2,-1,-1, 3, 4,-1, },
   { -1,-1,-1, 0,-1, 1, 2,-1, },{  0,-1,-1, 1,-1, 2, 3,-1, },{ -1, 0,-1, 1,-1, 2, 3,-1, },{  0, 1,-1, 2,-1, 3, 4,-1, },
   { -1,-1, 0, 1,-1, 2, 3,-1, },{  0,-1, 1, 2,-1, 3, 4,-1, },{ -1, 0, 1, 2,-1, 3, 4,-1, },{  0, 1, 2, 3,-1, 4, 5,-1, },
   { -1,-1,-1,-1, 0, 1, 2,-1, },{  0,-1,-1,-1, 1, 2, 3,-1, },{ -1, 0,-1,-1, 1, 2, 3,-1, },{  0, 1,-1,-1, 2, 3, 4,-1, },
   { -1,-1, 0,-1, 1, 2, 3,-1, },{  0,-1, 1,-1, 2, 3, 4,-1, },{ -1, 0, 1,-1, 2, 3, 4,-1, },{  0, 1, 2,-1, 3, 4, 5,-1, },
   { -1,-1,-1, 0, 1, 2, 3,-1, },{  0,-1,-1, 1, 2, 3, 4,-1, },{ -1, 0,-1, 1, 2, 3, 4,-1, },{  0, 1,-1, 2, 3, 4, 5,-1, },
   { -1,-1, 0, 1, 2, 3, 4,-1, },{  0,-1, 1, 2, 3, 4, 5,-1, },{ -1, 0, 1, 2, 3, 4, 5,-1, },{  0, 1, 2, 3, 4, 5, 6,-1, },
   { -1,-1,-1,-1,-1,-1,-1, 0, },{  0,-1,-1,-1,-1,-1,-1, 1, },{ -1, 0,-1,-1,-1,-1,-1, 1, },{  0, 1,-1,-1,-1,-1,-1, 2, },
   { -1,-1, 0,-1,-1,-1,-1, 1, },{  0,-1, 1,-1,-1,-1,-1, 2, },{ -1, 0, 1,-1,-1,-1,-1, 2, },{  0, 1, 2,-1,-1,-1,-1, 3, },
   { -1,-1,-1, 0,-1,-1,-1, 1, },{  0,-1,-1, 1,-1,-1,-1, 2, },{ -1, 0,-1, 1,-1,-1,-1, 2, },{  0, 1,-1, 2,-1,-1,-1, 3, },
   { -1,-1, 0, 1,-1,-1,-1, 2, },{  0,-1, 1, 2,-1,-1,-1, 3, },{ -1, 0, 1, 2,-1,-1,-1, 3, },{  0, 1, 2, 3,-1,-1,-1, 4, },
   { -1,-1,-1,-1, 0,-1,-1, 1, },{  0,-1,-1,-1, 1,-1,-1, 2, },{ -1, 0,-1,-1, 1,-1,-1, 2, },{  0, 1,-1,-1, 2,-1,-1, 3, },
   { -1,-1, 0,-1, 1,-1,-1, 2, },{  0,-1, 1,-1, 2,-1,-1, 3, },{ -1, 0, 1,-1, 2,-1,-1, 3, },{  0, 1, 2,-1, 3,-1,-1, 4, },
   { -1,-1,-1, 0, 1,-1,-1, 2, },{  0,-1,-1, 1, 2,-1,-1, 3, },{ -1, 0,-1, 1, 2,-1,-1, 3, },{  0, 1,-1, 2, 3,-1,-1, 4, },
   { -1,-1, 0, 1, 2,-1,-1, 3, },{  0,-1, 1, 2, 3,-1,-1, 4, },{ -1, 0, 1, 2, 3,-1,-1, 4, },{  0, 1, 2, 3, 4,-1,-1, 5, },
   { -1,-1,-1,-1,-1, 0,-1, 1, },{  0,-1,-1,-1,-1, 1,-1, 2, },{ -1, 0,-1,-1,-1, 1,-1, 2, },{  0, 1,-1,-1,-1, 2,-1, 3, },
   { -1,-1, 0,-1,-1, 1,-1, 2, },{  0,-1, 1,-1,-1, 2,-1, 3, },{ -1, 0, 1,-1,-1, 2,-1, 3, },{  0, 1, 2,-1,-1, 3,-1, 4, },
   { -1,-1,-1, 0,-1, 1,-1, 2, },{  0,-1,-1, 1,-1, 2,-1, 3, },{ -1, 0,-1, 1,-1, 2,-1, 3, },{  0, 1,-1, 2,-1, 3,-1, 4, },
   { -1,-1, 0, 1,-1, 2,-1, 3, },{  0,-1, 1, 2,-1, 3,-1, 4, },{ -1, 0, 1, 2,-1, 3,-1, 4, },{  0, 1, 2, 3,-1, 4,-1, 5, },
   { -1,-1,-1,-1, 0, 1,-1, 2, },{  0,-1,-1,-1, 1, 2,-1, 3, },{ -1, 0,-1,-1, 1, 2,-1, 3, },{  0, 1,-1,-1, 2, 3,-1, 4, },
   { -1,-1, 0,-1, 1, 2,-1, 3, },{  0,-1, 1,-1, 2, 3,-1, 4, },{ -1, 0, 1,-1, 2, 3,-1, 4, },{  0, 1, 2,-1, 3, 4,-1, 5, },
   { -1,-1,-1, 0, 1, 2,-1, 3, },{  0,-1,-1, 1, 2, 3,-1, 4, },{ -1, 0,-1, 1, 2, 3,-1, 4, },{  0, 1,-1, 2, 3, 4,-1, 5, },
   { -1,-1, 0, 1, 2, 3,-1, 4, },{  0,-1, 1, 2, 3, 4,-1, 5, },{ -1, 0, 1, 2, 3, 4,-1, 5, },{  0, 1, 2, 3, 4, 5,-1, 6, },
   { -1,-1,-1,-1,-1,-1, 0, 1, },{  0,-1,-1,-1,-1,-1, 1, 2, },{ -1, 0,-1,-1,-1,-1, 1, 2, },{  0, 1,-1,-1,-1,-1, 2, 3, },
   { -1,-1, 0,-1,-1,-1, 1, 2, },{  0,-1, 1,-1,-1,-1, 2, 3, },{ -1, 0, 1,-1,-1,-1, 2, 3, },{  0, 1, 2,-1,-1,-1, 3, 4, },
   { -1,-1,-1, 0,-1,-1, 1, 2, },{  0,-1,-1, 1,-1,-1, 2, 3, },{ -1, 0,-1, 1,-1,-1, 2, 3, },{  0, 1,-1, 2,-1,-1, 3, 4, },
   { -1,-1, 0, 1,-1,-1, 2, 3, },{  0,-1, 1, 2,-1,-1, 3, 4, },{ -1, 0, 1, 2,-1,-1, 3, 4, },{  0, 1, 2, 3,-1,-1, 4, 5, },
   { -1,-1,-1,-1, 0,-1, 1, 2, },{  0,-1,-1,-1, 1,-1, 2, 3, },{ -1, 0,-1,-1, 1,-1, 2, 3, },{  0, 1,-1,-1, 2,-1, 3, 4, },
   { -1,-1, 0,-1, 1,-1, 2, 3, },{  0,-1, 1,-1, 2,-1, 3, 4, },{ -1, 0, 1,-1, 2,-1, 3, 4, },{  0, 1, 2,-1, 3,-1, 4, 5, },
   { -1,-1,-1, 0, 1,-1, 2, 3, },{  0,-1,-1, 1, 2,-1, 3, 4, },{ -1, 0,-1, 1, 2,-1, 3, 4, },{  0, 1,-1, 2, 3,-1, 4, 5, },
   { -1,-1, 0, 1, 2,-1, 3, 4, },{  0,-1, 1, 2, 3,-1, 4, 5, },{ -1, 0, 1, 2, 3,-1, 4, 5, },{  0, 1, 2, 3, 4,-1, 5, 6, },
   { -1,-1,-1,-1,-1, 0, 1, 2, },{  0,-1,-1,-1,-1, 1, 2, 3, },{ -1, 0,-1,-1,-1, 1, 2, 3, },{  0, 1,-1,-1,-1, 2, 3, 4, },
   { -1,-1, 0,-1,-1, 1, 2, 3, },{  0,-1, 1,-1,-1, 2, 3, 4, },{ -1, 0, 1,-1,-1, 2, 3, 4, },{  0, 1, 2,-1,-1, 3, 4, 5, },
   { -1,-1,-1, 0,-1, 1, 2, 3, },{  0,-1,-1, 1,-1, 2, 3, 4, },{ -1, 0,-1, 1,-1, 2, 3, 4, },{  0, 1,-1, 2,-1, 3, 4, 5, },
   { -1,-1, 0, 1,-1, 2, 3, 4, },{  0,-1, 1, 2,-1, 3, 4, 5, },{ -1, 0, 1, 2,-1, 3, 4, 5, },{  0, 1, 2, 3,-1, 4, 5, 6, },
   { -1,-1,-1,-1, 0, 1, 2, 3, },{  0,-1,-1,-1, 1, 2, 3, 4, },{ -1, 0,-1,-1, 1, 2, 3, 4, },{  0, 1,-1,-1, 2, 3, 4, 5, },
   { -1,-1, 0,-1, 1, 2, 3, 4, },{  0,-1, 1,-1, 2, 3, 4, 5, },{ -1, 0, 1,-1, 2, 3, 4, 5, },{  0, 1, 2,-1, 3, 4, 5, 6, },
   { -1,-1,-1, 0, 1, 2, 3, 4, },{  0,-1,-1, 1, 2, 3, 4, 5, },{ -1, 0,-1, 1, 2, 3, 4, 5, },{  0, 1,-1, 2, 3, 4, 5, 6, },
   { -1,-1, 0, 1, 2, 3, 4, 5, },{  0,-1, 1, 2, 3, 4, 5, 6, },{ -1, 0, 1, 2, 3, 4, 5, 6, },{  0, 1, 2, 3, 4, 5, 6, 7, },
};

#endif
