// Copyright Epic Games Tools, LLC. All Rights Reserved.
/////////////////////////////////////////////////////////////////////
//
//   SSE2 only
//

#include "radaudio_common.h"
#include "radaudio_decoder_sse2.h"
#include "rrbits.h"
#include <stdio.h>

#ifdef RADAUDIO_WRAP
#define WRAPPED_NAME(name) RR_STRING_JOIN(RADAUDIO_WRAP, name##_)
#define radaudio_dummy_decode_sse2 WRAPPED_NAME(radaudio_dummy_decode_sse2)
#endif

int radaudio_dummy_decode_sse2(void) { return 0; } // avoid "empty translation unit" warning

#ifdef DO_BUILD_SSE4

#include <emmintrin.h> // SSE2

#define ps(x)  _mm_castsi128_ps(x)
#define pi(x)  _mm_castps_si128(x)

typedef __m128i int32x4;
typedef __m128i int16x8;
typedef __m128i int8x16;
typedef __m128  float4;

#define _mm_shuffle_ps_mine(a,imm) ps(_mm_shuffle_epi32(pi(a), (imm)))

#define MY_SHUFFLE(w,x,y,z) _MM_SHUFFLE(z,y,x,w)

void radaudio_sse2_unpack_nibbles_read_sentinel16_write_multiple32(S8 *unpacked, U8 *packed, int num_packed, U64 default_nibbles)
{
   int8x16 nibble_mask = _mm_set1_epi8(0x0f);
   int padded_size = (num_packed + 15) & ~15;
   // put default values in the padding
   RR_PUT64_NATIVE(&packed[num_packed+0], default_nibbles);
   RR_PUT64_NATIVE(&packed[num_packed+8], default_nibbles);

   for (int i=0; i < padded_size; i += 16) {
      // process 16 bytes at a time
      int8x16 value = _mm_loadu_si128((int8x16 *) &packed[i]);

      int8x16 lo_nibble = _mm_and_si128(      value             , nibble_mask);
      int8x16 hi_nibble = _mm_and_si128(_mm_srli_epi16(value,4) , nibble_mask);

      _mm_storeu_si128((int8x16*) &unpacked[i*2+ 0], _mm_unpacklo_epi8(lo_nibble, hi_nibble));
      _mm_storeu_si128((int8x16*) &unpacked[i*2+16], _mm_unpackhi_epi8(lo_nibble, hi_nibble));
   }
   return;
}

rrbool radaudio_sse2_expand_coefficients_excess_read15(S8 *nonzero_coefficients, int num_nonzero, S8 *big_coeff, S8 *big_limit)
{
   int8x16 zero = _mm_setzero_si128();
   int8x16 eight = _mm_set1_epi8(8);
   // nonzero_coefficients[] has been padded with dummy data to a multiple of 16
   int padded_size = (num_nonzero + 15) & ~15;
   for (int i=0; i < padded_size; i += 16) {

      // load the coefficients and check if any are 0
      int8x16 coeff   = _mm_loadu_si128((int8x16 *) &nonzero_coefficients[i]);
      int8x16 is_zero = _mm_cmpeq_epi8(coeff, zero);
      U32 result      = _mm_movemask_epi8(is_zero);

      // subtract 8 from all of them and store them back (the ones that were 0 will get overwritten later)
      _mm_storeu_si128((int8x16 *) &nonzero_coefficients[i], _mm_sub_epi8(coeff, eight));

      if (result != 0) {
         do {
            SINTa k = rrCtz32(result);
            result = rrClearLowestSetBit32(result);
            nonzero_coefficients[i+k] = *big_coeff++;
         } while (result);
         if (big_coeff > big_limit)
            return 0;
         // alternative strategy: do this all in SIMD using a prefix sum etc.
         // in "usual" case, there will be at most 1 big coefficient per 16, so we take
         // one mispredict for result != 0, but then the pattern of _BitScanForward will
         // always be true first time, false second time, so that will predict perfectly,
         // so probably a full SIMD is slower?
      }
   }
   return 1;
}

void radaudio_sse2_compute_band_energy_multiple4(F32 *band_energy, int num_bands, int band_exponent[], U16 fine_energy[], F32 band_scale_decode[])
{
   // num_bands may not be a multiple of 4, but all arrays are defined so we can run to a multiple of 4
   for (int j=0; j < num_bands; j += 4) {  // all arrays are safe to run extra
      int16x8 qe_raw = _mm_loadl_epi64((int16x8*) &fine_energy[j]);

      // quantized fine energy
      int32x4 qe = _mm_unpacklo_epi16(qe_raw, _mm_setzero_si128());

      // packed fine_energy (still in a pseudo-logarithmic encoding)
      float4 pe = _mm_mul_ps(_mm_cvtepi32_ps(qe), _mm_set1_ps(1.0f / (1 << MAX_FINE_ENERGY_BITS)));

      // approximate fe = pow(2,pe) with a quadratic: fe = 0.34375f*pe*pe + 0.65625f*pe + 1;
      float4 fe_1 = _mm_add_ps(_mm_mul_ps(pe  , _mm_set1_ps(0.34375f)), _mm_set1_ps(0.65625f));
      float4 fe   = _mm_add_ps(_mm_mul_ps(pe  , fe_1                 ), _mm_set1_ps(1.0f    ));

      int32x4 bandexp = _mm_loadu_si128((int32x4*) &band_exponent[j]);

      // +16 to make it non-negative (C code uses shift, so doesn't handle negative, and the +16 is compensated in the other multiplier
      // which we're also going to use), then +127 to convert to an IEEE biased expnonent
      int32x4 biased = _mm_add_epi32(bandexp, _mm_set1_epi32(16+127));

      // shift to the right place
      int32x4 exp_float = _mm_slli_epi32(biased, 23); // shift past 23 bits of mantissa

      // zero it if bandexp is BAND_EXPONENT_NONE
      int32x4 is_none = _mm_cmpeq_epi32(bandexp, _mm_set1_epi32(BAND_EXPONENT_NONE));

      // if is_none is true in a lane, then mask that lane to 0, which is floating point 0.0
      float4 ce = ps(_mm_andnot_si128(is_none, exp_float));

      // multiply the exponent, the fine energy, and the compensatory scaler
      float4 band_energy_decoded = _mm_mul_ps(_mm_mul_ps(fe, ce), _mm_loadu_ps(&band_scale_decode[j]));

      _mm_storeu_ps(&band_energy[j], band_energy_decoded);
   }
}

static RADFORCEINLINE float4 float4_reverse(float4 v)
{
   return _mm_shuffle_ps_mine(v, MY_SHUFFLE(3,2,1,0));
}

static RADFORCEINLINE float4 float4_load_rev(const float *data)
{
   return float4_reverse(_mm_loadu_ps(data));
}

static RADFORCEINLINE void load_8int16_scaled(float4 *data_r0, float4 *data_r1, const S16 *data, float4 scale)
{
   int16x8 data_ri  = _mm_loadu_si128((int16x8 *) data);
   // unpack signed 16 to top 16 bits of signed 32
   int32x4 data_r0i = _mm_unpacklo_epi16(_mm_setzero_si128(), data_ri);
   int32x4 data_r1i = _mm_unpackhi_epi16(_mm_setzero_si128(), data_ri);
   *data_r0 = _mm_mul_ps(scale, _mm_cvtepi32_ps(data_r0i));
   *data_r1 = _mm_mul_ps(scale, _mm_cvtepi32_ps(data_r1i));
}

// rev_data is read forwards, but uses a reversed window
void radaudio_sse2_compute_windowed_sum_multiple8(float *output, int n, const float *fwd_data, const S16 *rev_data, float rev_scale, const float *window)
{
   // we need to sign extend the S16 to float then multiply by ref_scale;
   // easiest solution is to put the S16 in high word of S32, with 0s at bottom, and scale down
   float4 scale = _mm_set1_ps(rev_scale / 65536.0f);
   SINTa N2 = n >> 1;

   // front half
   for (SINTa i = 0; i < N2; i += 8) {
      // read rev_data and scale
      float4 data_r0, data_r1;
      load_8int16_scaled(&data_r0, &data_r1, rev_data + i, scale);

      // fwd_data is read reversed (confusing naming, I know)
      float4 data_f0  = float4_load_rev(fwd_data+N2 - i - 4);
      float4 data_f1  = float4_load_rev(fwd_data+N2 - i - 8);

      // windows
      float4 win_r0   = float4_load_rev(window+n - i - 4);
      float4 win_r1   = float4_load_rev(window+n - i - 8);
      float4 win_f0   = _mm_loadu_ps(window + i);
      float4 win_f1   = _mm_loadu_ps(window + i + 4);

      // weighted sum
      float4 sum0     = _mm_sub_ps(_mm_mul_ps(data_r0, win_r0), _mm_mul_ps(data_f0, win_f0));
      float4 sum1     = _mm_sub_ps(_mm_mul_ps(data_r1, win_r1), _mm_mul_ps(data_f1, win_f1));
      _mm_storeu_ps(output+i  , sum0);
      _mm_storeu_ps(output+i+4, sum1);
   }

   // back half
   for (SINTa i = 0; i < N2; i += 8) {
      // read rev_data and scale
      // (this one is actually reversed, hence r1 and r0 being swapped)
      float4 data_r0, data_r1;
      load_8int16_scaled(&data_r1, &data_r0, rev_data+N2 - i - 8, scale);

      // fwd_data
      float4 data_f0  = _mm_loadu_ps(fwd_data + i);
      float4 data_f1  = _mm_loadu_ps(fwd_data + i + 4);

      // windows
      float4 win_r0   = _mm_loadu_ps(window+N2 - i - 4);
      float4 win_r1   = _mm_loadu_ps(window+N2 - i - 8);
      float4 win_f0   = _mm_loadu_ps(window+N2 + i);
      float4 win_f1   = _mm_loadu_ps(window+N2 + i + 4);

      // weighted sum
      float4 sum0     = _mm_add_ps(float4_reverse(_mm_mul_ps(data_r0, win_r0)), _mm_mul_ps(data_f0, win_f0));
      float4 sum1     = _mm_add_ps(float4_reverse(_mm_mul_ps(data_r1, win_r1)), _mm_mul_ps(data_f1, win_f1));
      _mm_storeu_ps(output+N2+i  , sum0);
      _mm_storeu_ps(output+N2+i+4, sum1);
   }
}

int radaudio_sse2_count_bytes_below_value_sentinel16(U8 *data, int num_bytes, U8 threshold)
{
   int num=0;
   int padded_size = (num_bytes+15) & ~15;
   RR_PUT64_NATIVE(&data[num_bytes+0], 0xffffffffffffffffull);
   RR_PUT64_NATIVE(&data[num_bytes+8], 0xffffffffffffffffull);

   int8x16 neg128 = _mm_set1_epi8(-128);
   int8x16 threshold_signed = _mm_set1_epi8((int) threshold-128);
   int max_in_group = 255 * 16;
   for (int base=0; base < padded_size; base += max_in_group) {
      int group_end = RR_MIN(base + max_in_group, padded_size);
      int8x16 accum = _mm_setzero_si128();
      // count number of bytes below threshold inside the group
      // each byte lane keeps its own total (can go up to 255)
      for (int i=base; i < group_end; i += 16) {
         int8x16 values   = _mm_loadu_si128((int8x16 *) &data[i]);
         int8x16 is_below = _mm_cmplt_epi8(_mm_add_epi8(values, neg128), threshold_signed);
         accum = _mm_sub_epi8(accum, is_below);
      }
      // use SAD against 0 to compute subtotals of lanes in groups of 8
      int32x4 subtotals = _mm_sad_epu8(accum, _mm_setzero_si128());
      // add the two 8-lane subtotals (which are <=8*255 so fit in U16) onto num
      num += _mm_cvtsi128_si32(subtotals) + _mm_extract_epi16(subtotals, 4);
   }
   return num;
}

U8 *radaudio_sse2_find_next_coarse_run_excess16(U8 *cur, U8 *end)
{
   int8x16 lower_bound = _mm_set1_epi8(COARSE_RUNLEN_THRESHOLD);
   int8x16 upper_bound = _mm_set1_epi8((S8)(U8)(MAX_RUNLEN - 1));
   while (cur < end) {
      int8x16 bytes = _mm_loadu_si128((int8x16 *) cur);
      cur += 16;

      // clamp to [COARSE_RUNLEN_THRESHOLD,MAX_RUNLEN)
      int8x16 clamped = _mm_min_epu8(_mm_max_epu8(bytes, lower_bound), upper_bound);

      // any byte that didn't change was in that range to begin with
      int8x16 in_range = _mm_cmpeq_epi8(bytes, clamped);
      int mask = _mm_movemask_epi8(in_range);
      if (mask != 0) {
         return (cur - 16) + rrCtz32(mask);
      }
   }

   return cur;
}

#endif // #if DO_BUILD_SSE4
