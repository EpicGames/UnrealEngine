// Copyright Epic Games Tools, LLC. All Rights Reserved.
#ifndef RADAUDIO_SSE4_DECODE_H
#define RADAUDIO_SSE4_DECODE_H

#include "radaudio_common.h"
#include "radaudio_sse.h"

#ifdef __cplusplus
extern "C" {
#endif

#ifdef RADAUDIO_WRAP
#define WRAPPED_NAME(name) RR_STRING_JOIN(RADAUDIO_WRAP, name##_)

#define radaudio_sse2_compute_windowed_sum_multiple8                    WRAPPED_NAME(radaudio_sse2_compute_windowed_sum_multiple8)
#define radaudio_sse2_compute_band_energy_multiple4                     WRAPPED_NAME(radaudio_sse2_compute_band_energy_multiple4)
#define radaudio_sse2_count_bytes_below_value_sentinel16                WRAPPED_NAME(radaudio_sse2_count_bytes_below_value_sentinel16)
#define radaudio_sse2_unpack_nibbles_read_sentinel16_write_multiple32   WRAPPED_NAME(radaudio_sse2_unpack_nibbles_read_sentinel16_write_multiple32)
#define radaudio_sse2_expand_coefficients_excess_read15                 WRAPPED_NAME(radaudio_sse2_expand_coefficients_excess_read15)
#define radaudio_sse2_save_samples                						WRAPPED_NAME(radaudio_sse2_save_samples)
#define radaudio_sse2_find_next_coarse_run_excess16 WRAPPED_NAME(radaudio_sse2_find_next_coarse_run_excess16)
#endif

extern void   radaudio_sse2_compute_windowed_sum_multiple8(float *output, int n, const float *fwd_data, const S16 *rev_data, float rev_scale, const float *window);
extern void   radaudio_sse2_compute_band_energy_multiple4(F32 *band_energy, int num_bands, int band_exponent[], U16 fine_energy[], F32 band_scale_decode[]);
extern int    radaudio_sse2_count_bytes_below_value_sentinel16(U8 *data, int num_bytes, U8 threshold);
extern void   radaudio_sse2_unpack_nibbles_read_sentinel16_write_multiple32(S8 *unpacked, U8 *packed, int num_packed, U64 default_nibbles);
extern rrbool radaudio_sse2_expand_coefficients_excess_read15(S8 *nonzero_coefficients, int num_nonzero, S8 *big_coeff, S8 *big_limit);
extern float  radaudio_sse2_save_samples(S16 *buffer, const float *data, int num);
extern U8 *   radaudio_sse2_find_next_coarse_run_excess16(U8 *cur, U8 *end);

#ifdef __cplusplus
}
#endif

#endif
