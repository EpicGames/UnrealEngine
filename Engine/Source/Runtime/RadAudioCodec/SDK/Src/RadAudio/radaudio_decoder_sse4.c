// Copyright Epic Games Tools, LLC. All Rights Reserved.
//////////////////////////////////////////////////////////////////////////////
//
//  SSE4.1 + POPCNT required
//

/* @cdep pre
   $when($NeedsClangMachineSwitches,
      $addlocalcswitches(-msse4.1)
   )
*/

#include "radaudio_common.h"
#include "radaudio_decoder_sse2.h"
#include "radaudio_decoder_sse4.h"
#include "radaudio_decoder_simd_tables.h"
#include <string.h>
#include <stdio.h>
#include <math.h>
#include <string.h> // memset

#ifdef RADAUDIO_WRAP
#define WRAPPED_NAME(name) RR_STRING_JOIN(RADAUDIO_WRAP, name##_)
#define radaudio_dummy_decode_sse4 WRAPPED_NAME(radaudio_dummy_decode_sse4)
#endif

int radaudio_dummy_decode_sse4(void) { return 0; } // avoid "empty translation unit" warning

#ifdef DO_BUILD_SSE4

#include <emmintrin.h> // SSE2
#include <tmmintrin.h> // SSSE3
#include <smmintrin.h> // SSE4.1

#ifdef _MSC_VER
#include <intrin.h> // POPCNT
#else
#define __popcnt64 __builtin_popcountll
#endif

#define ps(x)  _mm_castsi128_ps(x)
#define pi(x)  _mm_castps_si128(x)

typedef __m128i int32x4;
typedef __m128i int16x8;
typedef __m128i int8x16;
typedef __m128  float4;

#define _mm_shuffle_ps_mine(a,imm) ps(_mm_shuffle_epi32(pi(a), (imm)))

#define MY_SHUFFLE(w,x,y,z) _MM_SHUFFLE(z,y,x,w)

static inline void store_lo32(void *dest, __m128i bytes)
{
	*(int *)dest = _mm_cvtsi128_si32(bytes);
}

static inline void store_lo64(void *dest, __m128i bytes)
{
	_mm_storel_epi64((__m128i*)dest, bytes);
}

void radaudio_ssse3_distribute_bitflag_coefficients_multiple16(radaudio_cpu_features cpu,
                                           S8 *quantized_coeff, int num_coeff,
                                           U8 *nonzero_flagbits,
                                           S8 *nonzero_coeffs, int *pcur_nonzero_coeffs)
{
   SINTa cur_nonzero_coeffs = *pcur_nonzero_coeffs;

   for (int i=0; i < num_coeff; i += 8) {
      // load 8 bitflags. 0 = set coefficient to 0, 1 = decode from array
      U8 flags = *nonzero_flagbits++;

      // load next batch of nonzero coefficients and shuffles from a table
      SINTa advance = s_popcnt_table[flags];
      int16x8 shuffles = _mm_loadl_epi64((const int16x8*) s_shuffle_table[flags]);
      int16x8 nzcoeffs = _mm_loadl_epi64((const int16x8*) &nonzero_coeffs[cur_nonzero_coeffs]);

      // distribute 8 coefficients
      int16x8 distributed_coeffs = _mm_shuffle_epi8(nzcoeffs, shuffles);

      // write them out
      store_lo64(&quantized_coeff[i], distributed_coeffs);
      cur_nonzero_coeffs += advance;
   }
   *pcur_nonzero_coeffs = (int)cur_nonzero_coeffs;
}


void radaudio_sse4_randomize_long_block_8x8_Nx16(S8 *quantized_coeff, U32 rand_state_array[4], int num_subbands)
{
   int8x16 randtab                  = _mm_setr_epi8(-1,1, -2,2, -3,3, -4,4, -5,5, -6,6, -7,7, -8,8 );
   int32x4 rand_state               = _mm_load_si128((int32x4*) rand_state_array);
   const int8x16 nibble_mask        = _mm_set1_epi8(0x0f);
   const int32x4 lcg_mul            = _mm_set1_epi32(LCG_MUL);
   const int32x4 lcg_add            = _mm_set1_epi32(LCG_ADD);
   const int32x4 lcg_mul_partial    = _mm_setr_epi32(LCG_MUL, LCG_MUL, 1, 1); // *1 on disabled lanes
   const int32x4 lcg_add_partial    = _mm_setr_epi32(LCG_ADD, LCG_ADD, 0, 0); // +0 on disabled lanes

   int cb = 0;
   int j=0;
   for (; j < 8; ++j) {
      if (RR_GET64_NATIVE(&quantized_coeff[cb+0]) == 0) {
         // get the top 4 bits from each byte
         int8x16 rbits = _mm_and_si128(_mm_srli_epi16(rand_state, 4), nibble_mask);

         // lookup in random_table[]
         int8x16 values = _mm_shuffle_epi8(randtab, rbits);

         // those are the coefficients, write them out
         store_lo64(&quantized_coeff[cb], values);

         // update the LCG streams (NOTE: only updating two streams)
         rand_state = _mm_add_epi32(_mm_mullo_epi32(rand_state, lcg_mul_partial), lcg_add_partial);
      }
      cb += 8;
   }

   for (; j < num_subbands; ++j) {
      if ((RR_GET64_NATIVE(&quantized_coeff[cb+0]) | RR_GET64_NATIVE(&quantized_coeff[cb+8])) == 0) {
         // get the top 4 bits from each byte
         int8x16 rbits = _mm_and_si128(_mm_srli_epi16(rand_state, 4), nibble_mask);

         // lookup in random_table[]
         int8x16 values = _mm_shuffle_epi8(randtab, rbits);

         // those are the coefficients, write them out
         _mm_storeu_si128((int8x16*) &quantized_coeff[cb], values);

         // update the LCG streams
         rand_state = _mm_add_epi32(_mm_mullo_epi32(rand_state, lcg_mul), lcg_add);
      }

      cb += 16;
   }
}

void radaudio_sse4_compute_subband_energy_skip12_excess_read7(F32 *subband_energy, const F32 *band_energy, int num_bands, int *num_subbands_for_band, U16 *quantized_subbands)
{
   static RAD_ALIGN(U16, subband_mask[9][8], 16) =
   {
      { 0 },
      { 0xffff, },
      { 0xffff, 0xffff, },
      { 0xffff, 0xffff, 0xffff, },
      { 0xffff, 0xffff, 0xffff, 0xffff, },
      { 0xffff, 0xffff, 0xffff, 0xffff, 0xffff, },
      { 0xffff, 0xffff, 0xffff, 0xffff, 0xffff, 0xffff, },
      { 0xffff, 0xffff, 0xffff, 0xffff, 0xffff, 0xffff, 0xffff, },
      { 0xffff, 0xffff, 0xffff, 0xffff, 0xffff, 0xffff, 0xffff, 0xffff },
   };

   int start = 12;
   int j = 12;

   // we're going to add squares of quantized subband values.

   // this loop processes 4 bands with 2 subbands each
   for (; j+3 < num_bands && num_subbands_for_band[j+3] == 2; j += 4) {
      rrAssert(num_subbands_for_band[j] == 2 && num_subbands_for_band[j+1] == 2 && num_subbands_for_band[j+2] == 2);

      // load the eight subbands
      int16x8 qsub = _mm_loadu_si128((int16x8*) &quantized_subbands[start]);

      // square the subbands and add them together
      int32x4 total = _mm_madd_epi16(qsub, qsub);

      // compute the normalizing factor, 1/sqrt(sum_of_squares) and multiply by the band energy now so we don't have to do it later
      float4 band_scale = _mm_div_ps(_mm_loadu_ps(&band_energy[j]), _mm_sqrt_ps(_mm_cvtepi32_ps(total)));

      float4 subbandq_lo = _mm_cvtepi32_ps(_mm_cvtepi16_epi32(qsub));
      float4 subbandq_hi = _mm_cvtepi32_ps(_mm_cvtepi16_epi32(_mm_shuffle_epi32(qsub, MY_SHUFFLE(2,3,0,1))));

      float4 subband_lo = _mm_mul_ps(subbandq_lo, _mm_shuffle_ps_mine(band_scale, MY_SHUFFLE(0,0,1,1)));
      float4 subband_hi = _mm_mul_ps(subbandq_hi, _mm_shuffle_ps_mine(band_scale, MY_SHUFFLE(2,2,3,3)));

      _mm_storeu_ps(&subband_energy[start+0], subband_lo);
      _mm_storeu_ps(&subband_energy[start+4], subband_hi);

      start += 8;
   }

   // this loop processes 2 bands with 2 subbands each
   for (; j+1 < num_bands && num_subbands_for_band[j+1] == 2; j += 2) {
      rrAssert(num_subbands_for_band[j] == 2);

      // load the four subbands
      int16x8 qsub = _mm_loadl_epi64((int16x8*) &quantized_subbands[start]);

      // square the subbands and add them together.
      int32x4 total = _mm_madd_epi16(qsub, qsub);

      // compute the normalizing factor, 1/sqrt(sum_of_squares) and multiply by the band energy now so we don't have to do it later
      float4 band_scale = _mm_div_ps(_mm_loadu_ps(&band_energy[j]), _mm_sqrt_ps(_mm_cvtepi32_ps(total)));

      float4 subbandq_lo = _mm_cvtepi32_ps(_mm_cvtepi16_epi32(qsub));
      float4 subband_lo = _mm_mul_ps(subbandq_lo, _mm_shuffle_ps_mine(band_scale, MY_SHUFFLE(0,0,1,1)));
      _mm_storeu_ps(&subband_energy[start], subband_lo);

      start += 4;
   }

   // this loop process bands with 2, 3, or 4 subbands, one band at a time
   for (; j < num_bands && num_subbands_for_band[j] <= 4; ++j) {
      int num = num_subbands_for_band[j];

      int16x8 mask = _mm_load_si128((int16x8 *) &subband_mask[num][0]);

      // load four subbands, and 0 the unused ones
      int16x8 qsub = _mm_and_si128(mask, _mm_loadl_epi64((int16x8*) &quantized_subbands[start]));

      // square the subbands and add pairs
      int32x4 pairsum = _mm_madd_epi16(qsub, qsub);

      // final add of reduction
      int32x4 total = _mm_add_epi32(pairsum, _mm_shuffle_epi32(pairsum, MY_SHUFFLE(1,0,3,2)));

      // compute the normalizing factor, 1/sqrt(sum_of_squares) and multiply by the band energy now so we don't have to do it later
      float4 band_scale = _mm_div_ps(_mm_load_ss(&band_energy[j]), _mm_sqrt_ps(_mm_cvtepi32_ps(total)));

      float4 subbandq_lo = _mm_cvtepi32_ps(_mm_cvtepi16_epi32(qsub));
      float4 subband_lo = _mm_mul_ps(subbandq_lo, _mm_shuffle_ps_mine(band_scale, MY_SHUFFLE(0,0,0,0)));

      // store all 4, as we'll overwrite that with next band
      _mm_storeu_ps(&subband_energy[start], subband_lo);

      start += num;
   }

   // this loop processes bands with up to 8 subbands, one band at a time
   for (; j < num_bands && num_subbands_for_band[j] <= 8; ++j) {
      int num = num_subbands_for_band[j];

      int16x8 mask = _mm_load_si128((int16x8 *) &subband_mask[num][0]);

      // load eight subbands, and 0 the unused ones
      int16x8 qsub = _mm_and_si128(mask, _mm_loadu_si128((int16x8*) &quantized_subbands[start]));

      // square the subbands and add pairs
      int32x4 pairsum = _mm_madd_epi16(qsub, qsub);

      // add adjacent pairs of 32-bit values
      int32x4 quadsum = _mm_add_epi32(pairsum, _mm_shuffle_epi32(pairsum, MY_SHUFFLE(1,0,3,2)));

      // final add of reduction
      int32x4 total = _mm_add_epi32(quadsum, _mm_shuffle_epi32(quadsum, MY_SHUFFLE(2,3,0,1)));

      // compute the normalizing factor, 1/sqrt(sum_of_squares) and multiply by the band energy now so we don't have to do it later
      float4 band_scale = _mm_div_ps(_mm_load_ss(&band_energy[j]), _mm_sqrt_ps(_mm_cvtepi32_ps(total)));

      float4 subbandq_lo = _mm_cvtepi32_ps(_mm_cvtepi16_epi32(qsub));
      float4 subbandq_hi = _mm_cvtepi32_ps(_mm_cvtepi16_epi32(_mm_shuffle_epi32(qsub, MY_SHUFFLE(2,3,0,1))));
      float4 subband_lo = _mm_mul_ps(subbandq_lo, _mm_shuffle_ps_mine(band_scale, MY_SHUFFLE(0,0,0,0)));
      float4 subband_hi = _mm_mul_ps(subbandq_hi, _mm_shuffle_ps_mine(band_scale, MY_SHUFFLE(0,0,0,0)));

      // store all 8, as we'll overwrite unused ones when we do next band
      _mm_storeu_ps(&subband_energy[start+0], subband_lo);
      _mm_storeu_ps(&subband_energy[start+4], subband_hi);

      start += num;
   }

   // this loop processes bands with 9..16 subbands
   for (; j < num_bands && num_subbands_for_band[j] >= 9 && num_subbands_for_band[j] <= 16; ++j) {
      int num = num_subbands_for_band[j];

      int16x8 mask = _mm_load_si128((int16x8 *) &subband_mask[num-8][0]);

      // load sixteen subbands, and 0 the unused ones
      int16x8 qsub0 =                     _mm_loadu_si128((int16x8*) &quantized_subbands[start+0]);
      int16x8 qsub1 = _mm_and_si128(mask, _mm_loadu_si128((int16x8*) &quantized_subbands[start+8]));

      // square the subbands and add pairs
      int32x4 pairsum0 = _mm_madd_epi16(qsub0, qsub0);
      int32x4 pairsum1 = _mm_madd_epi16(qsub1, qsub1);

      // we now have 8 sums in 2 registers, so let's add those vertically
      int32x4 quadsum = _mm_add_epi32(pairsum0, pairsum1);

      // add adjacent pairs of 32-bit values
      int32x4 octsum = _mm_add_epi32(quadsum, _mm_shuffle_epi32(quadsum, MY_SHUFFLE(1,0,3,2)));

      // final add of reduction, same value in all lanes
      int32x4 total = _mm_add_epi32(octsum, _mm_shuffle_epi32(octsum, MY_SHUFFLE(2,3,0,1)));

      // compute the normalizing factor, 1/sqrt(sum_of_squares) and multiply by the band energy now so we don't have to do it later
      float4 band_scale = _mm_div_ps(_mm_set1_ps(band_energy[j]), _mm_sqrt_ps(_mm_cvtepi32_ps(total)));

      float4 subbandq0 = _mm_cvtepi32_ps(_mm_cvtepi16_epi32(qsub0));
      float4 subbandq1 = _mm_cvtepi32_ps(_mm_cvtepi16_epi32(_mm_shuffle_epi32(qsub0, MY_SHUFFLE(2,3,0,1))));
      float4 subbandq2 = _mm_cvtepi32_ps(_mm_cvtepi16_epi32(qsub1));
      float4 subbandq3 = _mm_cvtepi32_ps(_mm_cvtepi16_epi32(_mm_shuffle_epi32(qsub1, MY_SHUFFLE(2,3,0,1))));

      float4 subband0 = _mm_mul_ps(subbandq0, band_scale);
      float4 subband1 = _mm_mul_ps(subbandq1, band_scale);
      float4 subband2 = _mm_mul_ps(subbandq2, band_scale);
      float4 subband3 = _mm_mul_ps(subbandq3, band_scale);

      // store all 16, as we'll overwrite unused ones when we do next band
      _mm_storeu_ps(&subband_energy[start+ 0], subband0);
      _mm_storeu_ps(&subband_energy[start+ 4], subband1);
      _mm_storeu_ps(&subband_energy[start+ 8], subband2);
      _mm_storeu_ps(&subband_energy[start+12], subband3);

      start += num;
   }

   // bands with more than 16 subbands are processed with scalar logic, also a final smaller band for 24khz
   for (; j < num_bands; ++j) {
      int sum=0;
      int num = num_subbands_for_band[j];
      for (int i=0; i < num; ++i)
         sum += (quantized_subbands[start+i] * quantized_subbands[start+i]);

      F32 scale = band_energy[j] / sqrtf((F32) sum);
      for (int i=0; i < num; ++i) {
         subband_energy[start+i] = scale * quantized_subbands[start+i];
      }
      start += num;
   }
}

// sum-of-squares across groups of 8 signed 8-bit coeffs in [-127,127]
// result for first 8 duplicated into lanes 0 and 1,
// for second 8 duplicated into lanes 2 and 3.
static RADFORCEINLINE int32x4 subband8_sum_squares(int8x16 coeff_8)
{
   // absolute value is <=127 (we don't allow -128)
   int8x16 abscoeff_8 = _mm_abs_epi8(coeff_8);

   // square them and add pairs, maximum result is 127*127 < 16384
   int16x8 pair_sums16 = _mm_maddubs_epi16(abscoeff_8, abscoeff_8);

   // horizontal add across groups of 4
   int32x4 quad_sums = _mm_madd_epi16(pair_sums16, _mm_set1_epi16(1));

   // add adjacent pairs
   int32x4 full_sums = _mm_add_epi32(quad_sums, _mm_shuffle_epi32(quad_sums, MY_SHUFFLE(1,0,3,2)));

   return full_sums;
}

// sum-of-squares across 16 signed 8-bit coeffs in [-127,127]
static RADFORCEINLINE int32x4 subband16_sum_squares(int8x16 coeff_8)
{
   // groups of 8 works exactly the same as before
   int32x4 oct_sums = subband8_sum_squares(coeff_8);

   // rotate by 32-bits and add, final add of reduction
   int32x4 full_sums = _mm_add_epi32(oct_sums, _mm_shuffle_epi32(oct_sums, MY_SHUFFLE(2,3,0,1)));

   return full_sums;
}

// compute coefficient normalization factor for subband from sum-of-squares in float.
static RADFORCEINLINE float4 subband_normalization_factor(float* energy_ptr, float4 sum_of_squares)
{
   // the coefficient normalization factor is the reciprocal of the square-root of the squared sum.
   // in other words, we are going to compute:
   //          coefficient * (1/sqrt(sum))
   //
   // but then after normalizing that, we then want to multiply it by the subband energy:
   //          coefficient * (1/sqrt(sum)) * energy
   //
   // so we simplify this to:
   //          coefficient * (energy/sqrt(sum))
   //
   // use scalar math here since everything is the same in all 4 lanes and this helps on smaller machines.
   float4 energy = _mm_load_ss(energy_ptr);
   float4 scale_factor = _mm_div_ss(energy, _mm_sqrt_ss(sum_of_squares));

   // broadcast the result
   return _mm_shuffle_ps_mine(scale_factor, MY_SHUFFLE(0,0,0,0));
}

// convert 8 8-bit coeffs in [-127,127] to float32, multiply by scale and output
static RADFORCEINLINE void subband8_output(float *coeffs, int8x16 coeff_8, float4 scale)
{
   int32x4 unpacked_set0_i = _mm_cvtepi8_epi32(coeff_8);
   int32x4 unpacked_set1_i = _mm_cvtepi8_epi32(_mm_srli_epi64(coeff_8, 32));

   float4  unpacked_set0   = _mm_cvtepi32_ps(unpacked_set0_i);
   float4  unpacked_set1   = _mm_cvtepi32_ps(unpacked_set1_i);

   float4 unquantized_set0 = _mm_mul_ps(unpacked_set0, scale);
   float4 unquantized_set1 = _mm_mul_ps(unpacked_set1, scale);

   _mm_storeu_ps(coeffs+0, unquantized_set0);
   _mm_storeu_ps(coeffs+4, unquantized_set1);
}

// convert 16 8-bit coeffs in [-127,127] to float32, multiply by scale and output
static RADFORCEINLINE void subband16_output(float *coeffs, int8x16 coeff_8, float4 scale)
{
   subband8_output(coeffs+0, coeff_8, scale);
   subband8_output(coeffs+8, _mm_shuffle_epi32(coeff_8, MY_SHUFFLE(2,3,0,1)), scale);
}

void radaudio_sse4_dequantize_long_block_8x8_Nx16(float *coeffs, S8 *quantized_coeff, float *subband_energy, int num_subbands)
{
   int cb=0;

   // first 8 subbands are always 8 coefficients long
   int j;
   for (j=0; j < 8; ++j) {

      //
      // compute the squared sum of all 8 coefficients in the subband so they can be normalized to 1
      //

      // load 8 signed-byte coefficients
      int8x16 coeff_8 = _mm_loadl_epi64((int8x16 *) &quantized_coeff[cb]);

      int32x4 oct_sums = subband8_sum_squares(coeff_8);
      float4 oct_sum_float = _mm_cvtepi32_ps(oct_sums);
      float4 subband_scale = subband_normalization_factor(&subband_energy[j], oct_sum_float);

      subband8_output(coeffs+cb, coeff_8, subband_scale);

      cb += 8;
   }

   // all remaining subbands are 16 coefficients long

   // tried doing four-subbands-at-a-time, but it was slightly slower
   for (; j < num_subbands; ++j) {

      //
      // compute the squared sum of all 16 coefficients in the subband so they can be normalized to 1
      //

      // load 16 signed-byte coefficients
      int8x16 coeff_8 = _mm_loadu_si128((int8x16 *) &quantized_coeff[cb]);

      int32x4 full_sums = subband16_sum_squares(coeff_8);
      float4 full_sum_float = _mm_cvtepi32_ps(full_sums);
      float4 subband_scale = subband_normalization_factor(&subband_energy[j], full_sum_float);

      subband16_output(coeffs+cb, coeff_8, subband_scale);

      cb += 16;
   }

   memset(&coeffs[cb], 0, 4*(RADAUDIO_LONG_BLOCK_LEN-cb));
}

void radaudio_sse4_dequantize_long_block_replace_0_with_random_8x8_Nx16(F32 *coeffs, S8 *quantized_coeff, F32 *subband_energy, int num_subbands, U32 rand_state_array[4])
{
   int j,cb;

   int8x16 randtab = _mm_setr_epi8(-1,1, -2,2, -3,3, -4,4, -5,5, -6,6, -7,7, -8,8 );

   int32x4 rand_state = _mm_loadu_si128((int32x4*) rand_state_array);

   // setup constants that update the first two lcg states, but not the second two, for consistency with scalar impl
   const int32x4 half_lcg_mul     = _mm_setr_epi32(LCG_MUL,LCG_MUL,1,1);
   const int32x4 half_lcg_add     = _mm_setr_epi32(LCG_ADD,LCG_ADD,0,0);
   const int8x16 nibble_mask      = _mm_set1_epi8(0x0f);

   // first 8 subbands are always 8 coefficients long
   j = 0;
   cb = 0;
   for (; j < 8; ++j) {

      // load 8 signed-byte coefficients
      int8x16 coeff_8 = _mm_loadl_epi64((int8x16 *) &quantized_coeff[cb]);

      // randomize all-zero coefficient sets
      if (RR_GET64_NATIVE(&quantized_coeff[cb+0]) == 0) {
         // get 4 bits of random state for each of 8 lanes (other 8 are "don't care")
         int8x16 rbits = _mm_and_si128(_mm_srli_epi16(rand_state, 4), nibble_mask);

         // load the coefficients from the random table
         coeff_8 = _mm_shuffle_epi8(randtab, rbits);

         // update the first two LCG streams
         rand_state = _mm_add_epi32(_mm_mullo_epi32(rand_state, half_lcg_mul), half_lcg_add);
      }

      //
      // compute the squared sum of all 8 coefficients in the subband so they can be normalized to 1
      //

      int32x4 oct_sums = subband8_sum_squares(coeff_8);
      float4 oct_sum_float = _mm_cvtepi32_ps(oct_sums);
      float4 subband_scale = subband_normalization_factor(&subband_energy[j], oct_sum_float);

      // output scaled coeffs
      subband8_output(coeffs+cb, coeff_8, subband_scale);

      cb += 8;
   }

   // all remaining subbands are 16 coefficients long

   const int32x4 lcg_mul     = _mm_set1_epi32(LCG_MUL);
   const int32x4 lcg_add     = _mm_set1_epi32(LCG_ADD);

   for (; j < num_subbands; ++j) {

      // load 16 signed-byte coefficients
      int8x16 coeff_8 = _mm_loadu_si128((int8x16 *) &quantized_coeff[cb]);

      //
      // if coefficients are all 0, replace with random values from -8..8 (but not 0,
      // because if we got unlucky and they ALL were 0, we'd divide by 0 later)
      //

      if ((RR_GET64_NATIVE(&quantized_coeff[cb+0]) | RR_GET64_NATIVE(&quantized_coeff[cb+8])) == 0)
      //if (_mm_test_all_zeros(coeff_8, coeff_8)) // slower on my machine (skylake)
      {
         // get the top 4 bits from each byte of random state
         int8x16 rbits = _mm_and_si128(_mm_srli_epi16(rand_state, 4), nibble_mask);

         // lookup in random_table[]
         int8x16 values = _mm_shuffle_epi8(randtab, rbits);

         // those are the coefficients now
         coeff_8 = values;

         // update the LCG streams
         rand_state = _mm_add_epi32(_mm_mullo_epi32(rand_state, lcg_mul), lcg_add);
      }

      //
      // compute the squared sum of all 16 coefficients in the subband so they can be normalized to 1
      //
      int32x4 full_sums = subband16_sum_squares(coeff_8);
      float4 full_sum_float = _mm_cvtepi32_ps(full_sums);
      float4 subband_scale = subband_normalization_factor(&subband_energy[j], full_sum_float);

      // output scaled coeffs
      subband16_output(coeffs+cb, coeff_8, subband_scale);

      cb += 16;
   }

   memset(&coeffs[cb], 0, 4*(RADAUDIO_LONG_BLOCK_LEN-cb));
}

void radaudio_sse4_dequantize_short_block_sse4(float *coeffs, S8 *quantized_coeff, float *band_energy, int num_bands, int *num_coeffs_for_band)
{
   int cb=0;
   int band=0;

   // in the following code, each chunk is independent, so big out-of-order CPUs
   // should crunch through it, so we just want to minimize uops, so minimize instructions

   ////////////////////////////////////////////////////////////////////////////////
   //
   // Process bands 0..3
   //
   // first 4 bands are 1 coefficient long, and coefficient is always 1 or -1
   //

   // for (j=0; j < 4; ++j) {
   //   coeffs[j] = (F32) quantized_coeff[j] * band_energy[j];
   // }

   {
      // @OPTIMIZE: maybe replace this logic with logic to flip the sign bit instead
      int8x16 coeff16 = _mm_cvtepi8_epi32(_mm_loadu_si128((int8x16 *) &quantized_coeff[cb]));
      float4 coeff4 = _mm_cvtepi32_ps(coeff16);
      float4 unquantized_band4 = _mm_mul_ps(coeff4, _mm_loadu_ps(&band_energy[band]));
      _mm_storeu_ps(&coeffs[cb], unquantized_band4);
      cb += 4;
      band += 4;
   }

   ////////////////////////////////////////////////////////////////////////////////
   //
   // Process bands 4..7
   //
   // next 4 bands are 2 coefficients each
   //

   // for (j=0; j < 4; ++j) {
   //   float x = (F32) quantized_coeff[4+j*2+0];
   //   float y = (F32) quantized_coeff[4+j*2+1];
   //   float scale = band_energy[4+j] / (F32) sqrt(x*x+y*y+1.e-20f);
   //   coeffs[4+j*2+0] = x*scale;
   //   coeffs[4+j*2+1] = y*scale;
   // }

   float4 epsilon = _mm_set1_ps(1.e-20f);

   {
      // load 8 coefficients
      int8x16 coeff8_8 = _mm_loadl_epi64((int8x16 *) &quantized_coeff[cb]);

      // convert to 16-bit
      int16x8 coeff8 = _mm_cvtepi8_epi16(coeff8_8);

      // square and add pairs
      int32x4 squared = _mm_madd_epi16(coeff8, coeff8);

      // convert to float
      float4 sum = _mm_cvtepi32_ps(squared);
      sum = _mm_add_ps(sum, epsilon);

      float4 band_scale = _mm_div_ps(_mm_loadu_ps(&band_energy[band]), _mm_sqrt_ps(sum));

      float4 quantized_lo    = _mm_cvtepi32_ps(_mm_cvtepi8_epi32(coeff8_8));
      float4 quantized_hi    = _mm_cvtepi32_ps(_mm_cvtepi8_epi32(_mm_shuffle_epi32(coeff8_8, MY_SHUFFLE(1,0,3,2))));
      float4 unquantized_lo  = _mm_mul_ps(quantized_lo, _mm_shuffle_ps_mine(band_scale, MY_SHUFFLE(0,0,1,1)));
      float4 unquantized_hi  = _mm_mul_ps(quantized_hi, _mm_shuffle_ps_mine(band_scale, MY_SHUFFLE(2,2,3,3)));

      _mm_storeu_ps(&coeffs[cb+0], unquantized_lo);
      _mm_storeu_ps(&coeffs[cb+4], unquantized_hi);

      cb += 8;
      band += 4;
   }

   ////////////////////////////////////////////////////////////////////////////////
   //
   // Process bands 8..12
   //
   // next 5 bands are 4 coefficients each
   //

   // for (j=8; j < 13; ++j) {
   //   float sum=1.e-20f, scale;
   //   for (i=0; i < 4; ++i) {
   //      float n = (F32) quantized_coeff[cb+i];
   //      sum += n*n;
   //   }
   //   scale = band_energy[j] / (F32) sqrt(sum);
   //   for (i=0; i < 4; ++i)
   //      coeffs[cb+i] = (F32) quantized_coeff[cb+i] * scale;
   //   cb += 4;
   // }

   {
      // load 16 unique coefficients
      int8x16 coeff16 = _mm_loadu_si128((int8x16 *) &quantized_coeff[cb]);

      // square and add adjacent pairs
      int8x16 abscoeff16 = _mm_abs_epi8(coeff16); // max abs value allowed is 127
      int16x8 pair_sums = _mm_maddubs_epi16(abscoeff16, abscoeff16); // pairs sum at most to 127*127*2 < 32768

      // one more horizontal sum and widen to 32-bit
      int32x4 quad_sum_i = _mm_madd_epi16(pair_sums, _mm_set1_epi16(1));

      // convert to float
      float4 quad_sum = _mm_cvtepi32_ps(quad_sum_i);
      quad_sum = _mm_add_ps(quad_sum, epsilon);

      float4 band_scale = _mm_div_ps(_mm_loadu_ps(&band_energy[band]), _mm_sqrt_ps(quad_sum));

      float4 coeff4_0 = _mm_mul_ps(_mm_shuffle_ps_mine(band_scale,MY_SHUFFLE(0,0,0,0)), _mm_cvtepi32_ps(_mm_cvtepi8_epi32(                  coeff16                      )));
      float4 coeff4_1 = _mm_mul_ps(_mm_shuffle_ps_mine(band_scale,MY_SHUFFLE(1,1,1,1)), _mm_cvtepi32_ps(_mm_cvtepi8_epi32(_mm_shuffle_epi32(coeff16, MY_SHUFFLE(1,2,3,0)))));
      float4 coeff4_2 = _mm_mul_ps(_mm_shuffle_ps_mine(band_scale,MY_SHUFFLE(2,2,2,2)), _mm_cvtepi32_ps(_mm_cvtepi8_epi32(_mm_shuffle_epi32(coeff16, MY_SHUFFLE(2,3,0,1)))));
      float4 coeff4_3 = _mm_mul_ps(_mm_shuffle_ps_mine(band_scale,MY_SHUFFLE(3,3,3,3)), _mm_cvtepi32_ps(_mm_cvtepi8_epi32(_mm_shuffle_epi32(coeff16, MY_SHUFFLE(3,0,1,2)))));

      _mm_storeu_ps(&coeffs[cb+ 0], coeff4_0);
      _mm_storeu_ps(&coeffs[cb+ 4], coeff4_1);
      _mm_storeu_ps(&coeffs[cb+ 8], coeff4_2);
      _mm_storeu_ps(&coeffs[cb+12], coeff4_3);

      // at most one more band that is 4 long -- currently all of them are, but low-sample-rate probably shouldn't
      if (num_coeffs_for_band[band] == 4) {
         cb += 16;
         band += 4;
         F32 sum = 0;
         for (int k=0; k < 4; ++k) {
            F32 n = (F32) quantized_coeff[cb+k];
            sum += n*n;
         }
         F32 scale = band_energy[band] / sqrtf(sum);
         coeffs[cb+0] = quantized_coeff[cb+0] * scale;
         coeffs[cb+1] = quantized_coeff[cb+1] * scale;
         coeffs[cb+2] = quantized_coeff[cb+2] * scale;
         coeffs[cb+3] = quantized_coeff[cb+3] * scale;

         cb += 4;
         band += 1;
      }
      rrAssert(num_coeffs_for_band[band] > 4);
   }

   // now we have either [16,16,16,32]
   //                 or [16,16,32,32] for lower sample rates, or maybe even
   //                    [16,16,16,16,32]

   while (num_coeffs_for_band[band] == 16) {
      // this is same code as the 16-subband code from long blocks

      //
      // compute the squared sum of all 16 coefficients in the band so they can be normalized to 1
      //

      // load 16 signed-byte coefficients
      int8x16 coeff_8 = _mm_loadu_si128((int8x16 *) &quantized_coeff[cb]);

      int32x4 full_sums = subband16_sum_squares(coeff_8);
      float4 full_sum_float = _mm_cvtepi32_ps(full_sums);
      full_sum_float = _mm_add_ps(full_sum_float, epsilon);

      float4 band_scale = subband_normalization_factor(&band_energy[band], full_sum_float);

      // output scaled coeffs
      subband16_output(coeffs+cb, coeff_8, band_scale);

      cb += 16;
      band += 1;
   }

   // @OPTIMIZE - but these are probably rare
   for (; band < num_bands; ++band) {
      int count = num_coeffs_for_band[band];
      F32 sum=1.e-20f,scale;
      for (int i=0; i < count; i += 4) {
         for (int k=0; k < 4; ++k) {
            F32 n = (F32) quantized_coeff[cb+i+k];
            sum += n*n;
         }
      }
      scale = band_energy[band] / (F32) sqrtf(sum);
      for (int i=0; i < count; i += 4) {
         for (int k=0; k < 4; ++k) {
            coeffs[cb+i+k] = quantized_coeff[cb+i+k] * scale;
         }
      }
      cb += count;
   }

   rrAssert(band == num_bands);

   memset(&coeffs[cb], 0, 4*(RADAUDIO_SHORT_BLOCK_LEN-cb));
}


// num_bytes is rounded up to the nearest multiple of 8
int radaudio_intel_popcnt_count_set_bits_read_multiple8_sentinel8(U8 *data, int num_bytes)
{
   U64 num=0;
   int padded_size = (num_bytes+7) & ~7;
   if (padded_size != num_bytes)
      RR_PUT64_NATIVE(&data[num_bytes], 0);

   #ifdef __RADX64__
   for (int i=0; i < padded_size; i += 8)
      num += __popcnt64(RR_GET64_NATIVE(&data[i]));
   #else
   for (int i=0; i < padded_size; i += 4)
      num += __popcnt(RR_GET32_NATIVE(&data[i]));
   #endif

   return (int) num;
}

// num is a multiple of 16
//
// save the floats to buffer, and return a restoring factor that turns the buffer back to floats
// avoid taking two passes as often as possible by generating an initial unsafe version
float radaudio_sse2_save_samples(S16 *buffer, const float *data, int num)
{
   float4 scale = _mm_set1_ps(32767.0f);
   float4 maxabs4 = _mm_set1_ps( 1.0f);
   float4 absmask = ps(_mm_set1_epi32(0x7fffffff));

   // scale the data by 'scale' and simultaneously find the max absolute value
   for (int i=0; i < num; i += 16) {
      // unroll to 8-at-a-time to write 16 bytes per iteration
      // then unroll once more to reduce dependency chains in min/max calculation
      float4 v0 = _mm_loadu_ps(data+i+ 0);
      float4 v1 = _mm_loadu_ps(data+i+ 4);
      float4 v2 = _mm_loadu_ps(data+i+ 8);
      float4 v3 = _mm_loadu_ps(data+i+12);

      float4 v0abs = _mm_and_ps(v0,absmask);
      float4 v1abs = _mm_and_ps(v1,absmask);
      float4 v2abs = _mm_and_ps(v2,absmask);
      float4 v3abs = _mm_and_ps(v3,absmask);
      float4 maxa01 = _mm_max_ps(v0abs,v1abs);
      float4 maxa23 = _mm_max_ps(v2abs,v3abs);
      float4 scaled0 = _mm_mul_ps(v0, scale);
      float4 scaled1 = _mm_mul_ps(v1, scale);
      float4 scaled2 = _mm_mul_ps(v2, scale);
      float4 scaled3 = _mm_mul_ps(v3, scale);
      float4 maxa0123 = _mm_max_ps(maxa01,maxa23);
      // we want to round() to integer:
      int32x4 int0   = _mm_cvtps_epi32(scaled0);
      int32x4 int1   = _mm_cvtps_epi32(scaled1);
      int32x4 int2   = _mm_cvtps_epi32(scaled2);
      int32x4 int3   = _mm_cvtps_epi32(scaled3);
      int16x8 pack0  = _mm_packs_epi32(int0,int1);
      int16x8 pack1  = _mm_packs_epi32(int2,int3);

      maxabs4 = _mm_max_ps(maxabs4, maxa0123);
      _mm_storeu_si128((int16x8 *) (buffer+i+0), pack0);
      _mm_storeu_si128((int16x8 *) (buffer+i+8), pack1);
   }

   float4 maxabs2 = _mm_max_ps(maxabs4, _mm_shuffle_ps_mine(maxabs4, MY_SHUFFLE(2,3,0,1)));
   float4 maxabs  = _mm_max_ps(maxabs2, _mm_shuffle_ps_mine(maxabs2, MY_SHUFFLE(1,0,3,2)));
   if (_mm_cvtss_f32(maxabs) > 1.0f)
   {
      // rarely taken
      scale = _mm_div_ps(scale, maxabs);
      for (int i=0; i < num; i += 8) {
         float4 v0      = _mm_loadu_ps(data+i+0);
         float4 v1      = _mm_loadu_ps(data+i+4);
         float4 scaled0 = _mm_mul_ps(v0, scale);
         float4 scaled1 = _mm_mul_ps(v1, scale);
         int32x4 int0   = _mm_cvtps_epi32(scaled0);
         int32x4 int1   = _mm_cvtps_epi32(scaled1);
         int16x8 pack0  = _mm_packs_epi32(int0,int1);
         _mm_storeu_si128((int16x8 *) (buffer+i+0), pack0);
      }
   }
   return 1.0f / _mm_cvtss_f32(scale);
}

#endif // #if DO_BUILD_SSE4
