// Copyright Epic Games Tools, LLC. All Rights Reserved.
#ifndef RADAUDIO_SSE_DECODE_H
#define RADAUDIO_SSE_DECODE_H

#include "radaudio_common.h"
#include "radaudio_sse.h"

#ifdef RADAUDIO_WRAP
#define WRAPPED_NAME(name) RR_STRING_JOIN(RADAUDIO_WRAP, name##_)

#define radaudio_sse4_compute_subband_energy_skip12_excess_read7      	WRAPPED_NAME(radaudio_sse4_compute_subband_energy_skip12_excess_read7)
#define radaudio_sse4_dequantize_long_block_8x8_Nx16           				WRAPPED_NAME(radaudio_sse4_dequantize_long_block_8x8_Nx16)
#define radaudio_sse4_dequantize_short_block_sse4                   		WRAPPED_NAME(radaudio_sse4_dequantize_short_block_sse4)
#define radaudio_sse4_randomize_long_block_8x8_Nx16                       	WRAPPED_NAME(radaudio_sse4_randomize_long_block_8x8_Nx16)
#define radaudio_sse4_dequantize_long_block_replace_0_with_random_8x8_Nx16  WRAPPED_NAME(radaudio_sse4_dequantize_long_block_replace_0_with_random_8x8_Nx16)
#define radaudio_ssse3_distribute_bitflag_coefficients_multiple16           WRAPPED_NAME(radaudio_ssse3_distribute_bitflag_coefficients_multiple16)
#define radaudio_intel_popcnt_count_set_bits_read_multiple8_sentinel8		WRAPPED_NAME(radaudio_intel_popcnt_count_set_bits_read_multiple8_sentinel8)
#endif

extern void   radaudio_sse4_compute_subband_energy_skip12_excess_read7(F32 *subband_energy, const F32 *band_energy, int num_bands, int *num_subbands_for_band, U16 *quantized_subbands);
extern void   radaudio_sse4_dequantize_long_block_8x8_Nx16(float *coeffs, S8 *quantized_coeff, float *subband_energy, int num_subbands);
extern void   radaudio_sse4_dequantize_short_block_sse4(float *coeffs, S8 *quantized_coeff, float *band_energy, int num_bands, int *num_coeffs_for_band);
extern void   radaudio_sse4_randomize_long_block_8x8_Nx16(S8 *quantized_coeff, U32 rand_state[4], int num_subbands);
extern void   radaudio_sse4_dequantize_long_block_replace_0_with_random_8x8_Nx16(F32 *coeffs, S8 *quantized_coeff, F32 *subband_energy, int num_subbands, U32 rand_state[4]);
extern void   radaudio_ssse3_distribute_bitflag_coefficients_multiple16(radaudio_cpu_features cpu, S8 *quantized_coeff, int num_coeff, U8 *nonzero_flagbits, S8 *nonzero_coeffs, int *pcur_nonzero_coeffs);
extern int    radaudio_intel_popcnt_count_set_bits_read_multiple8_sentinel8(U8 *data, int num_bytes);

#endif
