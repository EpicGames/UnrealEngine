// Copyright Epic Games Tools, LLC. All Rights Reserved.
////////////////////////////////////////////////////////////////////////////
//
// RADaudio is a new audio codec made by Epic Game Tools for use in games,
// optimized for fast SIMD decoding and decent quality (roughly similar to
// Vorbis).
//
// It is a classical MDCT-based codec with two block sizes, and it uses
// the Oodle Data huffman entropy coder to store data.

#include <stdio.h>

#define ENCODER_VERSION 0x00000000   // unstable debug version
//#define ENCODER_VERSION 0x01000001 // use this for first released version
#define HUFFMAN_ENCODE // get Huffman encode tables

#include <math.h>
#include <stdlib.h>
#include "radaudio_encoder.h"
#include "radaudio_encoder_internal.h"
#include "radaudio_encoder_sse.h"
#include "radaudio_encoder_neon.h"
#include "radaudio_common.h"
#include "radaudio_mdct.h"
#include "rrCore.h"
#include "radaudio_common.inl"


RR_COMPILER_ASSERT(sizeof(radaudio_encoder_state) <= sizeof(radaudio_encoder));
RR_COMPILER_ASSERT(sizeof(radaudio_stream_header) <= RADAUDIO_STREAM_HEADER_MAX);

#ifdef RADAUDIO_DEVELOPMENT
#define PROFILE_ZONES()        \
   PROF(encode_all)            \
   PROF(coefficients_sum)      \
   PROF(coefficients_1)        \
   PROF(coefficients_2)        \
   PROF(coefficients_3)        \
   PROF(coefficients_4)        \
   PROF(coefficients_5)        \
   PROF(coefficients_n8)       \
   PROF(coefficients_n16)      \
   PROF(coefficients_normalize)\
   PROF(bands)                 \
   PROF(huffman)               \
   PROF(analysis1)             \
   PROF(mdct)                  \
   PROF(window)                \
   PROF(subbands)              \
   PROF(varbits)               \
   PROF(transient_analysis)    \
   PROF(check_mono)            \
   PROF(total_count)

enum
{
   #define PROF(x) PROF_##x,
   PROFILE_ZONES()
   #undef PROF

   PROF__end
};

#define PROF_BEGIN(var)  es->profile_times[PROF_##var] -= rrGetTicks()
#define PROF_END(var)    es->profile_times[PROF_##var] += rrGetTicks()
#else
#define PROF_BEGIN(var)
#define PROF_END(var)
#endif

typedef struct
{
   U8 *bitstream; // dynamic array
   U32 pending_bits;
   int pending_bitcount;
   int capacity;
   int length;
   int error;
} radaudio_bit_encoder;

typedef struct
{
   radaudio_bit_encoder stream[3];  // HUFF3 encoder
   size_t total_bits;
} huff3_encoder;

static void encode_vbstream_init(radaudio_bit_encoder *e, U8 *buffer, int capacity)
{
   e->capacity = capacity;
   e->bitstream = buffer;
   e->pending_bits = 0;
   e->pending_bitcount = 0;
   e->length = 0;
   e->error = 0;
}

static void encode_vbstream_bits(radaudio_bit_encoder *e, U32 bits, int bitlength)
{
   rrAssert(bitlength < 32);
   if (e->pending_bitcount+bitlength >= 32) {
      // at most 4 bytes output at once
      if (e->length + 4 > e->capacity) {
         e->error = 1;
         return;
      }
      while (e->pending_bitcount >= 8) {
         e->bitstream[e->length++] = (U8) (e->pending_bits & 255);
         e->pending_bits >>= 8;
         e->pending_bitcount -= 8;
      }
   }
   bits &= ((1 << bitlength) - 1);
   e->pending_bits |= bits << e->pending_bitcount;
   e->pending_bitcount += bitlength;
}

static void encode_vbstream_huff(radaudio_bit_encoder *e, radaudio_huffman *h, U8 symbol)
{
   rrAssert(h->encode[symbol].length > 0);
   encode_vbstream_bits(e, h->encode[symbol].code, h->encode[symbol].length);
}

static void encode_vbstream_flush(radaudio_bit_encoder *e)
{
   if (e->length + 4 > e->capacity) {
      e->error = 1;
      return;
   }
   while (e->pending_bitcount > 0) {
      e->bitstream[e->length++] = (U8) (e->pending_bits & 255);
      e->pending_bits >>= 8;
      e->pending_bitcount -= 8;
   }
   e->pending_bitcount = 0;
   e->pending_bits = 0;
}

static void encode_bits(huff3_encoder *es, U32 bits, int bitlength)
{
   // putting extra bits in stream[0] increases the offset of stream[2]
   // putting extra bits in stream[1] interacts with bit reversal
   // so stream[2] is the best place to put non-huffman data, although we could also multi-stream it
   encode_vbstream_bits(&es->stream[2], bits, bitlength);
   es->total_bits += bitlength;
}

static void encode_huff(huff3_encoder *es, radaudio_huffman *h, U8 symbol, unsigned int stream_index)
{
   rrAssert(stream_index >= 0 && stream_index < 3);
   encode_vbstream_huff(&es->stream[stream_index], h, symbol);
}

static void encode_recompute_stats(huff3_encoder *es)
{
   es->total_bits =
      8 * (es->stream[0].length           + es->stream[1].length           + es->stream[2].length          )
        + (es->stream[0].pending_bitcount + es->stream[1].pending_bitcount + es->stream[2].pending_bitcount);
}

static void encode_huff_array(huff3_encoder *es, radaudio_huffman *huff, U8 *values, int num_values, char *file, int line, int *error)
{
   for (int i=0; i < num_values; ++i) {
      if (huff->encode[values[i]].length == 0)
         *error = 1;
      else
         encode_huff(es, huff, values[i], (unsigned int) i % 3);
   }
   encode_recompute_stats(es);
}

static float linear_remap(float x, float a, float b, float c, float d)
{
   return (x-a)/(b-a)*(d-c)+c;
}

// original was 140. @TODO: optimize for different sample rates and bit rates; no decoder change needed
#define TRANSIENT_THRESHOLD  180

static int silence_exp_start = -11;
static float silence_exp_value = 0.75f;

static float inverse_approx_pow_2(float x)
{
   // solve x = 0.345*y^2 + 0.655*y + 1
   float A = 0.34375f, B = 0.65625f, C = 1-x;
   float B2 = 0.4306640625f; // B*B, 441/1024
   float discr = B2 - (4*A)*C;
   float d = sqrtf(discr);
   float y = (-B + d) / (2*A);
   rrAssert(x >= 1 && x <= 2);
   return y;
}

static void mdct_block(radaudio_encoder_state *es, float *coeff, int is_short_block, float *samples, int offset, int len, int prev_short, int next_short, int stride, rrbool average, radaudio_encode_info *info, int pad_channel)
{
   PROF_BEGIN(window);
   int i;
   static int block_length[2] = { RADAUDIO_LONG_BLOCK_LEN, RADAUDIO_SHORT_BLOCK_LEN };
   FFT_ALIGN(F32, data[MAX_MDCT_SAMPLES]);
   FFT_ALIGN(F32, workspace[MAX_MDCT_SAMPLES]);
   F32 *window;

   // the window type is the shortest of the two blocks
   int windowleft  = RR_MAX(is_short_block, prev_short);
   int windowright = RR_MAX(is_short_block, next_short);
   int n = block_length[is_short_block];
   int wlen;

   int raw_start = offset - n;
   int raw_end   = offset + n;

   // clamp range to sample, which is defined from 0..len
   int clamped_start = RR_MAX(raw_start, 0);
   int clamped_end   = RR_MIN(raw_end  , len);

   // convert into space where 0 = start of window, i.e. raw_start maps to 0
   int convert_offset = raw_start;
   raw_start     -= convert_offset;
   raw_end       -= convert_offset;
   clamped_start -= convert_offset;
   clamped_end   -= convert_offset;

   rrAssert(raw_start == 0);
   rrAssert(raw_end   == 2*n);
   raw_start = 0;
   raw_end = 2*n;

   if (clamped_start > raw_start) {
      if (info->padding_len > 0) {
         // this could be turned into two loops, one for the padded samples and one for the 0s,
         // but since it only happens in one block at start of file, let's just leave it clearer here
         for (i=raw_start    ; i < clamped_start; ++i) {
            // map back into original sample space:
            //   if i= -convert_offset, then sample_pos is 0
            int sample_pos = i+convert_offset;
            //   now sample_pos from -padding_len to -1 maps to 0..padlen-1 in padding
            int padding_pos = sample_pos + (int) info->padding_len;
            if (padding_pos >= 0 && padding_pos < (int) info->padding_len)
               data[i] = info->padding[padding_pos*stride+pad_channel];
            else
               data[i] = 0;
         }
      } else {
         for (i=raw_start    ; i < clamped_start; ++i)
            data[i] = 0;
      }
   }
   if (average)
      for (i=clamped_start; i < clamped_end  ; ++i)  data[i] = (samples[(offset-n+i)*stride] + samples[(offset-n+i)*stride]) * 0.5f;
   else
      for (i=clamped_start; i < clamped_end  ; ++i)  data[i] = samples[(offset-n+i)*stride];

   #if 0
   if (clamped_start > raw_start) {
      for (i=0; i < raw_end; ++i) {
         printf("%4d %d\n", i, (int) (data[i]*32767));
      }
      printf("seamless fix for channel %d ends at %d\n", pad_channel, clamped_start);
   }
   #endif

   if (clamped_end < raw_end) {
      if (info->padding_len > 0) {
         for (i=clamped_end; i < raw_end; ++i) {
            // map back into original sample space:
            //   if i=raw_end, then sample_pos is len
            rrAssert(clamped_end + convert_offset == len);
            int sample_pos = i+convert_offset;
            //   now sample_pos from len..len+padlen-1 maps to 0..padlen-1 in padding
            int padding_pos = sample_pos - len;
            if (padding_pos >= 0 && padding_pos < (int) info->padding_len)
               data[i] = info->padding[padding_pos*stride+pad_channel];
            else
               data[i] = 0;
         }
         #if 0
         for (i=0; i < raw_end; ++i) {
            printf("%4d %d\n", i, (int) (data[i]*32767));
         }
         printf("seamless fix for channel %d starts at %d\n", pad_channel, clamped_end);
         #endif
      } else {
         for (i=clamped_end  ; i < raw_end      ; ++i)
            data[i] = 0;
      }
   }

   // apply left window
   wlen = block_length[windowleft];
   window = radaudio_windows[windowleft];
   if (wlen < n) {
      // long block with short window
      int wstart = RADAUDIO_LONG_BLOCK_LEN/2 - RADAUDIO_SHORT_BLOCK_LEN/2;
      for (i=0; i < wstart; ++i)
         data[i] = 0;
      for (i=0; i < RADAUDIO_SHORT_BLOCK_LEN; ++i)
         data[wstart+i] *= window[i];
   } else {
      for (i=0; i < wlen; ++i)
         data[i] *= window[i];
   }

   // apply right window
   wlen = block_length[windowright];
   window = radaudio_windows[windowright];
   if (wlen < n) {
      // long block with short window
      int wstart = RADAUDIO_LONG_BLOCK_LEN/2 - RADAUDIO_SHORT_BLOCK_LEN/2;
      for (i=0; i < RADAUDIO_SHORT_BLOCK_LEN; ++i)
         data[n+wstart+i] *= window[wlen-1-i];
      for (i=n-wstart; i < n; ++i)
         data[n+i] = 0;
   } else {
      for (i=0; i < wlen; ++i)
         data[n+i] *= window[wlen-1-i];
   }
   PROF_END(window);

   PROF_BEGIN(mdct);
   radaudio_mdct_fft(es->cpu, coeff, (size_t) n, data, data+n, workspace);
   PROF_END(mdct);
}

// computing error betweeen two vectors that are normalized to have L2-norm of 1.0,
// without explicitly computing normalized values
//
// Square error is:   sum[(normalizedq[i]    -   desired[i])^2]
//                  = sum[(normalizedq[i]^2  - 2*normalizedq[i]*desired[i] + desired[i]^2)]
//                  = sum[ normalizedq[i]^2] - 2*sum[normalizedq[i]*desired[i]]     + sum[desired[i]^2]
//                  =         1    - 2*normalize_scale * sum[unnormalizedq*desired] + 1
//                  =         2    - 2*normalize_scale * sum[unnormalizedq*desired]
// normalize_scale = 1/sqrt(sum(unnormalizedq^2))

static float compute_quantized_coefficients(S16 quantized_coeff[], F32 normalized_coeff[], int num_coeff, F32 quantizer)
{
   float cross_sum=0;
   float unnorm_sum2=0;
   for (int k=0; k < num_coeff; ++k) {
      float unquantized = (normalized_coeff[k] * quantizer);
      int     quantized = (int) floor(unquantized+0.5);
      quantized_coeff[k] = (S16) quantized;
      cross_sum += quantized * normalized_coeff[k];
      unnorm_sum2 += (F32) quantized * quantized;
   }
   F32 normalize_scale = 1.0f / sqrtf(unnorm_sum2);
   return 2 - 2 * normalize_scale * cross_sum;
}

// iterate through a range of quantizers and pick the one with the
// smallest error; all of them will in theory be smaller than the
// "ideal" quantizer, so we don't bother trading off size vs error,
// we just accept the best.
static void compute_best_quantized_coeff8_loop(radaudio_encoder_state *es, S16 best_quantized[], F32 best_so_far, F32 ncoeff[], F32 quantizer, F32 step_quantizer, int num_quantizers)
{
   RR_UNUSED_VARIABLE(es);
   int n = 8;
   for (int q=0; q < num_quantizers; ++q, quantizer += step_quantizer) {
      S16 quantized_attempt[8];

      F32 dot_product=0;
      F32 unnorm_sum2=0;
      for (int z=0; z < n; ++z) {
         float unquantized = (ncoeff[z] * quantizer);
         int     quantized = (int) floor(unquantized+0.5f);
         quantized_attempt[z] = (S16) quantized;
         dot_product += quantized * ncoeff[z];
         unnorm_sum2 += (F32) quantized * quantized;
      }
      float err2 = (dot_product * dot_product) / unnorm_sum2;
      if (err2 > best_so_far) {
         best_so_far = err2;
         memcpy(best_quantized, quantized_attempt, n * sizeof(best_quantized[0]));
      }
   }
}

static void compute_best_quantized_coeff16_loop(radaudio_encoder_state *es, S16 best_quantized[], F32 best_so_far, F32 ncoeff[], F32 quantizer, F32 step_quantizer, int num_quantizers)
{
   RR_UNUSED_VARIABLE(es);
   #ifdef DO_BUILD_SSE4
   if (es->cpu.has_sse4_1) {
      radaudio_sse2_compute_best_quantized_coeff16_loop(best_quantized, best_so_far, ncoeff, quantizer, step_quantizer, num_quantizers);
      return;
   }
   #endif

   #ifdef DO_BUILD_NEON
   radaudio_neon_compute_best_quantized_coeff16_loop(best_quantized, best_so_far, ncoeff, quantizer, step_quantizer, num_quantizers);
   return;
   #endif

   #ifndef DO_BUILD_NEON // for unreachable code warnings
   int n = 16;
   for (int q=0; q < num_quantizers; ++q, quantizer += step_quantizer) {
      S16 quantized_attempt[16];

      for (int z=0; z < n; ++z) {
         float unquantized = (ncoeff[z] * quantizer);
         int     quantized = (int) floor(unquantized+0.5f);
         quantized_attempt[z] = (S16) quantized;
      }

      // same operation order as in SIMD case to be bitwise identical
      F32 dot_product_arr[4];
      F32 unnorm_sum2_arr[4];
      for (int z = 0; z < 4; z++)
      {
          F32 q0 = (F32) quantized_attempt[0 + z];
          F32 q1 = (F32) quantized_attempt[4 + z];
          F32 q2 = (F32) quantized_attempt[8 + z];
          F32 q3 = (F32) quantized_attempt[12 + z];
          unnorm_sum2_arr[z] = 1.0e-20f; // avoid divide by 0
          unnorm_sum2_arr[z] += q0 * q0;
          unnorm_sum2_arr[z] += q1 * q1;
          unnorm_sum2_arr[z] += q2 * q2;
          unnorm_sum2_arr[z] += q3 * q3;
          dot_product_arr[z] = ncoeff[0 + z] * q0;
          dot_product_arr[z] += ncoeff[4 + z] * q1;
          dot_product_arr[z] += ncoeff[8 + z] * q2;
          dot_product_arr[z] += ncoeff[12 + z] * q3;
      }
      F32 unnorm_sum2 = (unnorm_sum2_arr[0] + unnorm_sum2_arr[2]) + (unnorm_sum2_arr[1] + unnorm_sum2_arr[3]);
      F32 dot_product = (dot_product_arr[0] + dot_product_arr[2]) + (dot_product_arr[1] + dot_product_arr[3]);

      float err2 = (dot_product * dot_product) / unnorm_sum2;
      if (err2 > best_so_far) {
         best_so_far = err2;
         memcpy(best_quantized, quantized_attempt, n * sizeof(best_quantized[0]));
      }
   }
   #endif // !DO_BUILD_NEON
}

static int get_rounded_exponent(float n)
{
#if 0
   float recip_log2 = 1.0f / logf(2.0f);
   int exp2 = (int) floor(logf(n) * recip_log2 + 0.5);
#else
   float two_to_one_half = 1.414213562373f;
   int exp=0;
   frexp(n * 0.5 * two_to_one_half, &exp);
   return exp;
#endif
}

static void compute_band_energies(radaudio_rate_info *info, float coeff[], int band_exponent[], F32 band_energy[])
{
   // compute band energy
   int k = 0;
   for (int j=0; j < info->num_bands; ++j) {
      float be = 0;
      for (int i=0; i < info->num_coeffs_for_band[j]; ++i) {
         float x = coeff[k++];
         be += x*x;
      }
      be = sqrtf(be) * info->band_scale_encode[j]; // average energy over whole band... this allows coding all bands identically
      if (be < 0.00002f)
         be = 0;
      band_energy[j] = be;

      if (be == 0) // no need to check minimum, as that gets 0'd as an exponent
         band_exponent[j] = BAND_EXPONENT_NONE;
      else {
         int exp = get_rounded_exponent(be);
         if (exp < -16) exp = BAND_EXPONENT_NONE;
         if (exp >   4) exp = 4;
         band_exponent[j] = exp;
      }
   }
}

// find the num_data/2'th item if data were sorted
// can reorder data

static int imin(int p0, int p1) { return p0 < p1 ? p0 : p1; }
static int imax(int p0, int p1) { return p0 > p1 ? p0 : p1; }

static int ifind_nth_slow(S16 *data, int count, int n)
{
   for (int i=0; i <= n; ++i) {
      S16 smallest = data[i];
      int smallest_j = i;
      for (int j=i+1; j < count; ++j) {
         if (data[j] < smallest) {
            smallest = data[j];
            smallest_j = j;
         }
      }
      // smallest_j belongs in i
      S16 t = data[i];
      data[i] = smallest;
      data[smallest_j] = t;
   }
   rrAssert(data[n] >= 0);
   return data[n];
}

RR_COMPILER_ASSERT(MAX_COEFF_PER_SUBBAND == 32);
static int ifind_nth_fast(radaudio_encoder_state *es, S16 *data, int count, int n)
{
   rrAssert(n <= MAX_COEFF_PER_SUBBAND);
   S16 temp1[MAX_COEFF_PER_SUBBAND],temp2[MAX_COEFF_PER_SUBBAND], *buf1 = temp1, *buf2 = temp2, *(buf[2]);
   buf[0] = buf1;
   buf[1] = buf2;
   int next_buf=0;

   buf1 = data;
   for(;;) {
      if (count <= 8) {
         int result;
         #ifdef DO_BUILD_SSE4
         if (es->cpu.has_sse4_1)
            result = radaudio_sse4_findnth_of_8_or_less(buf1, count, n);
         else
         #endif
         result = ifind_nth_slow(buf1, count, n);
         return result;
      }

      int v0 = data[0];
      int v1 = data[count-1];
      int v2 = data[count>>1];
      int lo       = imin(v0,v1);
      int hi       = imax(v0,v1);
      int clamp_lo = imax(lo,v2);
      int mid      = imin(hi,clamp_lo);

      // count numbers relative to mid
      int below=0;
      int above=MAX_COEFF_PER_SUBBAND;

      for (int i=0; i < count; ++i)
         rrAssert(buf1[i] >= 0);

      for (int i=0; i < count; ++i)
         if (buf1[i] < mid)
            buf2[below++] = buf1[i];
         else
            buf2[--above] = buf1[i];

      if (n < below) {
         if (below == count)
            return ifind_nth_slow(buf1, count, n);

         // is the nth item in the below section?
         buf1 = buf2;
         count = below;
      } else {
         if (below == 0)
            return ifind_nth_slow(buf1, count, n);

         buf1 = buf2+above;
         count = MAX_COEFF_PER_SUBBAND-above;
         n -= below;
      }
      buf2 = buf[next_buf];
      next_buf ^= 1;
   }
}

// only actually ever called with:
//      num_data= 4
//      num_data= 8
//      num_data=16
//      num_data=32
//
// 8 & 16 are the most important to optimize, since they're the only subband sizes for long blocks;
// the others come up with short blocks only
static float find_median(radaudio_encoder_state *es, F32 *data, int num_data)
{
   // approximate median finder using top 16 bits of floats, because SSE4.1 has a horizontal min we can leverage.
   // floats are all non-negative (they were already fabs()d), so ordering as integer is valid
   // unmeasurable change in PEAQ basic to use this instead of float median
   S32 idata[MAX_COEFF_PER_SUBBAND];
   S16 idata16[MAX_COEFF_PER_SUBBAND];
   memcpy(idata, data, 4*num_data);
   for (int i=0; i < num_data; ++i)
      idata16[i] = (S16) (idata[i] >> 16);
      // since sign bit is clear, we could use U16 and shift 1 less to keep 1 more mantissa bit
      // but we keep it signed in case of limitations in other SIMD implementations

   S32 result;

   #ifdef DO_BUILD_SSE4
   if (num_data == 8 && es->cpu.has_sse4_1) {
      result = radaudio_sse4_find_median_4th_of_8(idata16);
   } else if (num_data == 16 && es->cpu.has_sse4_1 && es->cpu.has_popcnt) {
      result = radaudio_sse4popcnt_find_median_8th_of_16(idata16);
   }
   else
   #elif defined(DO_BUILD_NEON)
   if (num_data == 8) {
      result = radaudio_neon_find_median_4th_of_8(idata16);
   } else if (num_data == 16) {
      result = radaudio_neon_find_median_8th_of_16(idata16);
   }
   else
   #endif
   {
      result = ifind_nth_fast(es, idata16, num_data, num_data>>1);
   }

   // most mantissa bits will be 0, but good enough; the median is a heuristic anyway
   result <<= 16;
   F32 int_median;
   memcpy(&int_median, &result, 4);  
   return int_median;
}

static void encode_channel(radaudio_encoder_state *es,
                           radaudio_block_data *bd,
                           F32 *coeff,
                           int channel,
                           F32 *sbb_energy,
                           U32 midside_bands,
                           radaudio_block_data *mid // only if side encoding
                           ) // mid-side encoding
{
   float band_pulses_raw[2][MAX_BANDS] = { 0 };

   int is_short_block = es->current_block_short;
   radaudio_rate_info *info = es->info[is_short_block];
   int i,j,k,s,c;
   U8 has_nonempty_subbands[24] = { 0 };
   RAD_ALIGN(int, band_exponent[24], 16);
   RAD_ALIGN(F32, band_energy[24], 16);
   RAD_ALIGN(F32, subband_energy[72], 16);

   bd->num_bands = info->num_bands;
   bd->num_quantized_subbands = info->num_quantized_subbands;
   bd->num_quantized_coeffs = info->num_quantized_coeffs;

   int blocksize = is_short_block ? RADAUDIO_SHORT_BLOCK_LEN : RADAUDIO_LONG_BLOCK_LEN;

   // ramp off the highest frequencies (if they're even used, only 24/32khz)
   int ramp_to_zero_length = 8;
   for (i=1; i < ramp_to_zero_length; ++i)
      coeff[blocksize-i] *= (float) i / ramp_to_zero_length;

   for (j=info->num_quantized_coeffs; j < info->num_coeffs; ++j)
      bd->quantized_coeff_encode[j] = 0;

   PROF_BEGIN(bands);
   compute_band_energies(info, coeff, band_exponent, band_energy);
   for (j=0; j < info->num_bands; ++j)
      bd->band_exponent[j] = band_exponent[j];

   compute_mantissa_bitcount(es->samprate_mode, is_short_block,
         es->mantissa_param,
         bd->band_exponent, bd->band_mantissa_bitcount);

   // compute fine band energy
   for (j=0; j < info->num_bands; ++j) {
      if (band_exponent[j] == BAND_EXPONENT_NONE)
         bd->band_mantissa[j] = 0;
      else {
         int nb = bd->band_mantissa_bitcount[j];
         if (nb == 0)
            bd->band_mantissa[j] = 0;
         else {
            float be = band_energy[j];
            int raw_fine;
            float x =  be / powf(2.0f, (F32) band_exponent[j]);
            rrAssert(x >= 0.70f && x <= 1.42f);
            x *= sqrtf(2.0f);
            if (x < 1) x = 1;
            if (x > 1.9999f) x = 1.9999f;
            x = inverse_approx_pow_2(x); // output is 0..1
            if (x < 0) x = 0;
            if (x > 0.9999f) x = 0.9999f;
            raw_fine = (int) (((1 << MAX_FINE_ENERGY_BITS)-1) * x + 0.5);
            bd->band_mantissa[j] = raw_fine >> (MAX_FINE_ENERGY_BITS-nb); // truncate to get the correct bucket index
         }
      }
   }
   PROF_END(bands);

   PROF_BEGIN(subbands);
   // subbands:
   if (es->current_block_short) {
      for (j=0; j < info->num_bands; ++j) {
         has_nonempty_subbands[j] = 1;
         subband_energy[j] = band_energy[j];
      }
   } else {
      // compute subband energy
      for (j=0; j < info->num_subbands; ++j) {
         float sbe = 0;
         int start = info->first_coeff_for_subband[j];
         for (i=0; i < info->num_coeffs_for_subband[j]; ++i) {
            float x = coeff[start+i];
            sbe += x*x;
         }
         sbe = sqrtf(sbe);
         subband_energy[j] = sbe;
      }

      for (j=0; info->num_subbands_for_band[j] == 1; ++j) {
         has_nonempty_subbands[j] = (band_exponent[j] != BAND_EXPONENT_NONE);
         bd->quantized_subbands[j] = (U16) 1;
      }

      for (; j < info->num_bands; ++j) {
         int count = info->num_subbands_for_band[j];
         int start = info->first_subband_for_band[j];
         int num_pulses = es->subband_predicted_sum[j];

         rrAssert(count != 1);
         if (num_pulses == 0 || band_exponent[j] == BAND_EXPONENT_NONE) {
            // skipping empty subbands doesn't really save bitrate on normal files,
            // but it helps enormously for silence and for mono-in-stereo (where the side channel is silent after mid-side encoding)
            // for empty bands, we don't save enough bitrate to be worth the complexity of skipping decoded subbands
            // instead output subbands that are optimal to decode
            float subband_pulses = (float) num_pulses / count;
            if (SUBBANDS_SKIP_EMPTY_BANDS)
               subband_pulses = floorf(subband_pulses);
            for (i=0; i < count; ++i)
               bd->quantized_subbands[start+i] = (U16) (subband_pulses + 0.5*(i & 1));
            has_nonempty_subbands[j] = false;
         } else {
            S32 maxsub=0;
            F32 sum = 1.0e-20f, inv_sum;
            for (i=0; i < count; ++i)
               sum += subband_energy[start+i];

            inv_sum = 1.0f / sum;
            for (i=0; i < count; ++i) {
               U16 val = (U16) ((num_pulses * (subband_energy[start+i]*inv_sum)+0.5));
               bd->quantized_subbands[start+i] = val;
               maxsub = RR_MAX(maxsub,val);
            }
            if (maxsub > 63) {
               // need to scale the subbands down so the largest value isn't too large to be signalled,
               // so remap so that maxsub will be 1 lower than limit (to allow rounding/slop)
               float revised_pulses = num_pulses * 62 / (float) maxsub;
               maxsub = 0;
               for (i=0; i < count; ++i) {
                  U16 val = (U16) ((revised_pulses * (subband_energy[start+i]*inv_sum)+0.5));
                  bd->quantized_subbands[start+i] = val;
                  maxsub = RR_MAX(maxsub,val);
               }
               rrAssert(maxsub <= 63);
               es->stats.block_events[E_subband_renormalize]++;
            }

            has_nonempty_subbands[j] = false;
            for (i=0; i < count; ++i)
               if (bd->quantized_subbands[start+i] != 0) {
                  has_nonempty_subbands[j] = true;
                  break;
               }

            int total=0;
            for (i=0; i < count; ++i)
               total += bd->quantized_subbands[start+i];
         }
      }
   }
   for (j=0; j < info->num_subbands; ++j)
      sbb_energy[j] = subband_energy[j];
   PROF_END(subbands);

   PROF_BEGIN(analysis1);
   // compute the number of pulses per band for two adjacent quality modes so we can interpolate between them
   float total_desired=0;
   float total_expected=0;
   {
      float weighting[MAX_BANDS] = { 0 };
      float total_weighting=1.0e-20f, recip_total;
      float base_pulses = es->heur.pulse_quality * 100;
      float num_pulses = (base_pulses * base_pulses * (is_short_block ? es->heur.short_block_pulse_scale : 1));

      if (!is_short_block) {
         if (es->prev_block_short && es->next_block_short)
            num_pulses *= es->heur.short_overlap_scale2;
         else if (es->prev_block_short || es->next_block_short)
            num_pulses *= es->heur.short_overlap_scale1;
      }

      for (j=0; j < info->num_bands; ++j) {
         float w;
         rrbool midside=false;
         int be = band_exponent[j];
         if (midside_bands & (1 << j)) {
            be = RR_MAX(be, mid->band_exponent[j]); // weight it as if it was the mid weight, if that's larger; later we'll discard samples based on actual exponent
            midside = true;
         }
         if (be == BAND_EXPONENT_NONE)
            w = 0;
         else {
            float exp = (float) be;
            w = powf(es->heur.band_exponent_base[is_short_block], exp);
         }

         total_expected += 1.0;
         if (be < -14) {
            if (be == -15) total_desired += 0.75f;
            if (be == -16) total_desired += 0.33f;
         } else
            total_desired += 1.0;

         w *= powf((F32) info->num_coeffs_for_band[j], (F32) es->heur.band_count_exponent[is_short_block]);

         #define RADA_LERP(t,a,b)  ((a)+(t)*((b)-(a)))
         w *= RADA_LERP((float) j/info->num_bands, es->heur.quality_weight_low[is_short_block], 1.0f);

         #if 0 // this makes 0.002 PEAQ difference even after tuning, not worth the time it would take to tune at all rates
         // inter-band masking
         if (j > 0 && j < info->num_bands-1 && !midside) {
            int exp = band_exponent[j];
            int quietest_neighboring_band_exp = RR_MIN(band_exponent[j-1], band_exponent[j+1]);
            if      (exp < quietest_neighboring_band_exp-5) w *= es->heur.band_mask_8[es->samprate_mode][is_short_block][quality];
            else if (exp < quietest_neighboring_band_exp-3) w *= es->heur.band_mask_4[es->samprate_mode][is_short_block][quality];
            else if (exp < quietest_neighboring_band_exp-2) w *= es->heur.band_mask_2[es->samprate_mode][is_short_block][quality];
            else if (exp < quietest_neighboring_band_exp-1) w *= es->heur.band_mask_1[es->samprate_mode][is_short_block][quality];
         }
         #endif
         if (info->num_coeffs_for_band[j] == 1)
            w = 0; // if only one coefficient, only needs one pulse to indicate sign

         weighting[j] = w;
         total_weighting += weighting[j];
      }
      recip_total = 1.0f/total_weighting;

      num_pulses *= total_desired / total_expected;

      for (j=0; j < info->num_bands; ++j) {
         float raw_pulses = num_pulses * weighting[j] * recip_total;

         float expectation = (band_exponent[j] - es->heur.expectation_base) * es->heur.expectation_scale;
         if (expectation < 0.0f) expectation = 0.0f;
         if (expectation > 1.0f) expectation = 1.0f;
         if (midside_bands & (1 << j)) {
            if (band_exponent[j] < mid->band_exponent[j]) {
               float scale1=1, scale2=1;

               // we weighted this as if it was the actual mid exponent, now throw away pulses to save space

               if (midside_bands == 0xffffffff) {
                  // if all bands mid-side

                  // phase 1: if exponents are separate by N, start decaying
                  // phase 2: if exponent is less than some threshold, start decaying

                  if (band_exponent[j] < mid->band_exponent[j] - es->heur.side_exp_threshold_all) {
                     if (band_exponent[j] <= mid->band_exponent[j] - es->heur.side_exp_start2_all)
                        scale1 = 0;
                     else
                        scale1 = linear_remap((float) band_exponent[j], (float) mid->band_exponent[j]-es->heur.side_exp_start2, (float) mid->band_exponent[j]-es->heur.side_exp_threshold, 0.0f, 1.0);
                  }
                  if (band_exponent[j] < es->heur.side_exp_end_all) {
                     if (band_exponent[j] <= es->heur.side_exp_start_all)
                        scale2 = 0;
                     else
                        scale2 = linear_remap((float) band_exponent[j], (float) es->heur.side_exp_start, (float) es->heur.side_exp_end, 0.0f, 1.0);
                  }
               } else {
                  // if selected bands, which uses a different detector so can have different decay rules

                  // phase 1: if exponents are separate by N, start decaying
                  // phase 2: if exponent is less than some threshold, start decaying

                  if (band_exponent[j] < mid->band_exponent[j] - es->heur.side_exp_threshold) {
                     if (band_exponent[j] <= mid->band_exponent[j] - es->heur.side_exp_start2)
                        scale1 = 0;
                     else
                        scale1 = linear_remap((float) band_exponent[j], (float) mid->band_exponent[j]-es->heur.side_exp_start2, (float) mid->band_exponent[j]-es->heur.side_exp_threshold, 0.0f, 1.0);
                  }
                  if (band_exponent[j] < es->heur.side_exp_end) {
                     if (band_exponent[j] <= es->heur.side_exp_start)
                        scale2 = 0;
                     else
                        scale2 = linear_remap((float) band_exponent[j], (float) es->heur.side_exp_start , (float) es->heur.side_exp_end, 0.0f, 1.0);
                  }
               }

               if (scale1 > 1.00) scale1=1.00;
               if (scale1 < 0.00) scale1=0.00;
               if (scale2 > 1.00) scale2=1.00;
               if (scale2 < 0.00) scale2=0.00;

               float scale = RR_MIN(scale1,scale2);

               expectation *= scale;
            }
         }

         if (band_exponent[j] < -9) {
            float scale = linear_remap((float) band_exponent[j], -15.0f , (float) silence_exp_start,
                                                                   0.05f, (float) silence_exp_value);
            if (scale > 1.00) scale=1.00;
            if (scale < 0.00) scale=0.00;
            if (band_exponent[j] == -16)
               scale = 0;
            expectation *= scale;
         }
         raw_pulses *= expectation;

         band_pulses_raw[0][j] = raw_pulses;
      }
   }
   PROF_END(analysis1);

   PROF_BEGIN(coefficients_sum);
   c=s=0; // c = coefficient index, s = subband index
   for (j=0; j < info->num_bands; ++j) {
      float recip;
      int num_pulses;

      PROF_BEGIN(coefficients_1);
      if (band_exponent[j] == BAND_EXPONENT_NONE)
         num_pulses = 0;
      else if (info->num_coeffs_for_band[j] == 1)
         num_pulses = 1; // just need the sign
      else {
         float base_pulses = band_pulses_raw[0][j];
         num_pulses = (int) (base_pulses * 44100.0 / es->sample_rate);
      }

      {
         float sum = 1.0e-12f;
         for (i=0; i < info->num_subbands_for_band[j]; ++i) {
            float x = (float) subband_energy[s+i];
            sum += x*x;
         }
         recip = 1.0f/sum;
      }
      PROF_END(coefficients_1);

      for (i=0; i < info->num_subbands_for_band[j]; ++i) {
         PROF_BEGIN(coefficients_2);

         float x = (float) subband_energy[s+i];
         float data[32], median;
         int n = info->num_coeffs_for_subband[s+i];

         // allocate the pulses per subband, based on squared energy
         int sub_pulses = (int) (num_pulses * x*x * recip+0.5);

         // check if we need to boost the pulses because lots of large coefficients
         if (band_exponent[j] >= -13 && n >= 8) {

            // find the median
            #ifdef DO_BUILD_SSE4x
            if (es->cpu.has_sse2)
               radaudio_sse2_fabs_coefficients(data, coeff+c, n);
            else
            #endif
            {
               for (k=0; k < n; ++k)
                  data[k] = fabsf(coeff[c+k]);
            }
            median = find_median(es, data, n);

            // count how many coefficients are significantly above the median
            for (k=0; k < n; ++k) {
               if (data[k] > median * es->heur.large_boost_median_test[is_short_block])
                  sub_pulses += 2;
               else if (data[k] > median * es->heur.small_boost_median_test[is_short_block])
                  sub_pulses += 1;
            }
         }
         PROF_END(coefficients_2);

         /// now distribute the pulses to the coefficients
         {
            PROF_BEGIN(coefficients_3);
            rrbool no_pulses = false;
            if (sub_pulses == 0)
               no_pulses = true;
            if (band_exponent[j] == BAND_EXPONENT_NONE)
               no_pulses = true;

            if (!is_short_block)
               // if the subband is going to get 0 energy, then force all coefficients to 0
               if (info->num_subbands_for_band[j] > 1 && bd->quantized_subbands[s+i]==0 && has_nonempty_subbands[j])
                  no_pulses = true;

            if (no_pulses) {
               for (k=0; k < n; ++k)
                  bd->quantized_coeff_encode[c+k] = 0;
               PROF_END(coefficients_3);
            } else if (info->num_coeffs_for_subband[s+i] == 1) {
               // subband has only one coefficient, so just need the sign
               bd->quantized_coeff_encode[c] = coeff[c] < 0 ? -1 : 1;
               PROF_END(coefficients_3);
            } else {
               PROF_END(coefficients_3);
               PROF_BEGIN(coefficients_normalize);
               float ncoeff[MAX_COEFF_PER_BAND], sum2=0;
               for (k=0; k < n; ++k) {
                  sum2 += coeff[c+k]*coeff[c+k];
               }
               rrAssert(sum2 != 0);
               float sum=0;
               {
                  float scale = 1.0f / (sqrtf(sum2) + 1.e-24f);
                  for (k=0; k < n; ++k) {
                     float v = scale * coeff[c+k];
                     ncoeff[k] = v;
                     sum += fabsf(v);
                  }
               }
               rrAssert(sum != 0); // if coefficients were all 0, then subband should have been all 0, so no_pulses should have been true

               S16 best_coeff[MAX_COEFF_PER_BAND];
               memset(best_coeff, 0, n*sizeof(best_coeff[0]));

               // this rather arbitrary computation must be used as is,
               // attempts to alter it to non-integer values always lead
               // to significant quality loss
               float t0,t1;
               float pc = (float) sub_pulses;

               t0 = (F32) (int) (pc/1.30f + 0.5);
               t1 = (F32) (int) (pc*1.125f + 0.5);

               //if (t1 > pc+n)  t1 = (F32) pc+n;

               if (t1 > pc+8) t1 = pc+8;
               if (t0 < pc-8) t0 = pc-8;

               if (t1 > pc+n/2) t1 = pc+n/2;
               if (t0 < pc-n)   t0 = pc-n;

               // the squared error if you use 0 for all coefficients is 1, since squared error vs. 0 is same as squared sum
               float error2_for_zeroes = 1.0f;

               // but if we transmit all 0s, the decoder replaces with noise, i.e. with random coefficients.
               // the RMSE error due to using random coefficients will be worse, since when the signs mismatch,
               // the error will be even larger, up to (say) 2x larger, and squared that's 4x larger, and signs
               // mismatch half the time.
               //
               // BUT we know the perceptual error from randomness is LESS than using all 0s. so using any error
               // estimate that is LARGER than the error for all zeroes will be perceptually wrong. so we tune
               // this value. maybe should be per-band:
               float min_error2 = error2_for_zeroes / 2;  // best value experimentlaly
               PROF_END(coefficients_normalize);

               PROF_BEGIN(coefficients_4);
               float best_so_far = min_error2;
               
               // optimized error calculation avoids a sqrt() by computing this derived, monotonically consistent error instead:
               // see compute_quantized_coefficients() for naive version
               best_so_far -= 2;
               best_so_far *= best_so_far;
               best_so_far /= 4;

               int num_steps = (int) (t1-t0) + 1;
               float recip_sum = 1.0f / sum;
               float quantizer = t0 * recip_sum;
               float step_quantizer = recip_sum;

               if (n == 8) {
                  PROF_END(coefficients_4);
                  PROF_BEGIN(coefficients_n8);
                  compute_best_quantized_coeff8_loop(es, best_coeff, best_so_far, ncoeff, quantizer, step_quantizer, num_steps);
                  PROF_END(coefficients_n8);
               } else if (n == 16) {
                  PROF_END(coefficients_4);
                  PROF_BEGIN(coefficients_n16);
                  compute_best_quantized_coeff16_loop(es, best_coeff, best_so_far, ncoeff, quantizer, step_quantizer, num_steps);
                  PROF_END(coefficients_n16);
               } else {
                  for (int q=0; q < num_steps; ++q, quantizer += step_quantizer) {
                     S16 quantized_attempt[MAX_COEFF_PER_SUBBAND];

                     F32 cross_sum=0;
                     F32 unnorm_sum2=0;
                     for (int z=0; z < n; ++z) {
                        float unquantized = (ncoeff[z] * quantizer);
                        int     quantized = (int) floor(unquantized+0.5f);
                        quantized_attempt[z] = (S16) quantized;
                        cross_sum += quantized * ncoeff[z];
                        unnorm_sum2 += (F32) quantized * quantized;
                     }
                     // this optimized computation resembles normalizing and computing a dot product
                     float err2 = (cross_sum*cross_sum) / unnorm_sum2;
                     if (err2 > best_so_far) {
                        best_so_far = err2;
                        memcpy(best_coeff, quantized_attempt, n * sizeof(best_coeff[0]));
                     }
                  }
                  PROF_END(coefficients_4);
               }

               PROF_BEGIN(coefficients_5);
               // check if coefficients are too large
               int largest = 0;
               for (k=0; k < n; ++k)
                  largest = RR_MAX(largest, abs(best_coeff[k]));

               if (largest > 112) {
                  ++es->stats.block_events[E_coefficients_renormalize];

                  float flargest = 0.00001f;
                  for (k=0; k < n; ++k)
                     if (fabsf(ncoeff[k]) > flargest)
                        flargest = fabsf(ncoeff[k]);

                  int target = 112;
                  float scale = target / flargest; // scale * flargest = target
                  for (k=0; k < n; ++k) {
                     best_coeff[k] = (S16) floorf(ncoeff[k] * scale + 0.5f);
                  }
               }

               for (k=0; k < n; ++k)
                  bd->quantized_coeff_encode[c+k] = (S16) best_coeff[k];

               PROF_END(coefficients_5);
            }
         }
         c += info->num_coeffs_for_subband[s+i];
      }
      s += info->num_subbands_for_band[j];
   }
   PROF_END(coefficients_sum);
}

static int transient_analysis(F32 *in, int N, int stride, F32 threshold);

static int transient_analysis_wrapper(F32 *in, int offset, int stride, F32 threshold)
{
   int result = transient_analysis(in+offset*stride, 1024, stride, threshold);
   return result;
}

static int stereo_count_effective_channels(float *stereo_input, size_t input_len, size_t offset, int num_samples, int mono_detection_aggressiveness)
{
   float threshold = ((F32) mono_detection_aggressiveness+0.5f) / 32768.0f;

   // clamp range (out of range values are treated 0, so always match mono
   size_t begin = (offset < (size_t)num_samples) ? 0 : offset-num_samples;
   size_t end   = (offset+num_samples > input_len) ? input_len : offset+num_samples;

   for (size_t i=begin; i < end; ++i)
      // if signals deviate enough, it's stereo
      if (fabsf(stereo_input[i*2+0] - stereo_input[i*2+1]) > threshold)
         return 2;

   // otherwise it's mono
   return 1;
}

RADDEFFUNC int radaudio_determine_preferred_next_block_length(radaudio_encoder *rae,
                                      radaudio_blocktype firsttype,
                                      F32 *input,
                                      size_t input_len,
                                      size_t offset)
{
   int cur_short;
   radaudio_encoder_state *es = (radaudio_encoder_state *) rae;

   if (es->block_number == 0)
      cur_short = (firsttype == RADAUDIO_BLOCKTYPE_short);
   else
      cur_short = es->current_block_short;
   int num_samples = cur_short ? RADAUDIO_SHORT_BLOCK_LEN : RADAUDIO_LONG_BLOCK_LEN;

   if (offset + num_samples + RADAUDIO_LONG_BLOCK_LEN >= input_len)
      return RADAUDIO_BLOCKTYPE_short;
   else {
      int stride = es->num_channels;
      if (transient_analysis_wrapper(input, (int) offset + num_samples, stride, TRANSIENT_THRESHOLD))
         return RADAUDIO_BLOCKTYPE_short;
      if (es->num_channels == 2) {
         if (transient_analysis_wrapper(input+1, (int) offset + num_samples, stride, TRANSIENT_THRESHOLD))
            return RADAUDIO_BLOCKTYPE_short;
      }
   }
   return RADAUDIO_BLOCKTYPE_long;
}

RADDEFFUNC radaudio_blocktype radaudio_determine_preferred_first_block_length(radaudio_encoder *rae,
                                      F32 *input,
                                      size_t input_len)
{
   radaudio_encoder_state *es = (radaudio_encoder_state *) rae;
   size_t offset=0;
   if (offset + RADAUDIO_LONG_BLOCK_LEN >= input_len)
      return RADAUDIO_BLOCKTYPE_short;
   else {
      if (transient_analysis_wrapper(input, (int) offset, es->num_channels, TRANSIENT_THRESHOLD))
         return RADAUDIO_BLOCKTYPE_short;
      if (es->num_channels == 2) {
         if (transient_analysis_wrapper(input+1, (int) offset, es->num_channels, TRANSIENT_THRESHOLD))
            return RADAUDIO_BLOCKTYPE_short;
      }
   }
   return RADAUDIO_BLOCKTYPE_long;
}

RADDEFFUNC int radaudio_encode_block(radaudio_encoder *rae,
                                      float *input,
                                      size_t input_len, // in samples (stereo pairs count as one)
                                      size_t *poffset  , // in samples (stereo pairs count as one)
                                      U8 *encode_buffer,  // recommend MAX_ENCODED_BLOCK_SIZE
                                      size_t encode_buffer_size)
{
   radaudio_encode_info info = { 0 };
   return radaudio_encode_block_ext(rae, input, input_len, poffset, encode_buffer, encode_buffer_size, &info);
}

static int compute_rle_length(radaudio_encoder_state *es, radaudio_block_data bd[2], int out_channels, int start, int *numsym)
{
   radaudio_rate_info *bi = es->info[0];
   int numbits=0;
   int k=0;
   int syms=0;
   for (int c=0; c < out_channels; ++c) {
      k = 0;
      for (int i=start; i < bi->num_quantized_coeffs; ++i) {
         int bits = bd[c].quantized_coeff_encode[i];
         if (bits == 0)
            ++k;
         else {
            // new zero-run-length encoding
            int zr = k;

            while (zr >= MAX_RUNLEN) {
               numbits += rada_zero_runlength_huff.encode[MAX_RUNLEN].length;
               zr -= MAX_RUNLEN;
               ++syms;
            }
            if (zr >= COARSE_RUNLEN_THRESHOLD) {
               int coarse = zr & ~3;
               numbits += rada_zero_runlength_huff.encode[coarse].length;
               ++syms;
               numbits += 2;
            } else {
               numbits += rada_zero_runlength_huff.encode[zr].length;
               ++syms;
            }

            k = 0;
         }
      }
      if (c == 0 && out_channels == 2) {
         numbits += rada_zero_runlength_huff.encode[END_OF_ZERORUN].length;
         ++syms;
      }
   }
   if (numsym != NULL)
      *numsym = syms;

   return numbits;
}

// returns number of bytes encoded;
// @TODO: reverse the below, 0 should be "done" and -1 should be "too small"
// returns 0 if output buffer isn't big enough;
// returns -1 if stream is done
// returns -2 on internal error
int radaudio_encode_block_ext(radaudio_encoder *rae,
                                      F32 *input,
                                      size_t input_len,
                                      size_t *poffset,
                                      U8 *encode_buffer,
                                      size_t encode_buffer_max,
                                      radaudio_encode_info *info)
{
   int force_first_block = -1;
   int force_next_block = -1;
   radaudio_encoder_state *es = (radaudio_encoder_state *) rae;
   size_t offset = *poffset;

   // 'offset' is the middle of the region we're going to encode, and also
   // the offset of the samples we will fully encode when we finish encoding this block

   PROF_BEGIN(encode_all);

   radaudio_rate_info *bi;

   RAD_ALIGN(radaudio_block_data, bd[2], 16);
   int num_samples;

   if (info->force_first_blocktype == RADAUDIO_BLOCKTYPE_short)
      force_first_block = 1;
   else if (info->force_first_blocktype == RADAUDIO_BLOCKTYPE_long)
      force_first_block = 0;

   if (info->force_next_blocktype == RADAUDIO_BLOCKTYPE_short)
      force_next_block = 1;
   else if (info->force_next_blocktype == RADAUDIO_BLOCKTYPE_long)
      force_next_block = 0;

   // do transient analysis on the current block
   if (es->block_number == 0) {
      if (force_first_block >= 0)
         es->current_block_short = force_first_block;
      else {
         if (offset + RADAUDIO_LONG_BLOCK_LEN >= input_len)
            es->current_block_short = 1;
         else {
            PROF_BEGIN(transient_analysis);
            es->current_block_short = (U8) transient_analysis_wrapper(input, (int) offset, es->num_channels, TRANSIENT_THRESHOLD);
            if (es->num_channels == 2) {
               if (transient_analysis_wrapper(input+1, (int) offset, es->num_channels, TRANSIENT_THRESHOLD))
                  es->current_block_short = 1;
            }
            PROF_END(transient_analysis);
         }
      }
   }

   num_samples = es->current_block_short ? RADAUDIO_SHORT_BLOCK_LEN : RADAUDIO_LONG_BLOCK_LEN;

   if (force_next_block >= 0)
      es->next_block_short = force_next_block;
   else {
      // we need to lookahead to the NEXT block to know how to window the right side of our block
      //
      // so our current block is [offset-num_samples, off+num_samples)
      // and the next block will be centered at offset+num_samples, and
      // if it's LONG_BLOCK it will be [offset+num_samples-LONG_BLOCK, offset+num_samples+LONG_BLOCK)
      if (offset + num_samples + RADAUDIO_LONG_BLOCK_LEN >= input_len)
         es->next_block_short = 1;
      else {
         int stride = es->num_channels;
         es->next_block_short = (U8) transient_analysis_wrapper(input, (int) offset + num_samples, stride, TRANSIENT_THRESHOLD);
         if (es->num_channels == 2) {
            if (transient_analysis_wrapper(input+1, (int) offset + num_samples, stride, TRANSIENT_THRESHOLD))
               es->next_block_short = 1;
         }
      }
   }

   // if the leftmost sample we would encode is off the end of the input, we're done
   if (offset >= input_len + RADAUDIO_SHORT_BLOCK_LEN) {
      PROF_END(encode_all);
      return RADAUDIOENC_AT_EOF;
   }

   // always end with a short block, to avoid weird overlap rules. actually, two short blocks
   if (offset >= input_len + RADAUDIO_SHORT_BLOCK_LEN*2 + RADAUDIO_LONG_BLOCK_LEN)
      es->next_block_short = 1;

   bi = es->info[es->current_block_short];

   PROF_BEGIN(check_mono);
   int num_channels = es->num_channels;
   int out_channels = num_channels == 1 ? 1 : stereo_count_effective_channels(input, input_len, offset, num_samples, 12);
   if (out_channels == 1 && es->num_channels == 2)
      ++es->stats.block_events[E_stereo_as_mono];
   PROF_END(check_mono);

   F32 subband_energy[2][256];

   rrbool do_mid_side = false;
   rrbool do_mid_side_bands = false;

   U32 mid_side_bands=0; // set bit (1<<j) for band j if it's mid-side encoded

   int error = 0;

   if (out_channels == 1) {
      FFT_ALIGN(F32, coeff[MAX_COEFFS]);
      mdct_block(es, coeff, es->current_block_short, input, (int) offset, (int) input_len, es->prev_block_short, es->next_block_short, es->num_channels, es->num_channels==2, info, 0);
      encode_channel(es, &bd[0], coeff, 0, subband_energy[0], 0, NULL);
   } else {
      FFT_ALIGN(F32, coeff1[MAX_COEFFS]);
      FFT_ALIGN(F32, coeff2[MAX_COEFFS]);
      mdct_block(es, coeff1, es->current_block_short, input  , (int) offset, (int) input_len, es->prev_block_short, es->next_block_short, es->num_channels, false, info, 0);
      mdct_block(es, coeff2, es->current_block_short, input+1, (int) offset, (int) input_len, es->prev_block_short, es->next_block_short, es->num_channels, false, info, 1);
      if (es->allow_mid_side) {
         // mid-side encoding intentionally encodes the side channel with less accuracy,
         // so we should only use it if the side is significantly quieter than the mid
         FFT_ALIGN(F32, coeff_mid [MAX_COEFFS]);
         FFT_ALIGN(F32, coeff_side[MAX_COEFFS]);
         for (int i=0; i < bi->num_coeffs; ++i) {
            coeff_mid [i] = (coeff1[i] + coeff2[i]) * 0.5f;
            coeff_side[i] = (coeff1[i] - coeff2[i]) * 1.0f;
         }
         RAD_ALIGN(F32, band_energy_dummy [MAX_BANDS], 16);
         RAD_ALIGN(int, band_exponent_mid [MAX_BANDS], 16);
         RAD_ALIGN(int, band_exponent_side[MAX_BANDS], 16);
         compute_band_energies(bi, coeff_mid , band_exponent_mid , band_energy_dummy);
         compute_band_energies(bi, coeff_side, band_exponent_side, band_energy_dummy);
         int tiny=0, smaller=0, much_smaller=0, larger=0;
         for (int j=0; j < bi->num_bands; ++j) {
            if (band_exponent_side[j] <= es->heur.mid_side_tiny)
               ++tiny;
            else if (band_exponent_side[j] <= band_exponent_mid[j]+es->heur.mid_side_offset || (band_exponent_side[j] < band_exponent_mid[j] && band_exponent_mid[j] <= es->heur.mid_side_threshold))
               ++much_smaller;
            else if (band_exponent_side[j] <  band_exponent_mid[j])
               ++smaller;
            else
               ++larger;
         }

         if (larger == 0 && smaller < es->heur.mid_side_max_bad_bands) {
            // most are tiny or much_smaller
            do_mid_side = true;
            encode_channel(es, &bd[0], coeff_mid , 0, subband_energy[0], 0, NULL);
            encode_channel(es, &bd[1], coeff_side, 1, subband_energy[1], 0xffffffff, &bd[0]);
         } else if (!es->current_block_short) {
            // consider doing it per-band
            int do_band[MAX_BANDS] = { 0 };
            for (int j=0; j < bi->num_bands; ++j) {
               if (band_exponent_side[j] < es->heur.side_exp_end) // mid_side_tiny)
                  do_band[j] = 2;
               else if (band_exponent_side[j] < band_exponent_mid[j]-es->heur.side_exp_threshold) // + mid_side_offset || (band_exponent_side[j] < band_exponent_mid[j] && band_exponent_mid[j] <= mid_side_threshold))
                  do_band[j] = 2;
               else if (band_exponent_side[j] <= es->heur.side_exp_end) // mid_side_tiny)
                  do_band[j] = 1;
               else if (band_exponent_side[j] <= band_exponent_mid[j]-es->heur.side_exp_threshold) // + mid_side_offset || (band_exponent_side[j] < band_exponent_mid[j] && band_exponent_mid[j] <= mid_side_threshold))
                  do_band[j] = 1;
            }
            do_band[bi->num_bands] = do_band[bi->num_bands+1] = true;
            int count = 0;
            U32 actual_bands = 0;
            int k=0;
            // we signal in groups of 3 bands, so have to find groups of 3 bands where it's ok to mid-side
            for (int j=0; j < bi->num_bands; j += 3, ++k) {
               if (do_band[j+0]+do_band[j+1]+do_band[j+2] >= 5) { // if at least two of the three get reduced, and the third one is close
                  ++count;
                  mid_side_bands |= (1 << k);
                  actual_bands |= (7 << j);
               } else {
                  do_band[j+0] = do_band[j+1] = do_band[j+2] = 0;
               }
            }

            if (count >= 1) {
               do_mid_side_bands = true;
               // create composite coefficient arrays that have a mix of either one or the other in each band
               for (int j=0; j < bi->num_bands; ++j) {
                  if (do_band[j]) {
                     int start = bi->first_coeff_for_band[j];
                     int num_coeffs_for_band = bi->num_coeffs_for_band[j];
                     memcpy(coeff1 + start, coeff_mid  + start, sizeof(coeff_mid[0]) * num_coeffs_for_band);
                     memcpy(coeff2 + start, coeff_side + start, sizeof(coeff_mid[0]) * num_coeffs_for_band);
                  }
               }
               encode_channel(es, &bd[0], coeff1, 0, subband_energy[0],            0, NULL);
               encode_channel(es, &bd[1], coeff2, 1, subband_energy[1], actual_bands, &bd[0]);
            }
         }
      }

      if (!do_mid_side && !do_mid_side_bands) {
         encode_channel(es, &bd[0], coeff1, 0, subband_energy[0], 0, NULL);
         encode_channel(es, &bd[1], coeff2, 1, subband_energy[1], 0, NULL);
      }
   }

   int c;

   U8 band_exponents[32*2];
   int num_band_exponents=0;
   rrbool stereo_predict_exponent = false;

   // band exponents

   for (c=0; c < out_channels; ++c) {
      int lastn = PREDICT_FIRST_BAND_EXP;

      for (int j=0; j < bi->num_bands; ++j) {
         int n = bd[c].band_exponent[j];
         rrAssert(n == BAND_EXPONENT_NONE || (n >= -32 && n < 32));
         if (BAND_EXPONENT_NONE == -17)
            rrAssert(n >= -17 && n < 32);

         band_exponents[num_band_exponents++] = (U8) (n - lastn);
         lastn = n;
      }
   }

   if (out_channels == 2) {
      // try stereo predicting exponents
      int right_cost_nostereo=0, right_cost_stereo=0;
      for (int j=0; j < bi->num_bands; ++j) {
         right_cost_nostereo += rada_band_exponent_correct_huff.encode[band_exponents[bi->num_bands + j]].length;

         int predict = bd[1].band_exponent[j] - bd[0].band_exponent[j];
         int cost    = rada_band_exponent_stereo_correct_huff.encode[(U8) predict].length;
         if (cost == 0)
            right_cost_stereo = 256*bi->num_bands; // if invalid, can't use this path
         else
            right_cost_stereo += cost;
      }

      if (right_cost_stereo < right_cost_nostereo) {
         stereo_predict_exponent = true;
         num_band_exponents >>= 1;
         for (int i=0; i < bi->num_bands; ++i)
            band_exponents[32+i] = (U8) (bd[1].band_exponent[i] - bd[0].band_exponent[i]);
      }
   }

   U8 subband_value[128*2];
   U8 subband_correction[32*2];
   S8 big_coefficients[1024*2];
   U8 nonzero_coefficients[1024*2];
   U8 runlength_data[1025*2];
   U8 runlength_bottom[1024*2];
   U8 nonzero_flagbits[1024*2/8*16];
   U8 subband_stereo_correct[128];

   int num_subband_values0=0;
   int num_subband_corrections=0;
   int num_subband_stereo_correct=0;
   int num_big_coefficients=0;
   int num_nonzero_coefficients=0;
   int num_runlength_data=0;
   int num_runlength_bottom=0;

   rrbool disable_subband_predict = false;
   // compute subband predictions
   for (c=0; c < out_channels; ++c) {
      for (int j=0; j < bi->num_bands; ++j) {
         // skip subband data if subband occupies whole band
         if (bi->num_subbands_for_band[j] == 1)
            continue;

         int start = bi->first_subband_for_band[j];
         int predicted_sum = es->subband_predicted_sum[j];
         int actual_sum = 0;
         for (int i=0; i < bi->num_subbands_for_band[j]; ++i) {
            int v = bd[c].quantized_subbands[start+i];
            actual_sum += v;
         }
         int predict = (actual_sum - predicted_sum);
         if (predict < -128 || predict > 127) {
            disable_subband_predict = true;
         }
         else if (rada_subband_value_last_in_band_correct_huff.encode[(U8) predict].length == 0) {
            disable_subband_predict = true;
         }
      }
   }
   es->stats.block_events[E_subband_nopredict] += disable_subband_predict ? 1 : 0;

   rrbool stereo_predict_subbands = false;
   if (!es->current_block_short) {
      if (out_channels == 2) {
         // try stereo predicting subbands
         int right_cost_nostereo=0, right_cost_stereo=0;
         for (int j=0; j < bi->num_bands; ++j) {
            if (bi->num_subbands_for_band[j] == 1)
               continue;

            if (bd[1].band_exponent[j] == BAND_EXPONENT_NONE && SUBBANDS_SKIP_EMPTY_BANDS)
               continue;

            int start = bi->first_subband_for_band[j];
            int num   = bi->num_subbands_for_band[j];

            int bias = es->subband_bias[j];

            int actual_sum=0;
            for (int i=0; i < num; ++i) {
               actual_sum += bd[1].quantized_subbands[start+i];
               if (i == num-1 && !disable_subband_predict)
                  right_cost_nostereo += rada_subband_value_last_in_band_correct_huff.encode[(U8) (actual_sum - es->subband_predicted_sum[j])].length;
               else
                  right_cost_nostereo += rada_subband_value_huff.encode[(bias + bd[1].quantized_subbands[start+i])&63].length;

               int correct = bd[1].quantized_subbands[start+i] - bd[0].quantized_subbands[start+i];
               int cost    = rada_subband_value_stereo_correct_huff.encode[(U8) correct].length;
               if (cost == 0)
                  right_cost_stereo = 256*bi->num_subbands; // if invalid, can't use this path
               else
                  right_cost_stereo += cost;
            }
         }

         if (right_cost_stereo < right_cost_nostereo) {
            stereo_predict_subbands = true;
         }
      }

      // compute subband value encodings
      for (c=0; c < out_channels; ++c) {
         for (int j=0; j < bi->num_bands; ++j) {
            // skip subband data if subband occupies whole band, should be first 12
            if (bi->num_subbands_for_band[j] == 1)
               continue;

            int start = bi->first_subband_for_band[j];
            int num_to_write = bi->num_subbands_for_band[j];

            if (bd[c].band_exponent[j] == BAND_EXPONENT_NONE && SUBBANDS_SKIP_EMPTY_BANDS)
               continue;

            if (c == 1 && stereo_predict_subbands) {
               for (int i = 0; i < num_to_write; ++i) {
                  int correct = bd[1].quantized_subbands[start+i] - bd[0].quantized_subbands[start+i];
                  subband_stereo_correct[num_subband_stereo_correct++] = (U8) correct;
               }
            } else {
               int predicted_sum = es->subband_predicted_sum[j];
               int bias = es->subband_bias[j];
               int actual_sum = 0;

               if (!disable_subband_predict)
                  --num_to_write;

               for (int i=0; i < num_to_write; ++i) {
                  int v = bd[c].quantized_subbands[start+i];
                  actual_sum += v;
                  int code = (v + bias)&63;
                  if (code < 0 || code > LARGEST_BIASED_SUBBAND) fprintf(stderr, "encoded subband value %d (value %d) was outside of range 0..%d\n", code, v, LARGEST_BIASED_SUBBAND);
                  subband_value[num_subband_values0++] = (U8) code;
               }

               if (!disable_subband_predict) {
                  actual_sum += bd[c].quantized_subbands[start+num_to_write];
                  U8 correct = (U8) (actual_sum - predicted_sum);
                  subband_correction[num_subband_corrections++] = correct;
               }
            }
         }
      }
   }

   int nz_mode = 0; // for short blocks, use 0 value to avoid sending extra header flags

   if (!es->current_block_short) {
      // encode the nonzero coefficient positions in bitarray
      for (c=0; c < out_channels; ++c) {
         int p=c*(1024/8);
         for (int j=0; j < 1024; j += 8) {
            U8 flags=0;
            for (int k=0; k < 8; ++k) {
               int bits = bd[c].quantized_coeff_encode[j+k];
               if (bits != 0) {
                  flags |= (1 << k);
               }
            }
            nonzero_flagbits[p++] = flags;
         }
      }

      int best_no_rle=0, best_num=0;
      int best_mode=0, best_cost=0x7fffffff;
      // for each mode, measure the cost
      for (int m=0; m < 4; ++m) {
         radaudio_nonzero_blockmode_descriptor *nz_desc = &es->nz_desc[m];
         int bits= (m==0 ? 0 : 6); // anything other than mode 0 forces an extra header byte, though we might already be paying it for other reasons so approximate as 3/4ths of the header byte
         int nc = nz_desc->num_8byte_chunks;
         int num=0, numsym=0;
         int base;
         int bitc[8] = { 0 };
         //bits += compute_rle_length(es, bd, out_channels, nc*64);
         bits = compute_rle_length(es, bd, out_channels, nc*64, &numsym);
         base = bits;
         for (c=0; c < out_channels; ++c) {
            int p = (1024/8)*c;
            for (int i=0; i < nc; ++i) {
               radaudio_huffman *h = rada_nonzero_bitflags_huff[nz_desc->huffman_table_for_chunk[i]];
               num += 8;
               int pre = bits;

               if (nz_desc->invert_chunk[i])
                  for (int j=0; j < 8; ++j)
                     bits += h->encode[255^nonzero_flagbits[p++]].length;
               else
                  for (int j=0; j < 8; ++j)
                     bits += h->encode[    nonzero_flagbits[p++]].length;
               if (m == 0)
                  bitc[nz_desc->huffman_table_for_chunk[i]] += (bits - pre);
            }
         }

         if (bits < best_cost) {
            best_cost = bits;
            best_mode = m;
         }
         if (m == 0) {
            best_no_rle = bits - base;
            best_num = num;
         }
      }

      nz_mode = best_mode;
   }
   es->stats.block_events[E_nzmode0 + nz_mode] += 1;

   // encode non-zero coefficient locations
   for (c=0; c < out_channels; ++c) {
      int i,j,k;

      if (!es->current_block_short && nz_mode != 3) {
         int count = es->nz_desc[nz_mode].num_8byte_chunks*64;

         // encode the first `count` coefficients as non-zero based on 1-bit flags to be compressed later
         for (j=0; j < count; j += 8) {
            U8 flags=0;
            for (k=0; k < 8; ++k) {
               int bits = bd[c].quantized_coeff_encode[j+k];
               if (bits != 0) {
                  flags |= (1 << k);
                  if (abs(bits) > 7) {
                     nonzero_coefficients[num_nonzero_coefficients++] = 0;
                     big_coefficients    [num_big_coefficients++    ] = (S8) bits;
                  } else {
                     nonzero_coefficients[num_nonzero_coefficients++] = (U8) (bits+8);
                  }
               }
            }
         }
      }

      // encode the remaining non-zero coefficient locations by run-length compressing the intervening 0s

      k=0;
      int skip = es->nz_desc[nz_mode].num_8byte_chunks * 64;
      for (j=0; j < bi->num_bands; ++j) {
         int start = bi->first_coeff_for_band[j];
         int count = bi->num_coeffs_for_band[j];

         rrAssert(bi->num_coeffs <= 1024);
         for (i=0; i < count; ++i) {
            // skip the coefficients we encoded in the previous loop above
            if (!es->current_block_short && nz_mode != 3 && start+i < skip)
               continue;

            int bits = bd[c].quantized_coeff_encode[start+i];
            if (bits == 0)
               ++k;
            else {
               // new zero-run-length encoding
               int zr = k;

               while (zr >= MAX_RUNLEN) {
                  runlength_data[num_runlength_data++] = (U8) MAX_RUNLEN;
                  zr -= MAX_RUNLEN;
               }
               if (zr >= COARSE_RUNLEN_THRESHOLD) {
                  int coarse = zr & ~3;
                  runlength_data[num_runlength_data++] = (U8) coarse;
                  zr -= coarse;
                  runlength_bottom[num_runlength_bottom++] = (U8) zr;
               } else
                  runlength_data[num_runlength_data++] = (U8) zr;

               if (abs(bits) > 7) {
                  nonzero_coefficients[num_nonzero_coefficients++] = 0;
                  big_coefficients    [num_big_coefficients++    ] = (S8) bits;
               } else {
                  nonzero_coefficients[num_nonzero_coefficients++] = (U8) (bits+8);
               }
               k = 0;
            }
         }
      }

      // end of channel's data
      if (c != out_channels-1)
         runlength_data[num_runlength_data++] = (U8) END_OF_ZERORUN; // end of run marker
      // don't bother outputting the terminating end of zerorun marker, we can infer that from length we have to use with OodleData huffman
   }

   U8 coeff_pairs[1024/2 * 2];
   if ((num_nonzero_coefficients & 1) != 0)
      nonzero_coefficients[num_nonzero_coefficients++] = 7; // cheapest symbol to encode

   int num_coeff_pairs = (num_nonzero_coefficients+1)/2;
   for (int i=0; i < num_coeff_pairs; ++i)
      coeff_pairs[i] = nonzero_coefficients[i*2+0] | (nonzero_coefficients[i*2+1]<<4);

   int nz_selector = es->current_block_short ? 4 : nz_mode;

   huff3_encoder he;
   encode_vbstream_init(&he.stream[0], &es->buffer[0]       , 1000);
   encode_vbstream_init(&he.stream[1], &es->buffer[0]+1024*1, 1000);
   encode_vbstream_init(&he.stream[2], &es->buffer[0]+1024*2, 3000);
   he.total_bits = 0;

   size_t prev_bitcount;
   prev_bitcount = he.total_bits;

   // band exponents
   PROF_BEGIN(huffman);
   encode_huff_array(&he, &rada_band_exponent_correct_huff, band_exponents, num_band_exponents, __FILE__, __LINE__, &error);
   if (stereo_predict_exponent)
      encode_huff_array(&he, &rada_band_exponent_stereo_correct_huff, &band_exponents[32], num_band_exponents, __FILE__, __LINE__, &error);
   PROF_END(huffman);

   es->stats.bit_allocation[S_band_exponent] += (he.total_bits - prev_bitcount);

   // band mantissas -- pack them directly into varbits-array decoder
   U8 m_sizes[64];
   U32 m_values[64];
   int num_mantissas=0;

   prev_bitcount = he.total_bits;
   for (c=0; c < out_channels; ++c) {
      for (int j=0; j < bi->num_bands; ++j) {
         m_values[num_mantissas] =      bd[c].band_mantissa[j];
         m_sizes [num_mantissas] = (U8) bd[c].band_mantissa_bitcount[j];
         ++num_mantissas;
      }
   }

   for (int j=0; j < num_mantissas; ++j)
      encode_vbstream_bits(&he.stream[2], m_values[j], m_sizes[j]);

   encode_recompute_stats(&he);
   es->stats.bit_allocation[S_band_mantissa] += (he.total_bits - prev_bitcount);

   PROF_BEGIN(huffman);
   if (!es->current_block_short) {
      // subband values
      prev_bitcount = he.total_bits;
      encode_huff_array(&he, &rada_subband_value_huff                        , subband_value         , num_subband_values0       , __FILE__, __LINE__, &error);
      if (!disable_subband_predict)
         encode_huff_array(&he, &rada_subband_value_last_in_band_correct_huff, subband_correction    , num_subband_corrections   , __FILE__, __LINE__, &error);
      if (stereo_predict_subbands)
         encode_huff_array(&he, &rada_subband_value_stereo_correct_huff      , subband_stereo_correct, num_subband_stereo_correct, __FILE__, __LINE__, &error);
      es->stats.bit_allocation[S_subband] += (he.total_bits - prev_bitcount);
   }
   PROF_END(huffman);

   if (!es->current_block_short && nz_mode != 3) {
      U8 nonzero_flag_huff[1024*2/8*16];
      radaudio_nonzero_blockmode_descriptor *nz_desc = &es->nz_desc[nz_mode];

      // encode the non-zero flag bits
      // reorder the bits into an array where each huffman encoding is contiguous
      // also invert any chunks needing inversion
      int p=0, s=out_channels-1;
      for (c=0; c < out_channels; ++c) {
         p = (1024/8/8) * c;
         for (int i=0; i < nz_desc->num_8byte_chunks; ++i) {
            int off = nz_desc->source_pos[s+c][i];
            U64 xor = (U64)0 - nz_desc->invert_chunk[i]; // ~0 if invert_chunk, else 0 (assuming invert_chunk is either 0 or 1)
            RR_PUT64_NATIVE(&nonzero_flag_huff[8*off], xor ^ RR_GET64_NATIVE(&nonzero_flagbits[8*p]));
            p++;
         }
      }

      // now output the huffman arrays
      prev_bitcount = he.total_bits;
      p=0;
      PROF_BEGIN(huffman);
      for (int i=0; i < NUM_NZ_HUFF; ++i) {
         int nc = nz_desc->num_chunks_per_huff[i];
         if (nc != 0) {
            encode_huff_array(&he, rada_nonzero_bitflags_huff[i], &nonzero_flag_huff[p], nc*8*out_channels, __FILE__, __LINE__, &error);
            p += nc * 8 * out_channels;
         }
      }
      PROF_END(huffman);
      es->stats.bit_allocation[S_coeff_location] += (he.total_bits - prev_bitcount);
   }

   PROF_BEGIN(huffman);
   prev_bitcount = he.total_bits;
   encode_huff_array(&he, &rada_zero_runlength_huff, runlength_data, num_runlength_data, __FILE__, __LINE__, &error);
   es->stats.bit_allocation[S_coeff_location] += (he.total_bits - prev_bitcount);

   // coefficients -- need to have decoded the runlength data to know how many coefficients
   prev_bitcount = he.total_bits;
   int tp = es->nz_correlated_huffman_selectors[HS_COEFF_PAIR][nz_selector];
   encode_huff_array(&he, rada_nonzero_coefficient_pair_huff[tp], coeff_pairs, num_coeff_pairs, __FILE__, __LINE__, &error);
   es->stats.bit_allocation[S_coeff_value] += (he.total_bits - prev_bitcount);
   PROF_END(huffman);

   PROF_BEGIN(varbits);
   // encode bottom runlength bits to vbstream 2
   prev_bitcount = he.total_bits;
   for (int j=0; j < num_runlength_bottom; ++j)
      encode_vbstream_bits(&he.stream[2], runlength_bottom[j], 2);
   es->stats.bit_allocation[S_coeff_location] += (he.total_bits - prev_bitcount);
   PROF_END(varbits);

   PROF_BEGIN(huffman);
   // huffman encode big coefficients
   prev_bitcount = he.total_bits;
   int tb = es->nz_correlated_huffman_selectors[HS_COEFF_BIG][nz_selector];
   encode_huff_array(&he, rada_nonzero_coefficient_big_huff[tb], (U8*) big_coefficients, num_big_coefficients, __FILE__, __LINE__, &error);
   es->stats.bit_allocation[S_coeff_value_large] += (he.total_bits - prev_bitcount);
   PROF_END(huffman);

   prev_bitcount = he.total_bits;
   for (int i=0; i < 3; ++i)
      encode_vbstream_flush(&he.stream[i]);
   encode_recompute_stats(&he);
   es->stats.bit_allocation[S_padding] += (he.total_bits - prev_bitcount);

   // assemble the final block
   radaudio_block_header_unpacked bh = { 0 };

   // advance to the new center, which means advance by half of the current and half of the next
   int completed_samples = es->info[es->current_block_short]->num_coeffs/2
                         + es->info[es->next_block_short   ]->num_coeffs/2;

   *poffset = offset + completed_samples;

   if (error)
      return RADAUDIOENC_INTERNAL_ERROR;

   // this is the condition for being done:
   //    if (offset >= input_len + RADAUDIO_SHORT_BLOCK_LEN)
   // so the last block is the one that puts us in that state
   // Note this is the condition because we already update offset to point to the middle of the next block;
   // the next block minus RADAUDIO_SHORT_BLOCK_LEN is the number of overlapped samples that block will make if it or this block is short, so every sample before that is complete.
   // that's not quite right if the this block and the next block are long, but we never make long blocks at end of file, so
   // even if one long block ends right on the input, the next block will be short

   bh.final_block = (*poffset >= input_len + RADAUDIO_SHORT_BLOCK_LEN);
   int discard_samples = 0;
   if (bh.final_block)
      discard_samples = (int) (*poffset-RADAUDIO_SHORT_BLOCK_LEN - input_len); // number of fully decoded samples we didn't use

   bh.this_block_short = es->current_block_short;
   bh.next_block_short = es->next_block_short;
   bh.num_channels_encoded = out_channels;
   bh.final_samples_discard = discard_samples;

   bh.vbstream0_length = he.stream[0].length;
   bh.num_runlength_array = num_runlength_data;
   bh.nonzero_bitarray_mode = nz_mode;
   bh.disable_final_subband_predict = disable_subband_predict;
   bh.predict_stereo_subband = stereo_predict_subbands;
   bh.predict_stereo_exponent = stereo_predict_exponent;
   bh.mid_side_encoded = do_mid_side;
   bh.mid_side_bands   = do_mid_side_bands;

   size_t byte_offset = 0;

   if (encode_buffer_max < 10)
      return RADAUDIOENC_INSUFFICIENT_BUFFER;

   size_t midside_len = do_mid_side_bands ? (24/MACRO_BAND_SIZE+7)/8 : 0;

   size_t non_header_length = he.stream[0].length + he.stream[1].length + he.stream[2].length + midside_len;
   bh.block_bytes = (U32) non_header_length;

   int header_size = radaudio_encode_block_header(encode_buffer, &es->biases, &bh);
   rrAssert(header_size >= 0);
   if (header_size < 0)
      return RADAUDIOENC_INTERNAL_ERROR;

   byte_offset = header_size;
   es->stats.bit_allocation[S_header] += byte_offset * 8;
   rrAssert(encode_buffer[0] != 0);

   #ifdef _DEBUG
   size_t total_size = byte_offset + non_header_length;
   #endif

   if (byte_offset + midside_len > encode_buffer_max)
      return RADAUDIOENC_INSUFFICIENT_BUFFER;
   for (int i=0; i < (int)midside_len; ++i)
      encode_buffer[byte_offset++] = (U8) ((mid_side_bands >> (8*i)) & 255);

   if (byte_offset + he.stream[0].length > encode_buffer_max)
      return RADAUDIOENC_INSUFFICIENT_BUFFER;
   for (int i=0; i < he.stream[0].length; ++i)
      encode_buffer[byte_offset++] = he.stream[0].bitstream[i];

   if (byte_offset + he.stream[2].length > encode_buffer_max)
      return RADAUDIOENC_INSUFFICIENT_BUFFER;
   for (int i=0; i < he.stream[2].length; ++i)
      encode_buffer[byte_offset++] = he.stream[2].bitstream[i];

   if (byte_offset + he.stream[1].length > encode_buffer_max)
      return RADAUDIOENC_INSUFFICIENT_BUFFER;
   for (int i=he.stream[1].length-1; i >= 0; --i)
      encode_buffer[byte_offset++] = he.stream[1].bitstream[i];

   if (he.stream[0].error || he.stream[1].error | he.stream[2].error)
      return RADAUDIOENC_INTERNAL_ERROR;

   rrAssert(encode_buffer[0] != 0);

   es->prev_block_short = es->current_block_short;
   es->current_block_short = es->next_block_short;
   es->samples_fully_coded = (int) *poffset;
   ++es->block_number;

   es->lastblock_vbstream0_length    = bh.vbstream0_length;
   es->lastblock_num_runlength_array = bh.num_runlength_array;
   es->lastblock_block_bytes         = bh.block_bytes;

   rrAssert(byte_offset == total_size);

   PROF_END(encode_all);

   return (int) byte_offset;
}

/////////////////////////////////////////////////////////////////////////////
//
// Transient analysis
//

#define TA_MAX(a,b) ((a) > (b) ? (a) : (b))

int transient_analysis(float* input, int N, int stride, float transient_threshold)
{
   RAD_ALIGN(F32, w[512], 16);

	rrAssert(N >= 40 && N <= 1024 && N%2 == 0);
	int N2 = N/2;

	// IIR highpass, combined with 2:1 decimation
	float x1 = 0, x2 = 0;
	for (int i = 0; i < N2; i++)
	{
		float x0 = *input + x1 - 0.5f * x2;
		float ya = 0.625f * (x0 + x2) - 1.25f * x1;
		x2 = x1;
		x1 = x0;
		input += stride;

		x0 = *input + x1 - 0.5f * x2;
		float yb = 0.625f * (x0 + x2) - 1.25f * x1;
		x2 = x1;
		x1 = x0;
		input += stride;

		w[i] = ya*ya + yb*yb;
	}

	// clear boundary samples
	for (int i = 0; i < 6; i++)
	{
		w[i] = 0;
	}

	// compute sum, do forward exponential moving average
	float sum_sq = 0;
	float ema = 0;
	for (int i = 6; i < N2; i++)
	{
		sum_sq += w[i];
		const float k = 1.f / 16.f;
		ema = ema + k * (w[i] - ema);
		w[i] = ema;
	}

	// do backwards exponential moving average, find maximum
	ema = 0;
	float max_sq = 0;
	for (int i = N2-1; i >= 0; i--)
	{
		const float k = 1.f / 8.f;
		ema = ema + k * (w[i] - ema);
		w[i] = ema;
		max_sq = TA_MAX(ema, max_sq);
	}

	// set up for final pass
	const float eps = 1e-15f;
	float geomean = sqrtf(sum_sq * (max_sq * N2 * 0.5f));
	float scale = N2 / (geomean + eps);

	// subsampled harmonic mean
	const float min_threshold = 1.5f / 64.f;
	float rcp_sum = 0;
	for (int i = 12; i < N2-5; i += 4)
	{
		rcp_sum += 1.f / TA_MAX((w[i] + eps) * scale, min_threshold);
	}
	const int num_summed = (N2 - 17 + 3) >> 2;

	return rcp_sum > (transient_threshold / 64.f) * num_summed;
}

// BAND_EXPONENT_BASE0/1
// initial guess 1.43
// This constant affects how many coefficient pulses are assigned per band by
// weighting each band with this number raised to the computed band exponent
// (e.g. if this number were 2.0, then it would weight by the band energy, and 1.0 weights evenly)
static float band_exponent_base[4][2][10] =
{  // 1.1 instead of 1.2 has 1% better average ODG, but 8% higher standard deviation
   {
       { 1.2f, 1.2f, 1.2f, 1.2f, 1.2f,   1.18f, 1.14f, 1.10f, 1.08f, 0.96f },
       { 1.6f, 1.6f, 1.6f, 1.6f, 1.6f,   1.60f, 1.50f, 1.40f, 1.30f, 1.20f },
   },
   {
       { 1.2f, 1.2f, 1.2f, 1.2f, 1.2f,   1.15f, 1.13f, 1.11f, 1.08f, 1.05f },
       { 1.6f, 1.6f, 1.6f, 1.6f, 1.6f,   1.60f, 1.50f, 1.40f, 1.30f, 1.20f },
   },
   {
       { 1.2f, 1.2f, 1.2f, 1.2f, 1.2f,   1.20f, 1.16f, 1.08f, 1.00f, 0.98f },
       { 1.6f, 1.6f, 1.6f, 1.6f, 1.6f,   1.60f, 1.50f, 1.40f, 1.30f, 1.20f },
   },
   {
       { 1.2f, 1.2f, 1.2f, 1.2f, 1.2f,   1.30f, 1.20f, 1.06f, 1.04f, 1.02f },
       { 1.6f, 1.6f, 1.6f, 1.6f, 1.6f,   1.60f, 1.50f, 1.40f, 1.30f, 1.20f },
   },
};

// BAND_COUNT_EXPONENT0/1
// This constant affects how many coefficient pulses are assigned per band by
// weighting each band with the number of coefficients in the band raised to this power
// (if this number is 1.0, then they're weighted evenly by # coefficients_sum)
// initial guess ~0.85.
// std.dev appears to be minimized closer to 1.0, depending on the mode
static float band_count_exponent[4][2][10] =
{
   {
       { 0.86f, 0.86f, 0.86f, 0.86f, 0.86f,  0.96f, 0.98f, 0.98f, 0.96f, 0.95f },
       { 0.86f, 0.86f, 0.86f, 0.86f, 0.86f,  0.86f, 0.86f, 0.86f, 0.86f, 0.86f },
   },
   {
       { 0.86f, 0.86f, 0.86f, 0.86f, 0.86f,  0.88f, 0.90f, 0.94f, 0.96f, 0.98f },
       { 0.86f, 0.86f, 0.86f, 0.86f, 0.86f,  0.86f, 0.86f, 0.86f, 0.86f, 0.86f },
   },
   {
       { 0.86f, 0.86f, 0.86f, 0.86f, 0.86f,  0.98f, 0.98f, 0.98f, 0.98f, 0.98f },
       { 0.86f, 0.86f, 0.86f, 0.86f, 0.86f,  0.86f, 0.86f, 0.86f, 0.86f, 0.86f },
   },
   {
       { 0.86f, 0.86f, 0.86f, 0.86f, 0.86f,  0.96f, 0.96f, 0.96f, 0.95f, 0.95f },
       { 0.86f, 0.86f, 0.86f, 0.86f, 0.86f,  0.86f, 0.86f, 0.86f, 0.86f, 0.86f },
   },
};

// weight to assign to lowest bands, lerps towards 1.0 at highest bands
// QUALITY_WEIGHT_LOW0/1
static float quality_weight_low[4][2][10] =
{
    //     0      1      2      3      4       5      6      7      8      9
   {
      { 3.50f, 3.50f, 3.50f, 3.50f, 3.50f,  7.00f, 7.00f, 5.50f, 1.75f, 1.50f },
      { 3.50f, 3.50f, 3.50f, 3.50f, 3.50f,  4.00f, 4.00f, 4.00f, 4.00f, 4.00f },
   },
   {    // 6.5f
      { 3.50f, 3.50f, 3.50f, 3.50f, 3.50f,  5.25f, 4.50f, 3.25f, 2.25f, 2.00f },
      { 3.50f, 3.50f, 3.50f, 3.50f, 3.50f,  4.00f, 4.00f, 4.00f, 4.00f, 4.00f },
   },
   {
      { 3.50f, 3.50f, 3.50f, 3.50f, 3.50f,  5.50f, 4.25f, 3.50f, 3.50f, 3.50f },
      { 3.50f, 3.50f, 3.50f, 3.50f, 3.50f,  4.00f, 4.00f, 4.00f, 4.00f, 4.00f },
   },
   {
      { 3.50f, 3.50f, 3.50f, 3.50f, 3.50f,  2.50f, 2.50f, 2.00f, 1.50f, 1.50f },
      { 3.50f, 3.50f, 3.50f, 3.50f, 3.50f,  4.00f, 4.00f, 4.00f, 4.00f, 4.00f },
   },
};

// SHORT_BLOCK_PULSES - initial guess 0.2
static float short_block_pulses[4][10] =
{
    //  0      1      2      3      4       5      6      7      8      9
    { 0.11f, 0.11f, 0.11f, 0.11f, 0.11f,  0.11f, 0.11f, 0.13f, 0.15f, 0.080f },
    { 0.12f, 0.12f, 0.12f, 0.12f, 0.12f,  0.12f, 0.13f, 0.14f, 0.18f, 0.075f },
    { 0.09f, 0.09f, 0.09f, 0.09f, 0.09f,  0.09f, 0.10f, 0.13f, 0.17f, 0.045f },
    { 0.08f, 0.08f, 0.08f, 0.08f, 0.08f,  0.08f, 0.09f, 0.09f, 0.09f, 0.060f },
};

// EXTRA_THRESHOLD_BIG0/1
static float extra_threshold_big[4][2][10] =
{
   {
       { 16.0,16.0,16.0,16.0,16.0,  16.0,16.0,16.0,16.0,16.0, },
       { 16.0,16.0,16.0,16.0,16.0,  16.0,16.0,16.0,16.0,16.0, },
   },
   {
       { 16.0,16.0,16.0,16.0,16.0,  16.0,16.0,16.0,16.0,16.0, },
       { 16.0,16.0,16.0,16.0,16.0,  16.0,16.0,16.0,16.0,16.0, },
   },
   {
       { 16.0,16.0,16.0,16.0,16.0,  16.0, 8.0, 6.0, 6.0, 6.0, },
       { 16.0,16.0,16.0,16.0,16.0,  16.0,16.0,16.0,16.0,16.0, },
   },
   {
       { 16.0,16.0,16.0,16.0,16.0,  16.0,16.0,16.0,16.0,16.0, },
       { 16.0,16.0,16.0,16.0,16.0,  12.0,16.0,16.0,16.0,16.0, },
   },
};

// EXTRA_THRESHOLD_SMALL0/1
static float extra_threshold_small[4][2][10] =
{
   {
       {  8.0, 8.0, 8.0, 8.0, 8.0,   8.0, 8.0, 8.0, 8.0, 8.0, },
       {  2.0, 2.0, 2.0, 2.0, 2.0,   8.0, 8.0, 8.0, 8.0, 8.0, },
   },
   {
       {  8.0, 8.0, 8.0, 8.0, 8.0,   8.0, 8.0, 8.0, 8.0, 8.0, },
       {  2.0, 2.0, 2.0, 2.0, 2.0,   8.0, 8.0, 8.0, 8.0, 8.0, },
   },
   {
       {  8.0, 8.0, 8.0, 8.0, 8.0,   8.0, 4.0, 3.0, 1.5, 2.0, },
       {  2.0, 2.0, 2.0, 2.0, 2.0,   6.0, 8.0, 8.0, 8.0, 8.0, },
   },
   {
       {  8.0, 8.0, 8.0, 8.0, 8.0,   8.0, 8.0, 8.0, 8.0, 8.0, },
       {  2.0, 2.0, 2.0, 2.0, 2.0,   8.0, 8.0, 8.0, 8.0, 8.0, },
   },
};

#define MANTISSA_SCALE 1.00f   // used while tuning, env=MANTISSA_SCALE
static float band_mantissa_base[11][2][6] =
{
   { { 5.50f,5.50f,5.50f,5.50f,5.00f,4.50f }, { 3.25f,3.25f,3.25f,3.25f,2.95f,2.66f }, },
   { { 5.50f,5.50f,5.50f,5.50f,5.00f,4.50f }, { 3.25f,3.25f,3.25f,3.25f,2.95f,2.66f }, },
   { { 5.50f,5.50f,5.50f,5.50f,5.00f,4.50f }, { 3.25f,3.25f,3.25f,3.25f,2.95f,2.66f }, },
   { { 5.50f,5.50f,5.50f,5.50f,5.00f,4.50f }, { 3.25f,3.25f,3.25f,3.25f,2.95f,2.66f }, },
   { { 5.50f,5.50f,5.50f,5.50f,5.00f,4.50f }, { 3.25f,3.25f,3.25f,3.25f,2.95f,2.66f }, },
   { { 5.50f,5.50f,5.50f,5.50f,5.00f,4.50f }, { 3.25f,3.25f,3.25f,3.25f,2.95f,2.66f }, },
   { { 5.50f,5.50f,5.50f,5.50f,5.00f,4.50f }, { 3.25f,3.25f,3.25f,3.25f,2.95f,2.66f }, },
   { { 6.00f,6.00f,6.00f,6.00f,6.00f,5.50f }, { 3.50f,3.50f,3.50f,3.50f,3.25f,3.00f }, },
   { { 6.50f,6.50f,6.50f,6.50f,6.00f,5.50f }, { 4.00f,4.00f,4.00f,4.00f,3.75f,3.25f }, },
   { { 6.50f,6.50f,6.50f,6.50f,6.50f,5.50f }, { 4.00f,4.00f,4.00f,4.00f,3.75f,3.25f }, },
   { { 5.50f,5.50f,5.50f,5.50f,5.00f,4.50f }, { 3.25f,3.25f,3.25f,3.25f,2.95f,2.66f }, },
};

static float band_mantissa_decay[11][2][6] =
{
   { { 0.18f,0.17f,0.16f,0.14f,0.13f,0.12f }, { 0.11f,0.10f,0.09f,0.08f,0.08f,0.07f }, },
   { { 0.18f,0.17f,0.16f,0.14f,0.13f,0.12f }, { 0.11f,0.10f,0.09f,0.08f,0.08f,0.07f }, },
   { { 0.18f,0.17f,0.16f,0.14f,0.13f,0.12f }, { 0.11f,0.10f,0.09f,0.08f,0.08f,0.07f }, },
   { { 0.18f,0.17f,0.16f,0.14f,0.13f,0.12f }, { 0.11f,0.10f,0.09f,0.08f,0.08f,0.07f }, },
   { { 0.18f,0.17f,0.16f,0.14f,0.13f,0.12f }, { 0.11f,0.10f,0.09f,0.08f,0.08f,0.07f }, },
   { { 0.18f,0.17f,0.16f,0.14f,0.13f,0.12f }, { 0.11f,0.10f,0.09f,0.08f,0.08f,0.07f }, },
   { { 0.18f,0.17f,0.16f,0.14f,0.13f,0.12f }, { 0.11f,0.10f,0.09f,0.08f,0.08f,0.07f }, },
   { { 0.19f,0.18f,0.17f,0.16f,0.15f,0.14f }, { 0.12f,0.11f,0.10f,0.09f,0.09f,0.08f }, },
   { { 0.21f,0.20f,0.19f,0.18f,0.16f,0.15f }, { 0.13f,0.12f,0.10f,0.11f,0.10f,0.09f }, },
   { { 0.21f,0.20f,0.19f,0.18f,0.16f,0.15f }, { 0.13f,0.12f,0.10f,0.11f,0.10f,0.09f }, },
   { { 0.18f,0.17f,0.16f,0.14f,0.13f,0.12f }, { 0.11f,0.10f,0.09f,0.08f,0.08f,0.07f }, },
};

static float band_mantissa_base_improved_short_dc[11][2][6] =
{
   { { 5.50f,5.50f,5.50f,5.50f,5.00f,4.50f }, { 5.25f,3.25f,3.25f,3.25f,2.95f,2.66f }, },
   { { 5.50f,5.50f,5.50f,5.50f,5.00f,4.50f }, { 5.25f,3.25f,3.25f,3.25f,2.95f,2.66f }, },
   { { 5.50f,5.50f,5.50f,5.50f,5.00f,4.50f }, { 5.25f,3.25f,3.25f,3.25f,2.95f,2.66f }, },
   { { 5.50f,5.50f,5.50f,5.50f,5.00f,4.50f }, { 5.25f,3.25f,3.25f,3.25f,2.95f,2.66f }, },
   { { 5.50f,5.50f,5.50f,5.50f,5.00f,4.50f }, { 5.25f,3.25f,3.25f,3.25f,2.95f,2.66f }, },
   { { 5.50f,5.50f,5.50f,5.50f,5.00f,4.50f }, { 4.50f,3.25f,3.25f,3.25f,2.95f,2.66f }, },
   { { 5.75f,5.50f,5.50f,5.50f,5.00f,4.50f }, { 5.00f,3.25f,3.25f,3.25f,2.95f,2.66f }, },
   { { 6.00f,6.00f,6.00f,6.00f,6.00f,5.50f }, { 5.50f,3.50f,3.50f,3.50f,3.25f,3.00f }, },
   { { 6.50f,6.50f,6.50f,6.50f,6.00f,5.50f }, { 6.00f,4.00f,4.00f,4.00f,3.75f,3.25f }, },
   { { 7.00f,6.50f,6.50f,6.50f,6.50f,5.50f }, { 6.50f,4.00f,4.00f,4.00f,3.75f,3.25f }, },
   { { 6.50f,5.50f,5.50f,5.50f,5.00f,4.50f }, { 7.50f,3.25f,3.25f,3.25f,2.95f,2.66f }, },
};

static float band_mantissa_decay_improved_short_dc[11][2][6] =
{
   { { 0.18f,0.17f,0.16f,0.14f,0.13f,0.12f }, { 0.11f,0.10f,0.09f,0.08f,0.08f,0.07f }, },
   { { 0.18f,0.17f,0.16f,0.14f,0.13f,0.12f }, { 0.11f,0.10f,0.09f,0.08f,0.08f,0.07f }, },
   { { 0.18f,0.17f,0.16f,0.14f,0.13f,0.12f }, { 0.11f,0.10f,0.09f,0.08f,0.08f,0.07f }, },
   { { 0.18f,0.17f,0.16f,0.14f,0.13f,0.12f }, { 0.11f,0.10f,0.09f,0.08f,0.08f,0.07f }, },
   { { 0.18f,0.17f,0.16f,0.14f,0.13f,0.12f }, { 0.11f,0.10f,0.09f,0.08f,0.08f,0.07f }, },
   { { 0.18f,0.17f,0.16f,0.14f,0.13f,0.12f }, { 0.13f,0.10f,0.09f,0.08f,0.08f,0.07f }, },
   { { 0.18f,0.17f,0.16f,0.14f,0.13f,0.12f }, { 0.15f,0.10f,0.09f,0.08f,0.08f,0.07f }, },
   { { 0.19f,0.18f,0.17f,0.16f,0.15f,0.14f }, { 0.17f,0.11f,0.10f,0.09f,0.09f,0.08f }, },
   { { 0.21f,0.20f,0.19f,0.18f,0.16f,0.15f }, { 0.19f,0.12f,0.10f,0.11f,0.10f,0.09f }, },
   { { 0.22f,0.20f,0.19f,0.18f,0.16f,0.15f }, { 0.21f,0.12f,0.10f,0.11f,0.10f,0.09f }, },
   { { 0.18f,0.17f,0.16f,0.14f,0.13f,0.12f }, { 0.11f,0.10f,0.09f,0.08f,0.08f,0.07f }, },
};


#define SUBBAND_PULSES_SCALE 1.0   // SUBBAND_SCALE
static float subband_pulses_for_band[4][10][24] =
{
   {
      { 0,0,0,0, 0,0,0,0, 0,0,0,0,   9.50f,9.20f,8.80f,8.50f,5.00f,4.50f,4.00f,3.00f,2.50f },
      { 0,0,0,0, 0,0,0,0, 0,0,0,0,   9.50f,9.20f,8.80f,8.50f,5.00f,4.50f,4.00f,3.00f,2.50f },
      { 0,0,0,0, 0,0,0,0, 0,0,0,0,   9.50f,9.20f,8.80f,8.50f,5.00f,4.50f,4.00f,3.00f,2.50f },
      { 0,0,0,0, 0,0,0,0, 0,0,0,0,   9.50f,9.20f,8.80f,8.50f,5.00f,4.50f,4.00f,3.00f,2.50f },
      { 0,0,0,0, 0,0,0,0, 0,0,0,0,   9.50f,9.20f,8.80f,8.50f,5.00f,4.50f,4.00f,3.00f,2.50f },
      { 0,0,0,0, 0,0,0,0, 0,0,0,0,   9.50f,9.20f,8.80f,8.50f,5.00f,4.50f,4.00f,3.00f,2.50f },
      { 0,0,0,0, 0,0,0,0, 0,0,0,0,   14.3f,13.8f,13.2f,12.8f,7.50f,6.75f,6.00f,4.50f,3.80f }, // 6
      { 0,0,0,0, 0,0,0,0, 0,0,0,0,   14.3f,13.8f,13.2f,12.8f,7.50f,6.75f,6.00f,4.50f,3.80f }, // 7
      { 0,0,0,0, 0,0,0,0, 0,0,0,0,   19.0f,18.4f,17.6f,17.0f,10.0f,9.00f,8.00f,6.00f,5.00f }, // 8
      { 0,0,0,0, 0,0,0,0, 0,0,0,0,   23.8f,23.0f,22.0f,21.3f,12.5f,11.3f,10.0f,7.50f,6.25f }, // 9
   },
   {
      { 0,0,0,0, 0,0,0,0, 0,0,0,0,   9.50f,9.20f,8.80f,8.50f,5.00f,4.50f,4.00f,3.00f,2.50f },
      { 0,0,0,0, 0,0,0,0, 0,0,0,0,   9.50f,9.20f,8.80f,8.50f,5.00f,4.50f,4.00f,3.00f,2.50f },
      { 0,0,0,0, 0,0,0,0, 0,0,0,0,   9.50f,9.20f,8.80f,8.50f,5.00f,4.50f,4.00f,3.00f,2.50f },
      { 0,0,0,0, 0,0,0,0, 0,0,0,0,   9.50f,9.20f,8.80f,8.50f,5.00f,4.50f,4.00f,3.00f,2.50f },
      { 0,0,0,0, 0,0,0,0, 0,0,0,0,   9.50f,9.20f,8.80f,8.50f,5.00f,4.50f,4.00f,3.00f,2.50f },
      { 0,0,0,0, 0,0,0,0, 0,0,0,0,   9.50f,9.20f,8.80f,8.50f,5.00f,4.50f,4.00f,3.00f,2.50f },
      { 0,0,0,0, 0,0,0,0, 0,0,0,0,   14.3f,13.8f,13.2f,12.8f,7.50f,6.75f,6.00f,4.50f,3.80f }, // 6
      { 0,0,0,0, 0,0,0,0, 0,0,0,0,   14.3f,13.8f,13.2f,12.8f,7.50f,6.75f,6.00f,4.50f,3.80f }, // 7
      { 0,0,0,0, 0,0,0,0, 0,0,0,0,   19.0f,18.4f,17.6f,17.0f,10.0f,9.00f,8.00f,6.00f,5.00f }, // 8
      { 0,0,0,0, 0,0,0,0, 0,0,0,0,   23.8f,23.0f,22.0f,21.3f,12.5f,11.3f,10.0f,7.50f,6.25f }, // 9
   },
   {
      { 0,0,0,0, 0,0,0,0, 0,0,0,0,   9.50f,9.20f,8.80f,8.50f,5.00f,4.50f,4.00f,3.00f,2.50f,2.50f,2.50f },
      { 0,0,0,0, 0,0,0,0, 0,0,0,0,   9.50f,9.20f,8.80f,8.50f,5.00f,4.50f,4.00f,3.00f,2.50f,2.50f,2.50f },
      { 0,0,0,0, 0,0,0,0, 0,0,0,0,   9.50f,9.20f,8.80f,8.50f,5.00f,4.50f,4.00f,3.00f,2.50f,2.50f,2.50f },
      { 0,0,0,0, 0,0,0,0, 0,0,0,0,   9.50f,9.20f,8.80f,8.50f,5.00f,4.50f,4.00f,3.00f,2.50f,2.50f,2.50f },
      { 0,0,0,0, 0,0,0,0, 0,0,0,0,   9.50f,9.20f,8.80f,8.50f,5.00f,4.50f,4.00f,3.00f,2.50f,2.50f,2.50f },
      { 0,0,0,0, 0,0,0,0, 0,0,0,0,   9.50f,9.20f,8.80f,8.50f,5.00f,4.50f,4.00f,3.00f,2.50f,2.50f,2.50f },
      { 0,0,0,0, 0,0,0,0, 0,0,0,0,   14.3f,13.8f,13.2f,12.8f,7.50f,6.75f,6.00f,4.50f,3.80f,2.50f,2.50f }, // 6
      { 0,0,0,0, 0,0,0,0, 0,0,0,0,   14.3f,13.8f,13.2f,12.8f,7.50f,6.75f,6.00f,4.50f,3.80f,3.50f,3.50f }, // 7
      { 0,0,0,0, 0,0,0,0, 0,0,0,0,   19.0f,18.4f,17.6f,17.0f,10.0f,9.00f,8.00f,6.00f,5.00f,3.50f,3.50f }, // 8
      { 0,0,0,0, 0,0,0,0, 0,0,0,0,   23.8f,23.0f,22.0f,21.3f,12.5f,11.3f,10.0f,7.50f,6.25f,4.50f,4.50f }, // 9
   },
   {
      { 0,0,0,0, 0,0,0,0, 0,0,0,0,   9.50f,9.20f,8.80f,8.50f,5.00f,4.50f,4.00f,3.00f,2.50f,2.50f,2.50f },
      { 0,0,0,0, 0,0,0,0, 0,0,0,0,   9.50f,9.20f,8.80f,8.50f,5.00f,4.50f,4.00f,3.00f,2.50f,2.50f,2.50f },
      { 0,0,0,0, 0,0,0,0, 0,0,0,0,   9.50f,9.20f,8.80f,8.50f,5.00f,4.50f,4.00f,3.00f,2.50f,2.50f,2.50f },
      { 0,0,0,0, 0,0,0,0, 0,0,0,0,   9.50f,9.20f,8.80f,8.50f,5.00f,4.50f,4.00f,3.00f,2.50f,2.50f,2.50f },
      { 0,0,0,0, 0,0,0,0, 0,0,0,0,   9.50f,9.20f,8.80f,8.50f,5.00f,4.50f,4.00f,3.00f,2.50f,2.50f,2.50f },
      { 0,0,0,0, 0,0,0,0, 0,0,0,0,   9.50f,9.20f,8.80f,8.50f,5.00f,4.50f,4.00f,3.00f,2.50f,2.50f,2.50f },
      { 0,0,0,0, 0,0,0,0, 0,0,0,0,   14.3f,13.8f,13.2f,12.8f,7.50f,6.75f,6.00f,4.50f,3.80f,2.50f,2.50f }, // 6
      { 0,0,0,0, 0,0,0,0, 0,0,0,0,   14.3f,13.8f,13.2f,12.8f,7.50f,6.75f,6.00f,4.50f,3.80f,3.50f,3.50f }, // 7
      { 0,0,0,0, 0,0,0,0, 0,0,0,0,   19.0f,18.4f,17.6f,17.0f,10.0f,9.00f,8.00f,6.00f,5.00f,3.50f,3.50f }, // 8
      { 0,0,0,0, 0,0,0,0, 0,0,0,0,   23.8f,23.0f,22.0f,21.3f,12.5f,11.3f,10.0f,7.50f,6.25f,4.50f,4.50f }, // 9
   }
};

static U16 header_size_bias[10] =
{
   0,0,0,0,0, 90,140,250,500,1000
};

static float pulse_quality_table[4][10] =
{
   { 0.03f,0.05f,0.08f,0.1500f,0.1650f,  0.1908f, 0.2131f, 0.2559f, 0.3383f, 0.5489f },
   { 0.03f,0.05f,0.08f,0.1500f,0.1650f,  0.1910f, 0.2124f, 0.2541f, 0.3405f, 0.5622f },
   { 0.03f,0.05f,0.08f,0.1500f,0.1650f,  0.1935f, 0.2075f, 0.2415f, 0.3104f, 0.6206f },
   { 0.03f,0.05f,0.08f,0.1500f,0.1650f,  0.1998f, 0.2228f, 0.2696f, 0.3841f, 0.7721f },
};

#ifdef RADAUDIO_DEVELOPMENT
static void radaudio_load_heuristics_from_environment_variables(radaudio_encoder_state *es, radaudio_stream_header_unpacked *h)
{
   char *env;

   int qmode = es->quality_mode;
   int rate_mode = es->samprate_mode;
   RR_UNUSED_VARIABLE(qmode);
   RR_UNUSED_VARIABLE(rate_mode);

   int side_exp_spacing  = es->heur.side_exp_start2 - es->heur.side_exp_threshold;
   int side_exp_deadzone = es->heur.side_exp_end    - es->heur.side_exp_start;

   env = getenv("SIDE_EXP_START"    ); if (env) es->heur.side_exp_start      = atoi(env);
   env = getenv("SIDE_EXP_THRESHOLD"); if (env) es->heur.side_exp_threshold  = atoi(env);
   env = getenv("SIDE_EXP_DEADZONE" ); if (env) side_exp_deadzone   = atoi(env);
   env = getenv("SIDE_EXP_SPACING"  ); if (env) side_exp_spacing    = atoi(env);

   es->heur.side_exp_start2 = es->heur.side_exp_threshold + side_exp_spacing;
   es->heur.side_exp_end    = es->heur.side_exp_start     + side_exp_deadzone;

   env = getenv("MID_SIDE_TINY"         ); if (env) es->heur.mid_side_tiny          = atoi(env);
   env = getenv("MID_SIDE_OFFSET"       ); if (env) es->heur.mid_side_offset        = atoi(env);
   env = getenv("MID_SIDE_THRESHOLD"    ); if (env) es->heur.mid_side_threshold     = atoi(env);
   env = getenv("MID_SIDE_MAX_BAD_BANDS"); if (env) es->heur.mid_side_max_bad_bands = atoi(env);

   env = getenv("EXPECTATION_SCALE"      ); if (env) es->heur.expectation_scale      = strtof(env,NULL);
   env = getenv("EXPECTATION_BASE"       ); if (env) es->heur.expectation_base       = strtof(env,NULL);
   env = getenv("SHORT_OVERLAP_SCALE1"   ); if (env) es->heur.short_overlap_scale1   = strtof(env,NULL);
   env = getenv("SHORT_OVERLAP_SCALE2"   ); if (env) es->heur.short_overlap_scale2   = strtof(env,NULL);

   env = getenv("BAND_EXPONENT_BASE0"    ); if (env) es->heur.band_exponent_base   [0] = strtof(env,NULL);
   env = getenv("BAND_EXPONENT_BASE1"    ); if (env) es->heur.band_exponent_base   [1] = strtof(env,NULL);
   env = getenv("BAND_COUNT_EXPONENT0"   ); if (env) es->heur.band_count_exponent  [0] = strtof(env,NULL);
   env = getenv("BAND_COUNT_EXPONENT1"   ); if (env) es->heur.band_count_exponent  [1] = strtof(env,NULL);
   env = getenv("QUALITY_WEIGHT_LOW0"    ); if (env) es->heur.quality_weight_low   [0] = strtof(env,NULL);
   env = getenv("QUALITY_WEIGHT_LOW1"    ); if (env) es->heur.quality_weight_low   [1] = strtof(env,NULL);
   env = getenv("EXTRA_THRESHOLD_BIG0"   ); if (env) es->heur.large_boost_median_test[0] = strtof(env,NULL);
   env = getenv("EXTRA_THRESHOLD_BIG1"   ); if (env) es->heur.large_boost_median_test[1] = strtof(env,NULL);
   env = getenv("EXTRA_THRESHOLD_SMALL0" ); if (env) es->heur.small_boost_median_test[0] = strtof(env,NULL);
   env = getenv("EXTRA_THRESHOLD_SMALL1" ); if (env) es->heur.small_boost_median_test[1] = strtof(env,NULL);

   env = getenv("SHORT_BLOCK_PULSES"     ); if (env) es->heur.short_block_pulse_scale    = strtof(env,NULL);
   #if 0
   env = getenv("BAND_MASK_8"            ); if (env) es->heur.band_mask_8          [0] = strtof(env,NULL);
   env = getenv("BAND_MASK_4"            ); if (env) es->heur.band_mask_4          [0] = strtof(env,NULL);
   env = getenv("BAND_MASK_2"            ); if (env) es->heur.band_mask_2          [0] = strtof(env,NULL);
   env = getenv("BAND_MASK_1"            ); if (env) es->heur.band_mask_1          [0] = strtof(env,NULL);
   #endif

   float mantissa_scale[2] = { 1.00,1.00f };
   env = getenv("MANTISSA_SCALE_LONG"    ); if (env) mantissa_scale[0] = strtof(env,NULL);
   env = getenv("MANTISSA_SCALE_SHORT"   ); if (env) mantissa_scale[1] = strtof(env,NULL);

   float band_mantissa_base0 = 0, band_mantissa_decay0 = 0;
   int band_mantissa_slot=-1;

   env = getenv("BAND_MANTISSA_BASE0_I" ); if (env) band_mantissa_base0    = strtof(env,NULL);
   env = getenv("BAND_MANTISSA_DECAY0_I"); if (env) band_mantissa_decay0   = strtof(env,NULL);
   env = getenv("BAND_MANTISSA_I"       ); if (env) band_mantissa_slot     = atoi(env);

   for (int i=0; i < radaudio_rateinfo[0][rate_mode].num_bands; ++i) {
      F32 base0  = band_mantissa_base [qmode][0][i/4];
      F32 base1  = band_mantissa_base [qmode][1][i/4];
      F32 decay0 = band_mantissa_decay[qmode][0][i/4];
      F32 decay1 = band_mantissa_decay[qmode][1][i/4];
      if (i/4 == band_mantissa_slot) {
         base0  = band_mantissa_base0;
         decay0 = band_mantissa_decay0;
      }
      base0  *= mantissa_scale[0];
      base1  *= mantissa_scale[1];
      decay0 *= mantissa_scale[0];
      decay1 *= mantissa_scale[1];

      h->mantissa_param[0][i][0] = (S8) ( base0 *   8 + 0.5);
      h->mantissa_param[1][i][0] = (S8) ( base1 *   8 + 0.5);
      h->mantissa_param[0][i][1] = (S8) (decay0 * 256 + 0.5);
      h->mantissa_param[1][i][1] = (S8) (decay1 * 256 + 0.5);
   }

   float pulse_value_lo = 5.0;
   float pulse_value_hi = 5.0;
   int pulse_range_lo = -1;
   int pulse_range_hi = -1;

   env = getenv("PULSE_VALUE_LO"         ); if (env) pulse_value_lo         = strtof(env,NULL);
   pulse_value_hi = pulse_value_lo;
   env = getenv("PULSE_VALUE_HI"         ); if (env) pulse_value_hi         = strtof(env,NULL);
   env = getenv("PULSE_RANGE_LO"         ); if (env) pulse_range_lo         = atoi(env);
   env = getenv("PULSE_RANGE_HI"         ); if (env) pulse_range_hi         = atoi(env);

   float subband_scale =  1.0f;
   env = getenv("SUBBAND_SCALE"          ); if (env) subband_scale = strtof(env,NULL);
   for (int i=0; i < radaudio_rateinfo[0][rate_mode].num_bands; ++i) {
      float pulses_per = subband_pulses_for_band[rate_mode][qmode][i] * subband_scale;
      if (i >= pulse_range_lo && i <= pulse_range_hi) {
         pulses_per = linear_remap((float) i, (float) pulse_range_lo, (float) pulse_range_hi, (float) pulse_value_lo, (float) pulse_value_hi);
      }
      int predicted_subband_sum =  (int) (pulses_per * radaudio_rateinfo[0][rate_mode].num_subbands_for_band[i] + 0.5f);
      h->subband_predicted_sum[i] = (U8) RR_MIN(255, predicted_subband_sum);
   }
}
#endif

static radaudio_nonzero_blockmode_descriptor nz_encode[RADAUDIO_NUM_RATES][10][NUM_SELECTOR_MODES] =
{
   // 48 Khz
   { 
      { { 0 } }, // 0
      { { 0 } }, // 1
      { { 0 } }, // 2
      { { 0 } }, // 3
      { { 0 } }, // 4
      // bitrate:5
      { 
         { 4, { 0,2,1,1     }, },
         { 6, { 0,2,1,1,1,1 }, },
         { 2, { 2,1         }, },
         { 0,                  },
         { 0,                  }
      },
      // bitrate:6
      { 
         { 6, { 0,2,1,1,1,1 }, },
         { 4, { 0,2,1,1     }, },
         { 2, { 2,1         }, },
         { 0,                  },
         { 0,                  }
      },
      // bitrate:7
      { 
         { 8, { 0,0,2,1,1,1,1,1, }, },
         { 6, { 0,2,1,1,1,1      }, },
         { 2, { 5,1              }, },
         { 0,                       },
         { 0,                       }
      },
      // bitrate:8
      { 
         { 12, { 0,0,0,0, 0,0,2,1, 2,1,2,1 }, { 0,0,0,0, 0,0,0,1, }, },
         { 11, { 0,0,1,1,1,1, 1,1,1,1,1    }, { 0,0,1,1, 1,1,1    }, },
         {  4, { 5,5,1,1                   },                        },
         {  0,                                                       },
         {  0,                                                       }
      },
      // bitrate:9
      { 
         { 12, { 4,4,4,4, 4,4,4,4, 4,1,4,0 }, { 0,0,0,0, 0,0,0,0, 0,1,0,0 } },
         { 12, { 4,4,5,5, 5,5,5,5, 5,1,5,1 }, { 0,0,0,0, 0,0,0,0, 0,0,0,1 } },
         {  6, { 5,5,5,5, 1,1              },                               },
         {  0,                                                              },
         {  0,                                                              }
      },
   },

   // 44.1 Khz
   { 
      { { 0 } }, // bitrate:0
      { { 0 } }, // 1
      { { 0 } }, // 2
      { { 0 } }, // 3
      { { 0 } }, // 4
      // bitrate:5
      { 
         { 4, { 0,2,1,1     }, },
         { 2, { 2,1         }, },
         { 1, { 1           }, },
         { 0,                  },
         { 0,                  }
      },
      // bitrate:6
      { 
         { 5, { 0,2,1,1,1   }, },
         { 3, { 0,2,1,      }, },
         { 2, { 5,1         }, },
         { 0,                  },
         { 0,                  }
      },
      // bitrate:7
      { 
         { 7, { 0,0,2,2,1,1,1    }, },
         { 6, { 0,2,1,1,1,1      }, },
         { 2, { 5,1              }, },
         { 0,                       },
         { 0,                       }
      },
      // bitrate:8
      { 
         { 12, { 4,0,0,0, 2,2,2,3, 2,2,2,2, }, },
         { 11, { 4,0,5,5, 5,1,1,1, 1,1,1    }, },
         {  5, { 5,5,1,1, 1                 }, },
         {  0,                                 },
         {  0,                                 }
      },
      // bitrate:9
      { 
         { 12, { 3,3,4,4,4,4,4,0,4,0,5,4 }, },
         { 12, { 3,4,5,5,5,5,5,5,5,5,5,5 }, },
         {  6, { 5,5,5,5,5,1             }, },
         {  0,                              },
         {  0,                              }
      },
   },

   // 32 Khz
   { 
      { { 0 } }, // bitrate:0
      { { 0 } }, // 1
      { { 0 } }, // 2
      { { 0 } }, // 3
      { { 0 } }, // 4
      // bitrate:5
      { 
         { 6, { 0,2,2,1,1,1 }, },
         { 5, { 0,2,1,1,1,1 }, },
         { 2, { 5,1         }, },
         { 0,                  },
         { 0,                  }
      },
      // bitrate:6
      { 
         { 8, { 0,0,2,1,1,1,1,1  }, },
         { 6, { 0,2,1,1,1,1      }, },
         { 2, { 5,1              }, },
         { 0,                       },
         { 0,                       }
      },
      // bitrate:7
      { 
         { 12, { 0,0,2,2, 2,1,1,1, 1,1,1,1 },           },
         { 10, { 0,2,1,1, 1,1,1,1, 1,1     }, { 0,0,1 } },
         {  5, { 0,1,1,1, 1                },           },
         {  0,                                          },
         {  0,                                          }
      },
      // bitrate:8
      { 
         { 12, { 4,0,0,0, 0,0,0,2, 0,2,2,5 },                   },
         { 12, { 0,0,1,1, 1,1,1,1, 1,1,1,1 }, { 0,0,1,1,1,1,1 } },
         {  3, { 5,5,1,                    },                   },
         {  0,                                                  },
         {  0,                                                  }
      },
      // bitrate:9
      { 
         { 12, { 3,3,4,4,4,4,4,4,4,4,4,4 }, },
         { 12, { 3,4,5,5,5,5,5,5,5,5,5,5 }, },
         {  6, { 4,5,5,5,5,5             }, },
         {  0,                              },
         {  0,                              }
      },
   },

   // 24 Khz
   { 
      { { 0 } }, // bitrate:0
      { { 0 } }, // 1
      { { 0 } }, // 2
      { { 0 } }, // 3
      { { 0 } }, // 4
      // bitrate:5
      { 
         { 9, { 0,0,1,1, 1,1,1,1, 1 }, { 0,0,1 } },
         { 6, { 0,2,1,1,1,1         },           },
         { 2, { 5,1                 },           },
         { 0,                                    },
         { 0,                                    }
      },
      // bitrate:6
      { 
         { 12, { 0,0,1,1, 1,1,1,1, 1,1,1,1 }, { 0,0,1 } },
         { 12, { 0,0,2,2, 2,1,1,1, 1,1,1,1 },           },
         {  7, { 0,2,1,1, 1,1,1            },           },
         {  0,                                          },
         {  0,                                          }
      },
      // bitrate:7
      { 
         { 12, { 0,0,5,5, 5,5,1,1, 1,1,1,1 },                               },
         { 12, { 0,0,0,0, 0,2,2,2, 2,1,2,2 }, { 0,0,0,0, 0,0,0,0, 0,1,0,0 } },
         {  5, { 5,5,1,1, 1                },                               },
         {  0,                                                              },
         {  0,                                                              }
      },
      // bitrate:8
      { 
         { 12, { 4,4,4,4, 4,4,4,4, 0,4,4,5 }, },
         { 12, { 4,4,5,5, 5,5,5,5, 5,5,5,5 }, },
         {  7, { 5,5,1,1, 1                }, },
         {  0,                                },
         {  0,                                }
      },
      // bitrate:9
      { 
         { 12, { 3,3,3,3,3,4,4,3,3,4,3,3 }, },
         { 12, { 3,3,5,5,5,5,5,5,5,5,5,5 }, },
         { 12, { 5,5,5,5,5,5,5,5,5,5,5,5 }, },
         {  0,                              },
         {  0,                              }
      },
   },
};

static U8 nz_mode_correlated_selectors_pair[4][10][NUM_SELECTOR_MODES] =
{
   {
      { 0,0,2,2,0,},
      { 0,0,2,2,0,},
      { 0,0,2,2,0,},
      { 0,0,2,2,0,},
      { 0,0,2,2,0,},
      { 0,0,2,2,0,},
      { 0,2,1,2,0,},
      { 3,2,1,1,3,},
      { 1,1,1,1,1,},
      { 1,1,1,3,1,},
   },{
      { 0,2,2,2,0,},
      { 0,2,2,2,0,},
      { 0,2,2,2,0,},
      { 0,2,2,2,0,},
      { 0,2,2,2,0,},
      { 0,2,2,2,0,},
      { 0,2,2,1,0,},
      { 3,2,1,1,3,},
      { 1,1,1,1,1,},
      { 1,1,1,3,1,},
   },{
      { 0,2,1,1,0,},
      { 0,2,1,1,0,},
      { 0,2,1,1,0,},
      { 0,2,1,1,0,},
      { 0,2,1,1,0,},
      { 0,2,1,1,0,},
      { 0,2,1,1,0,},
      { 3,2,1,1,3,},
      { 1,1,1,1,1,},
      { 1,1,1,3,1,},
   },{
      { 3,1,1,1,3,},
      { 3,1,1,1,3,},
      { 3,1,1,1,3,},
      { 3,1,1,1,3,},
      { 3,1,1,1,3,},
      { 3,1,1,1,3,},
      { 2,3,1,1,3,},
      { 1,1,1,1,1,},
      { 1,1,1,1,1,},
      { 1,1,1,1,1,},
   }
};

static U8 nz_mode_correlated_selectors_big[4][10][NUM_SELECTOR_MODES] =
{
   {
      { 0,0,0,1,0,},
      { 0,0,0,1,0,},
      { 0,0,0,1,0,},
      { 0,0,0,1,0,},
      { 0,0,0,1,0,},
      { 0,0,0,1,0,},
      { 0,0,1,1,0,},
      { 0,1,1,2,0,},
      { 0,1,1,2,1,},
      { 1,2,2,2,1,},
   },{
      { 0,0,1,1,0,},
      { 0,0,1,1,0,},
      { 0,0,1,1,0,},
      { 0,0,1,1,0,},
      { 0,0,1,1,0,},
      { 0,0,1,1,0,},
      { 0,0,1,1,0,},
      { 0,1,1,2,0,},
      { 0,1,2,2,1,},
      { 2,2,2,3,2,},
   },{
      { 0,1,2,2,0,},
      { 0,1,2,2,0,},
      { 0,1,2,2,0,},
      { 0,1,2,2,0,},
      { 0,1,2,2,0,},
      { 0,1,2,2,0,},
      { 0,1,2,2,0,},
      { 0,1,2,2,0,},
      { 1,2,2,2,1,},
      { 2,3,3,3,2,},
   },{
      { 1,2,2,2,1,},
      { 1,2,2,2,1,},
      { 1,2,2,2,1,},
      { 1,2,2,2,1,},
      { 1,2,2,2,1,},
      { 1,2,2,2,1,},
      { 1,1,2,2,1,},
      { 2,0,2,2,1,},
      { 1,2,2,3,1,},
      { 3,3,3,3,3,},
   }
};

static void set_nz_desc(radaudio_nonzero_blockmode_descriptor nz_desc[NUM_NZ_MODE], U8 nz_correlated_huffman_selectors[NUM_NZ_SELECTOR][NUM_SELECTOR_MODES], int ratemode, int quality_mode)
{
   // quality under 5 is untuned, so just use 5
   if (quality_mode < 5)
      quality_mode = 5;
   memcpy(nz_desc, nz_encode[ratemode][quality_mode], sizeof(nz_desc[0]) * NUM_NZ_MODE);
   memset(nz_correlated_huffman_selectors, 0, NUM_NZ_SELECTOR*NUM_SELECTOR_MODES);
   memcpy(nz_correlated_huffman_selectors[HS_COEFF_PAIR], nz_mode_correlated_selectors_pair[ratemode][quality_mode], NUM_SELECTOR_MODES);
   memcpy(nz_correlated_huffman_selectors[HS_COEFF_BIG ], nz_mode_correlated_selectors_big [ratemode][quality_mode], NUM_SELECTOR_MODES);
}

// returns 1 on success, or 0 if inputs are invalid or internal error
size_t radaudio_encode_create_internal(radaudio_encoder *rae,
                                U8 header[RADAUDIO_STREAM_HEADER_MAX],
                                int num_channels,      // 1..2
                                int sample_rate,       // in HZ
                                int qmode,             // 0..9
                                float quality_pulse,
                                U32 flags)             // used for ratesearch during development
{
   radaudio_encoder_state *es = (radaudio_encoder_state *) rae;
   int i, rate_mode;
   size_t pack_length, unpack_length;

   if (qmode > 9)
      qmode = 9;
   if (qmode < 0)
      qmode = 0;

   // need to know samprate mode before we can fill full header, so do a first conversion

   rate_mode = radaudio_code_sample_rate(sample_rate);
   if (rate_mode < 0)
      return 0;

   memset(es, 0, sizeof(*es));

   es->quality_mode   = (U8) qmode;
   es->cpu            = cpu_detect();

   es->num_channels   = num_channels;
   es->sample_rate    = sample_rate;
   es->samprate_mode  = rate_mode;
   es->allow_mid_side = true;

   if (quality_pulse == 0)
      es->heur.pulse_quality = pulse_quality_table[es->samprate_mode][es->quality_mode];
   else
      es->heur.pulse_quality = quality_pulse / 100.0f;

   for (i=0; i < 2; ++i) {
      es->heur.band_exponent_base [i] = band_exponent_base [rate_mode][i][qmode];
      es->heur.band_count_exponent[i] = band_count_exponent[rate_mode][i][qmode];
      es->heur.quality_weight_low [i] = quality_weight_low [rate_mode][i][qmode];
      es->heur.large_boost_median_test[i] = extra_threshold_big  [rate_mode][i][qmode];
      es->heur.small_boost_median_test[i] = extra_threshold_small[rate_mode][i][qmode];
   }

   es->heur.short_block_pulse_scale = short_block_pulses[rate_mode][qmode];

   es->heur.side_exp_threshold_all =   3;
   es->heur.side_exp_start2_all    =   6;
   es->heur.side_exp_threshold     =   2;
   es->heur.side_exp_end_all       = -13;
   es->heur.side_exp_start_all     = -15;
   es->heur.side_exp_start         = -15;
   es->heur.side_exp_start2        =   4;
   es->heur.side_exp_end           =  -13;

   es->heur.mid_side_tiny          = -15;
   es->heur.mid_side_offset        = - 4;
   es->heur.mid_side_threshold     = -16;
   es->heur.mid_side_max_bad_bands =   6;

   es->heur.expectation_base       = -16;
   es->heur.expectation_scale      = 0.195f;
   es->heur.short_overlap_scale1   = 1.0f;
   es->heur.short_overlap_scale2   = 1.0f;

   set_nz_desc(es->nz_desc, es->nz_correlated_huffman_selectors, rate_mode, qmode);

   radaudio_stream_header_unpacked h;
   memset(&h, 0, sizeof(h));

   h.num_channels     = num_channels;
   h.sample_rate      = sample_rate;
   h.version          = ENCODER_VERSION;
   h.bytes_bias       = header_size_bias[qmode];
   compute_bias_set(&es->biases, h.bytes_bias);

   for (i=0; i < radaudio_rateinfo[0][rate_mode].num_bands; ++i) {
      F32 base0,base1,decay0,decay1;
      if (flags & RADAUDIO_ENC_FLAG_improve_seamless_loop) {
         base0  = band_mantissa_base_improved_short_dc [qmode][0][i/4];
         base1  = band_mantissa_base_improved_short_dc [qmode][1][i/4];
         decay0 = band_mantissa_decay_improved_short_dc[qmode][0][i/4];
         decay1 = band_mantissa_decay_improved_short_dc[qmode][1][i/4];
      } else {
         base0  = band_mantissa_base [qmode][0][i/4];
         base1  = band_mantissa_base [qmode][1][i/4];
         decay0 = band_mantissa_decay[qmode][0][i/4];
         decay1 = band_mantissa_decay[qmode][1][i/4];
      }

      h.mantissa_param[0][i][0] = (S8) ( base0 *   8 + 0.5);
      h.mantissa_param[1][i][0] = (S8) ( base1 *   8 + 0.5);
      h.mantissa_param[0][i][1] = (S8) (decay0 * 256 + 0.5);
      h.mantissa_param[1][i][1] = (S8) (decay1 * 256 + 0.5);
   }

   for (i=0; i < radaudio_rateinfo[0][rate_mode].num_bands; ++i) {
      float pulses_per = subband_pulses_for_band[rate_mode][qmode][i];
      int predicted_subband_sum =  (int) (pulses_per * radaudio_rateinfo[0][rate_mode].num_subbands_for_band[i] + 0.5f);
      h.subband_predicted_sum[i] = (U8) RR_MIN(255, predicted_subband_sum);
   }

   for (i=0; i < NUM_NZ_MODE; ++i) {
      h.nzmode_num64[i] = es->nz_desc[i].num_8byte_chunks;
      for (int j=0; j < MAX_NZ_BLOCKS; ++j)
         h.nzmode_huff[i][j] = es->nz_desc[i].huffman_table_for_chunk[j] | (es->nz_desc[i].invert_chunk[j] ? NZ_MODE_INVERT : 0);
   }
   for (int j=0; j < NUM_NZ_SELECTOR; ++j)
      for (i=0; i < NUM_SELECTOR_MODES; ++i)
         h.nzmode_selectors[j][i] = es->nz_correlated_huffman_selectors[j][i];

   #ifdef RADAUDIO_DEVELOPMENT
   radaudio_load_heuristics_from_environment_variables(es, &h);
   #endif

   pack_length = radaudio_pack_stream_header(header, &h);
   if (pack_length == 0)
      return 0;

   unpack_length = radaudio_unpack_stream_header(header, RADAUDIO_STREAM_HEADER_MAX, &h);
   if (unpack_length != pack_length)
      return 0;


   memcpy(es->subband_predicted_sum, h.subband_predicted_sum, 24);
   memcpy(es->mantissa_param       , h.mantissa_param       ,    sizeof(es->mantissa_param ));
   memcpy(es->subband_bias         , h.subband_bias         , 24*sizeof(es->subband_bias[0]));

   for (i=0; i < 2; ++i)
      es->info[i] = &radaudio_rateinfo[i][es->samprate_mode];

   radaudio_init_nz_desc(es->nz_desc);

   return unpack_length;
}

size_t radaudio_encode_create(radaudio_encoder *es, U8 header[RADAUDIO_STREAM_HEADER_MAX], int num_channels, int sample_rate, int quality, U32 flags)
{
   return radaudio_encode_create_internal(es, header, num_channels, sample_rate, quality, 0.0f, flags);
}

#ifdef RADAUDIO_DEVELOPMENT
// internal use
int RadAudioCompressGetProfileData(radaudio_encoder *hradaud, radaudio_eprofile_value *profile, int num_profile)
{
   radaudio_encoder_state *es = (radaudio_encoder_state *) hradaud;
   int n = RR_MIN(num_profile, PROF_total_count);
   static const char *names[] = {
      #define PROF(x) #x,
      PROFILE_ZONES()
      #undef PROF
   };
   for (int i=0; i < n; ++i) {
      profile[i].name = names[i];
      profile[i].time = rrTicksToSeconds(es->profile_times[i]);
   }
   return n;
}
#else
int RadAudioCompressGetProfileData(radaudio_encoder *hradaud, radaudio_eprofile_value *profile, int num_profile)
{
   RR_UNUSED_VARIABLE(hradaud); RR_UNUSED_VARIABLE(profile); RR_UNUSED_VARIABLE(num_profile);
   return 0;
}
#endif
