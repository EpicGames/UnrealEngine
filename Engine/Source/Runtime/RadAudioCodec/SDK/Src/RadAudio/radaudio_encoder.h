// Copyright Epic Games Tools, LLC. All Rights Reserved.
#ifndef INCLUDE_RADAUDIO_ENCODER_H
#define INCLUDE_RADAUDIO_ENCODER_H

#include "egttypes.h"
#include <stddef.h>

// for feedback to user. actual rate varies, and is sample rate dependent
// the 0th value is support but should never be used; it allows you to hear what the native artifacts sound like
static const int approximate_data_rate_for_quality_setting_in_kilobits[2][10] =
{
   { 30, 36, 39, 43, 48, 55, 65, 85,125,205 }, // mono (@TODO: measure these, currently just ~half of stereo)
   { 55, 64, 72, 80, 90,100,120,160,240,400 }, // stereo
};

// encode context
typedef struct
{
   // placeholder fields to make it large enough
   RAD_U64 dummy[200];
   U8 buffer[5000];
} radaudio_encoder;

#define RADAUDIOENC_AT_EOF                0
#define RADAUDIOENC_INSUFFICIENT_BUFFER  -1
#define RADAUDIOENC_INTERNAL_ERROR       -2
#define RADAUDIOENC_MAX_OUTPUT_SAMPLES_PER_CHANNEL_PER_CHUNK    1024

RADDEFSTART

// RADAUDIO compatibility version - if these are the same the exports are named the same so we expect
// that if the linker selects a different copy they all work.
#define RADAUDIO_ENC_LIBRARY_VERSION    1

#ifndef RR_STRING_JOIN3
#define RR_STRING_JOIN3(arg1, arg2, arg3)            RR_STRING_JOIN_DELAY3(arg1, arg2, arg3)
#define RR_STRING_JOIN_DELAY3(arg1, arg2, arg3)      RR_STRING_JOIN_IMMEDIATE3(arg1, arg2, arg3)
#define RR_STRING_JOIN_IMMEDIATE3(arg1, arg2, arg3)  arg1 ## arg2 ## arg3
#endif

#ifdef RADAUDIO_WRAP
#define RADAUDIO_ENC_NAME(name) RR_STRING_JOIN3(RADAUDIO_WRAP, name##_, RADAUDIO_ENC_LIBRARY_VERSION )
#else
#define RADAUDIO_ENC_NAME(name) RR_STRING_JOIN( name##_, RADAUDIO_ENC_LIBRARY_VERSION )
#endif

#define radaudio_encode_create                              RADAUDIO_ENC_NAME(radaudio_encode_create)
#define radaudio_encode_block                               RADAUDIO_ENC_NAME(radaudio_encode_block)
#define radaudio_encode_create_internal                     RADAUDIO_ENC_NAME(radaudio_encode_create_internal)
#define RadAudioCompressGetProfileData                      RADAUDIO_ENC_NAME(RadAudioCompressGetProfileData)
#define radaudio_determine_preferred_next_block_length      RADAUDIO_ENC_NAME(radaudio_determine_preferred_next_block_length)
#define radaudio_determine_preferred_first_block_length     RADAUDIO_ENC_NAME(radaudio_determine_preferred_first_block_length)
#define radaudio_encode_block_ext                           RADAUDIO_ENC_NAME(radaudio_encode_block_ext)

// radaudio_encode_create()
//
// Creates a new RADaudio encoder.
//
// Pass in an uninitialized radaudio_encoder structure, space to output the stream header,
// and then a description of the stream and the quality settings desired.
//
// - `num_channels` must be 1 or 2
// - `sample_rate` must be 48000, 44100, 32000, or 24000
//   - if the automated quality measurement is to be trusted, then 32K and 24K encodings are
//     lower quality at same size than converting the files to 44.1K or 48K
//     (but ogg vorbis shows same behavior, so maybe automatic quality measurement is wrong for 32K and 24K)
// - `quality` should be 1..9.
//   - larger values are higher quality and larger files
//   - 5 is recommended setting; it gives comparable results to vorbis 96 kbps
//   - going above 5 gives larger, higher quality files; could default to 6 if paranoid
//   - settings below 5 have not been tuned, as they are not expected to be "transparent" quality
//     - they are only provided for people who want to experiment and explore what it sounds like
// - `header` is a buffer to store the header
//   - supply RADAUDIO_STREAM_HEADER_MAX
//   - the actual size of the header is returned
//
// returns size of header on success, otherwise returns 0
//
// there is no destroy() function as the structure holds no other resources
// just free the memory you passed in as necessary
#define RADAUDIO_STREAM_HEADER_MAX    128
RADDEFFUNC size_t radaudio_encode_create( radaudio_encoder *rae,   
                                          U8  header[RADAUDIO_STREAM_HEADER_MAX],
                                          int num_channels,
                                          int sample_rate,
                                          int quality,
                                          U32 flags);

#define RADAUDIO_ENC_FLAG_improve_seamless_loop    1  // actually boosts precision of short block low-frequency coefficients


// radaudio_encode_block()
//
// Generate a block of audio data in RADaudio format.
//
// Pass in a radaudio_encoder initialized by radaudio_encode_init.
//
// - `encode_buffer`     :  storage where the encoder will write compressed data
// - `encode_buffer_max` :  size of encode buffer
// - `input`             :  an array of audio samples
//      - values from [-1..1] (can exceed this range, but will distort if you go too far)
//      - for stereo input, interleave the channels in a single array
// - `input_len`         :  the number of mono samples or stereo sample pairs in the input
// - `offset`            :  the current encoding offset within the input buffer (in mono samples or stereo pairs)
//   - initialize to 0 for the first call
//   - the encoder will encode samples in the range up to [offset-1024, offset+1024)
//   - the encoder will potentially look at samples up to [offset-1024, offset+2047)
//   - the encoder will update `*offset` to reflect the number of samples partially encoded
//   - samples up to `*offset` are only fully encoded after the NEXT encoder step
//
// returns:
//              n > 0                   number of bytes of data output for this block
//    RADAUDIOENC_AT_EOF                at end of stream, previously generated block is last block
//    RADAUDIOENC_INSUFFICIENT_BUFFER   if output buffer isn't large enough
//    RADAUDIOENC_INTERNAL_ERROR        an internal error occurred. this is probably a bug
RADDEFFUNC int radaudio_encode_block(radaudio_encoder *es,
                                      float *input,
                                      size_t input_len, // in samples (stereo pairs count as one)
                                      size_t *offset  , // in samples (stereo pairs count as one)
                                      unsigned char *encode_buffer,  // recommend MAX_ENCODED_BLOCK_SIZE
                                      size_t encode_buffer_size);
// In normal use, just load an audio file, convert to float, and pass the obvious
// values to input / input_len, and allow the encoder to control the value of `*offset`.
//
// It is possible to use this API to stream input data by controlling
// `input`, `input_len, and `*offset`.
//
//   - you can freely manipulate the value of `*offset`.
//   - the encoder will look at samples i in [*offset-1024, *offset+1024)
//     - with the caveat that 0 <= i < input_len (samples outside this range are treated as 0, and reaching input_len ends the stream)
//   - The initial value of `*offset` must be 0.
//
// So, you could do something like:
//
//   - Keep track of the actual offset within the file where `*offset` indexes; call this `fo`, set to 0.
//   - Iterate:
//     - set `*offset` to 1024, so [input..input+2048) is used, aka input[offset-1024, offset+1024)
//     - set `input_len` = subtract `fo` from the actual length of the file in mono samples or stereo pairs, add *offset
//     - load the file data from [fo-1024,fo+1024) into input[0..2048)
//     - encode a block
//     - update `fo` by the delta change in `*offset`
//     - note that the encoder will read the overlapping samples in each pair of successive blocks
//       - it is crucial that those samples be identical each time
//       - the naive approach, and the above strategy, both make sure this happens

#define MAX_ENCODED_BLOCK_SIZE   5000 // 5000 plus slop due to forgetting to count stuff
// upper bound on largest size; in reality, 400 Kbps is less than 2KB per block
//
// largest possible block:
//        max header size                     =>                        8 bytes
//     48 largest-encoding band exponents@11b =>   480 +   48 bits
//     48 largest-encoding band mantissa @16b =>                       96 bytes
//    112 largest-encoding subbands      @11b =>  1120 +  112 bits
//   1536 run-length entries of 0             =>         1536 bits
//        run-length end of stream code   @7b =>            7 bits
//    512 1-bit coefficient locations w/huff  =>          576 bits
//   1024 signals for 1-byte coefficients@11b => 10240 + 1024 bits  
//   2048 coefficients encoded 11 bits        => 20480 + 2048 bits
//      3 streams of padding                  =>                        3 bytes
//                                               32320 + 5351 bits +  107 bytes
//                                               == 4816 bytes

typedef enum
{
   RADAUDIO_BLOCKTYPE_short = 1,
   RADAUDIO_BLOCKTYPE_long  = 2,

   // radaudio_encode_block_ext only
   RADAUDIO_BLOCKTYPE_default = 0 // use transient detector like normal
} radaudio_blocktype;

typedef struct
{
   radaudio_blocktype force_first_blocktype; // only used on first call
   radaudio_blocktype force_next_blocktype;  // used on all calls

   // you can achieve seamless looping using the following fields:
   // use this as the data to pad the beginning and/or end of the stream with, instead of 0s
   F32 *padding;
   size_t padding_len; // in samples (stereo pairs count as one). Set to 0 to disable.
   //
   // At the beginning and end of the stream, based on offset going from 0..input_len, the above
   // data will be used to "pad" the audio stream instead of the 0 value used by default.
   //
   // - at the beginning of the stream, the END of the padding block will be used, pushed up against the start of the stream audio
   // - at the end of the stream, the BEGINNING of the padding block will be used, pushed right up against the end of the stream audio
   //
   // for best results when looping, supply 2048 samples, or the entire stream if the stream is shorter than 2048 samples.
   // the above logic should do the right thing if the entire stream is very small, even smaller than a block
   //
   // In theory the logic should be something like this (assuming the stream is > 2048 samples):
   //    if (offset < 1024)
   //        point padding at end of stream
   //    if (offset > length-2048)
   //        point padding at beginning of stream
   //
   // In theory the padding data should only be needed in the first block and the last two blocks, but
   // in debugging I actually saw the data accessed in the last 3 blocks. I didn't investigate since the
   // output was correct, but I suspect when this happened it was on short/long block transitions in which
   // the data in the third-to-last-block was used but windowed to 0 anyway, so it wouldn't matter if it
   // wasn't available. Also because it requires a short/long transition the above logic should cover the
   // case anyway. But maybe I'm misunderstanding and the end logic might need to be triggered further from the end--stb
} radaudio_encode_info;

RADDEFFUNC int radaudio_encode_block_ext(radaudio_encoder *es,
                                      float *input,
                                      size_t input_len, // in samples (stereo pairs count as one)
                                      size_t *offset  , // in samples (stereo pairs count as one)
                                      unsigned char *encode_buffer,  // recommend MAX_ENCODED_BLOCK_SIZE
                                      size_t encode_buffer_size,
                                      radaudio_encode_info *info);

// determine the preferred blocktype for the first block in the stream;
// pass this in as radaudio_encode_info.force_first_blocktype.
// you can specify a different value than this function returns by overriding radaudio_encode_info,
// but specifying long when short is requested will decrease quality, and
// specifying short when long is requested will increase rate
RADDEFFUNC radaudio_blocktype radaudio_determine_preferred_first_block_length(radaudio_encoder *rae,
                                      F32 *input,
                                      size_t input_len);

// determine the preferred blocktype for the next block in the stream;
// this is stateful; it assumes `offset` points into the middle of the block to encode, as described for encode_block
// you can specify a different value than this function returns by overriding radaudio_encode_info,
// but specifying long when short is requested will decrease quality, and
// specifying short when long is requested will increase rate
RADDEFFUNC int radaudio_determine_preferred_next_block_length(radaudio_encoder *rae,
                                      radaudio_blocktype firsttype,
                                      F32 *input,
                                      size_t input_len,
                                      size_t offset);


RADDEFEND

#endif//INCLUDE_RADAUDIO_ENCODER_H
