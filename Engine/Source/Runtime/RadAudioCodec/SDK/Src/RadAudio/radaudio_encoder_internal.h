// Copyright Epic Games Tools, LLC. All Rights Reserved.
#ifndef INCLUDE_RADAUDIO_ENCODER_INTERNAL_H
#define INCLUDE_RADAUDIO_ENCODER_INTERNAL_H

#include "radaudio_common.h"

typedef struct
{
   size_t bit_allocation[8];
   U32    block_events  [16];
} encode_stats;

enum
{
   S_header,
   S_band_exponent,
   S_band_mantissa,
   S_subband,
   S_coeff_location,
   S_coeff_value,
   S_coeff_value_large,
   S_padding,

   S_count
};

enum
{
   E_predict_band_stereo,
   E_predict_subband_stereo,
   E_stereo_as_mono,
   E_subband_nopredict,
   E_subband_renormalize,
   E_coefficients_renormalize,
   E_many_big_coefficients,
   E_toomany_big_coefficients,
   E_nzmode0,
   E_nzmode1,
   E_nzmode2,
   E_nzmode3,

   E_count
};

typedef struct coefficient_encode_heuristics
{
   // base calculation for number of pulses to use
   float pulse_quality;

   // short block total pulses are reduced compared to long blocks
   float short_block_pulse_scale;

   // weight coefficient pulse counts by band
   float band_exponent_base [2];   // weight by pow(band_exp_base, band_energy    )
   float band_count_exponent[2];  // weight by pow(num_coeff    , band_ecount_exp)
   float quality_weight_low [2];   // weight by lerp(band_index, quality_weight_low, 1.0)

   // coefficients in a subband get a boost depending on how many coefficients greatly exceed the median
   float large_boost_median_test[2];
   float small_boost_median_test[2];

   // miscellany
   float expectation_base;
   float expectation_scale;

   int side_exp_threshold_all;
   int side_exp_start2_all;
   int side_exp_start2;
   int side_exp_threshold;
   int side_exp_end_all;
   int side_exp_start_all;
   int side_exp_start;
   int side_exp_end;
   int mid_side_tiny;
   int mid_side_offset;
   int mid_side_threshold;
   int mid_side_max_bad_bands;

   float short_overlap_scale1;
   float short_overlap_scale2;
} coefficient_encode_heuristics;

typedef struct
{
   int    num_channels;

   rrbool current_block_short;
   rrbool prev_block_short;
   rrbool next_block_short;
   int    samprate_mode;          // ba2r_bitrate_code

   int    band_mantissa_exp_scale, band_mantissa_band_decay;
   rrbool allow_mid_side;

   int    sample_rate;      // implied by bitrate_mode
   int    quality_mode;
   S8     subband_bias             [MAX_BANDS]   ;
   U8     subband_predicted_sum    [MAX_BANDS]   ;
   S8     mantissa_param        [2][MAX_BANDS][2];
   U8     subband_sum_adjusted  [2][MAX_BANDS]   ;
   int    subband_bias_adjusted [2][MAX_BANDS]   ;

   U32    block_number;
   U64    samples_fully_coded;

   radaudio_block_header_biases biases;
   radaudio_cpu_features cpu;
   radaudio_rate_info * info[2];  // indexed by short/long

   radaudio_nonzero_blockmode_descriptor nz_desc[4];
   U8 nz_correlated_huffman_selectors[NUM_NZ_SELECTOR][NUM_SELECTOR_MODES];

   coefficient_encode_heuristics heur;

   U32 lastblock_block_bytes;
   U32 lastblock_vbstream0_length;
   U32 lastblock_num_runlength_array;
   encode_stats stats;
   U64 profile_times[32];

   U8 buffer[5000];
} radaudio_encoder_state;

RADDEFFUNC size_t radaudio_encode_create_internal(radaudio_encoder *es,
                            unsigned char header[64], int num_channels,
                            int sample_rate, int quality,
                            float pulse_quality_override, // 0..100, replaces normal pulse quality values
                            U32 flags);

typedef struct
{
   const char *name;
   double time;
} radaudio_eprofile_value;

RADDEFFUNC int RadAudioCompressGetProfileData(radaudio_encoder *hradaud, radaudio_eprofile_value *profile, int num_profile);

RADDEFFUNC void radaudio_hack_encoder(int *data, U8 *small, U8 *large);

#endif//INCLUDE_RADAUDIO_ENCODER_INTERNAL_H
