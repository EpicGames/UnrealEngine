// Copyright Epic Games Tools, LLC. All Rights Reserved.
#include "radaudio_common.h"
#include "radaudio_encoder_neon.h"

#ifdef RADAUDIO_WRAP
#define WRAPPED_NAME(name) RR_STRING_JOIN(RADAUDIO_WRAP, name##_)
#define radaudio_dummy_encode_neon WRAPPED_NAME(radaudio_dummy_encode_neon)
#endif


int radaudio_dummy_encode_neon(void) { return 0; } // avoid "empty translation unit" warning

#ifdef DO_BUILD_NEON

#include <arm_neon.h>

void radaudio_neon_compute_best_quantized_coeff16_loop(S16 best_quantized[], F32 best_so_far, F32 ncoeff[], F32 quantizer, F32 step_quantizer, int num_quantizers)
{
   float32x4x4_t data = vld1q_f32_x4(ncoeff);

   for (int q=0; q < num_quantizers; ++q, quantizer += step_quantizer) {
      float32x4_t v0f = vrndnq_f32(vmulq_n_f32(data.val[0], quantizer));
      float32x4_t v1f = vrndnq_f32(vmulq_n_f32(data.val[1], quantizer));
      float32x4_t v2f = vrndnq_f32(vmulq_n_f32(data.val[2], quantizer));
      float32x4_t v3f = vrndnq_f32(vmulq_n_f32(data.val[3], quantizer));

      float32x4_t sum2_0 = vdupq_n_f32(1.0e-20f); // avoid divide by 0
      float32x4_t sum2_1 = vaddq_f32(sum2_0, vmulq_f32(v0f, v0f));
      float32x4_t sum2_2 = vaddq_f32(sum2_1, vmulq_f32(v1f, v1f));
      float32x4_t sum2_3 = vaddq_f32(sum2_2, vmulq_f32(v2f, v2f));
      float32x4_t sum2_4 = vaddq_f32(sum2_3, vmulq_f32(v3f, v3f));
      float sum2 = vpadds_f32(vadd_f32(vget_low_f32(sum2_4), vget_high_f32(sum2_4)));

      float32x4_t dotproduct_1 =                         vmulq_f32(data.val[0], v0f) ;
      float32x4_t dotproduct_2 = vaddq_f32(dotproduct_1, vmulq_f32(data.val[1], v1f));
      float32x4_t dotproduct_3 = vaddq_f32(dotproduct_2, vmulq_f32(data.val[2], v2f));
      float32x4_t dotproduct_4 = vaddq_f32(dotproduct_3, vmulq_f32(data.val[3], v3f));
      float dotproduct = vpadds_f32(vadd_f32(vget_low_f32(dotproduct_4), vget_high_f32(dotproduct_4)));

      float err2 = (dotproduct * dotproduct) / sum2;
      if (err2 > best_so_far) {
         best_so_far = err2;
         int32x4_t v0 = vcvtq_s32_f32(v0f);
         int32x4_t v1 = vcvtq_s32_f32(v1f);
         int32x4_t v2 = vcvtq_s32_f32(v2f);
         int32x4_t v3 = vcvtq_s32_f32(v3f);
         int16x8x2_t v =
         {
            vcombine_s16(vqmovn_s32(v0), vqmovn_s32(v1)),
            vcombine_s16(vqmovn_s32(v2), vqmovn_s32(v3)),
         };
         vst1q_s16_x2(best_quantized, v);
      }
   }
}

// interleaves every 16-bit, 32-bit or 64-bits from two arguments
static uint16x8_t interleave16(uint16x8_t a, uint16x8_t b) { return vtrn1q_u16(a, b); }
static uint16x8_t interleave32(uint16x8_t a, uint16x8_t b) { return vreinterpretq_u16_u32(vtrn1q_u32(vreinterpretq_u32_u16(a), vreinterpretq_u32_u16(b))); }
static uint16x8_t interleave64(uint16x8_t a, uint16x8_t b) { return vreinterpretq_u16_u64(vtrn1q_u64(vreinterpretq_u64_u16(a), vreinterpretq_u64_u16(b))); }

// in the below functions, the term "4th" means arr[4], i.e. technically the 5th

int radaudio_neon_find_median_4th_of_8(S16 *data)
{
   uint16x8_t vdata = vreinterpretq_u16_s16(vld1q_s16(data));
   uint16x8_t t, a, b;

   // abcd'efgh -> badc'fehg
   t = vrev32q_u16(vdata);
   a = vminq_u16(vdata, t);
   b = vmaxq_u16(vdata, t);
   vdata = interleave16(a, b);

   // abcd'efgh -> dcba'hgfe
   t = vrev64q_u16(vdata);
   a = vminq_u16(vdata, t);
   b = vmaxq_u16(vdata, t);
   vdata = interleave32(a, b);

   // abcd'efgh -> badc'fehg
   t = vrev32q_u16(vdata);
   a = vminq_u16(vdata, t);
   b = vmaxq_u16(vdata, t);
   vdata = interleave16(a, b);

   // abcd'efgh -> hgfe'dcba
   t = vrev64q_u16(vdata);
   t = vcombine_u16(vget_high_u16(t), vget_low_u16(t));
   a = vminq_u16(vdata, t);
   b = vmaxq_u16(vdata, t);
   vdata = interleave64(a, b);

   // abcd'efgh -> cdab'ghef
   t = vreinterpretq_u16_u32(vrev64q_u32(vreinterpretq_u32_u16(vdata)));
   a = vminq_u16(vdata, t);
   b = vmaxq_u16(vdata, t);
   vdata = interleave32(a, b);

   // abcd'efgh -> badc'fehg
   t = vrev32q_u16(vdata);
   a = vminq_u16(vdata, t);
   // b = vmaxq_u16(vdata, t);
   // vdata = interleave16(a, b);

   return vgetq_lane_u16(a, 4);
}

#ifdef _MSC_VER
static RADINLINE U32 rrPopCnt32( U32 v )
{
   // from "Bit Twiddling Hacks" :
   v = v - ((v >> 1) & 0x55555555);                    // reuse input as temporary
   v = (v & 0x33333333) + ((v >> 2) & 0x33333333);     // temp
   U32 c = (((v + (v >> 4)) & 0xF0F0F0F) * 0x1010101) >> 24; // count
   return c;
}
#endif

// in the below functions, the term 8th means arr[8], i.e. technically the 9th

int radaudio_neon_find_median_8th_of_16(S16 *data)
{
   int n = 0;

   int16x8x2_t v = vld1q_s16_x2(data);
   uint16x8_t v0 = vreinterpretq_u16_s16(v.val[0]);
   uint16x8_t v1 = vreinterpretq_u16_s16(v.val[1]);

   // doesn't compile with vc so we use the below.
   //const uint16x8_t pos0 = { 0x0080, 0x0040, 0x0020, 0x0010, 0x0008, 0x0004, 0x0002, 0x0001 };
   //const uint16x8_t pos1 = { 0x8000, 0x4000, 0x2000, 0x1000, 0x0800, 0x0400, 0x0200, 0x0100 };

   static const uint16_t pos0_source[8] = { 0x0080, 0x0040, 0x0020, 0x0010, 0x0008, 0x0004, 0x0002, 0x0001 };
   static const uint16_t pos1_source[8] = { 0x8000, 0x4000, 0x2000, 0x1000, 0x0800, 0x0400, 0x0200, 0x0100 };

   const uint16x8_t pos0 = vld1q_u16(pos0_source);
   const uint16x8_t pos1 = vld1q_u16(pos1_source);


   while (n <= 8)
   {
      // find the smallest item across both
      uint16_t min = vminvq_u16(vpminq_u16(v0, v1));

      // locate that item; it might be in more than one place
      uint16x8_t mask0 = vceqq_u16(v0, vdupq_n_u16(min));
      uint16x8_t mask1 = vceqq_u16(v1, vdupq_n_u16(min));

      // count how many of them there are
      uint16x8_t bits0 = vandq_u16(pos0, mask0);
      uint16x8_t bits1 = vandq_u16(pos1, mask1);
      uint16_t count = vaddvq_u16(vpaddq_u16(bits0, bits1));

#ifdef _MSC_VER
      n += rrPopCnt32(count);
#else
      n += __builtin_popcount(count);
#endif

      // if the total number of min items we've eliminated is 8 or more, then we've found the 8th item (the median)
      if (n > 8)
      {
         return min;
      }

      // replace all minimum items with max
      v0 = vorrq_u16(v0, mask0);
      v1 = vorrq_u16(v1, mask1);
   }
   // we've replaced exactly 8 items

   // find the minimum of the remaining items and return that
   return vminvq_u16(vpminq_u16(v0, v1));
}

#endif // #if DO_BUILD_NEON
