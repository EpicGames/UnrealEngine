// Copyright Epic Games Tools, LLC. All Rights Reserved.
#include "radaudio_common.h"
#include "radaudio_encoder_sse.h"
#include <stdio.h>

#ifdef RADAUDIO_WRAP
#define WRAPPED_NAME(name) RR_STRING_JOIN(RADAUDIO_WRAP, name##_)
#define radaudio_dummy_encode_sse WRAPPED_NAME(radaudio_dummy_encode_sse)
#endif

int radaudio_dummy_encode_sse(void) { return 0; } // avoid "empty translation unit" warning

/* @cdep pre
   $when($NeedsClangMachineSwitches,
      $addlocalcswitches(-msse4.1)
   )
*/

#ifdef DO_BUILD_SSE4

#include <emmintrin.h> // SSE2

/////////////////////////////////////////////////////////////////////
//
//   SSE2 only
//

#define ps(x)  _mm_castsi128_ps(x)
#define pi(x)  _mm_castps_si128(x)

typedef __m128i int4x32;
typedef __m128i int8x16;
typedef __m128i int16x8;
typedef __m128  float4;

#define _mm_shuffle_ps_mine(a,imm) ps(_mm_shuffle_epi32(pi(a), (imm)))

#define MY_SHUFFLE(w,x,y,z) _MM_SHUFFLE(z,y,x,w)

#if defined(_MSC_VER) && _MSC_VER <= 1900
   #ifndef _mm_storeu_si32
      #define _mm_storeu_si32(addr,val)    _mm_store_ss((float *) (addr), ps(val))
   #endif
   #ifndef _mm_storeu_si64
      #define _mm_storeu_si64(addr,val)    _mm_store_sd((double *) (addr), ps(val))
   #endif
#endif

void radaudio_sse2_compute_best_quantized_coeff16_loop(S16 best_quantized[], F32 best_so_far, F32 ncoeff[], F32 quantizer, F32 step_quantizer, int num_quantizers)
{
   float4 data0 = _mm_loadu_ps(ncoeff +  0);
   float4 data1 = _mm_loadu_ps(ncoeff +  4);
   float4 data2 = _mm_loadu_ps(ncoeff +  8);
   float4 data3 = _mm_loadu_ps(ncoeff + 12);
   for (int q=0; q < num_quantizers; ++q, quantizer += step_quantizer) {
      float4 quant = _mm_set1_ps(quantizer);
      int4x32 v0  = _mm_cvtps_epi32(_mm_mul_ps(data0, quant));
      int4x32 v1  = _mm_cvtps_epi32(_mm_mul_ps(data1, quant));
      int4x32 v2  = _mm_cvtps_epi32(_mm_mul_ps(data2, quant));
      int4x32 v3  = _mm_cvtps_epi32(_mm_mul_ps(data3, quant));
      float4  v0f = _mm_cvtepi32_ps(v0);
      float4  v1f = _mm_cvtepi32_ps(v1);
      float4  v2f = _mm_cvtepi32_ps(v2);
      float4  v3f = _mm_cvtepi32_ps(v3);
      float4 sum2_0  = _mm_set1_ps(1.0e-20f); // avoid divide by 0
      float4 sum2_1  = _mm_add_ps(sum2_0, _mm_mul_ps(  v0f, v0f));
      float4 sum2_2  = _mm_add_ps(sum2_1, _mm_mul_ps(  v1f, v1f));
      float4 sum2_3  = _mm_add_ps(sum2_2, _mm_mul_ps(  v2f, v2f));
      float4 sum2_4  = _mm_add_ps(sum2_3, _mm_mul_ps(  v3f, v3f));
      float4 sum2_8  = _mm_add_ps(sum2_4, _mm_shuffle_ps_mine(sum2_4, MY_SHUFFLE(2,3,0,1)));
      float4 sum2_16 = _mm_add_ps(sum2_8, _mm_shuffle_ps_mine(sum2_8, MY_SHUFFLE(1,0,3,2)));

      float4 dotproduct_1  =                          _mm_mul_ps(data0, v0f) ;
      float4 dotproduct_2  = _mm_add_ps(dotproduct_1, _mm_mul_ps(data1, v1f));
      float4 dotproduct_3  = _mm_add_ps(dotproduct_2, _mm_mul_ps(data2, v2f));
      float4 dotproduct_4  = _mm_add_ps(dotproduct_3, _mm_mul_ps(data3, v3f));
      float4 dotproduct_8  = _mm_add_ps(dotproduct_4, _mm_shuffle_ps_mine(dotproduct_4, MY_SHUFFLE(2,3,0,1)));
      float4 dotproduct_16 = _mm_add_ps(dotproduct_8, _mm_shuffle_ps_mine(dotproduct_8, MY_SHUFFLE(1,0,3,2)));

      float4 err2_sse      = _mm_div_ss(_mm_mul_ps(dotproduct_16, dotproduct_16), sum2_16);
      float err2 = _mm_cvtss_f32(err2_sse);

      if (err2 > best_so_far) {
         best_so_far = err2;
         _mm_storeu_si128((int8x16 *) (best_quantized+0), _mm_packs_epi32(v0,v1));
         _mm_storeu_si128((int8x16 *) (best_quantized+8), _mm_packs_epi32(v2,v3));
      }
   }
}

void radaudio_sse2_fabs_coefficients(float *absval, float *data, int num_coeff)
{
   if (num_coeff == 16) {
      // most coefficients go this path
      float4 data0 = _mm_loadu_ps(data+ 0);
      float4 data1 = _mm_loadu_ps(data+ 4);
      float4 data2 = _mm_loadu_ps(data+ 8);
      float4 data3 = _mm_loadu_ps(data+12);

      float4 abs0  = _mm_and_ps(data0, ps(_mm_set1_epi32(0x7fffffff)));
      float4 abs1  = _mm_and_ps(data1, ps(_mm_set1_epi32(0x7fffffff)));
      float4 abs2  = _mm_and_ps(data2, ps(_mm_set1_epi32(0x7fffffff)));
      float4 abs3  = _mm_and_ps(data3, ps(_mm_set1_epi32(0x7fffffff)));

      _mm_storeu_ps(absval+ 0, abs0);
      _mm_storeu_ps(absval+ 4, abs1);
      _mm_storeu_ps(absval+ 8, abs2);
      _mm_storeu_ps(absval+12, abs3);
   } else if (num_coeff == 8) {
      float4 data0 = _mm_loadu_ps(data+ 0);
      float4 data1 = _mm_loadu_ps(data+ 4);
      float4 abs0  = _mm_and_ps(data0, ps(_mm_set1_epi32(0x7fffffff)));
      float4 abs1  = _mm_and_ps(data1, ps(_mm_set1_epi32(0x7fffffff)));
      _mm_storeu_ps(absval+0, abs0);
      _mm_storeu_ps(absval+4, abs1);
   } else {
      for (int i=0; i < num_coeff; i += 4) {
         float4 data0 = _mm_loadu_ps(data+ 0);
         float4 abs0  = _mm_and_ps(data0, ps(_mm_set1_epi32(0x7fffffff)));
         _mm_storeu_ps(absval+i, abs0);
      }
   }
}

//////////////////////////////////////////////////////////////////////////////
//
//  functions requiring SSSE3 are below this point
//  primarily PSHUFB aka _mm_shuffle_epi8
//

#include <tmmintrin.h> // SSSE3

//////////////////////////////////////////////////////////////////////////////
//
//  functions requiring SSE4.1 are below this point
//

#include <smmintrin.h> // SSE4.1

// in the below functions, the term "4th" means arr[4], i.e. technically the 5th

int radaudio_sse4_find_median_4th_of_8(S16 *data)
{
   int16x8 select_index  = _mm_setr_epi8( 2,3,2,3, 2,3,2,3, 2,3,2,3, 2,3,2,3 );
   int8x16 index_of_lane = _mm_setr_epi16(0,1,2,3,4,5,6,7);

   int8x16 v0    = _mm_loadu_si128((int8x16 *) data);

   // find the smallest item
   int8x16 min0  = _mm_minpos_epu16(v0);

   // replicate bits 31:16 into all 16-bit entries (could use _mm_shufflelo_epi16, _mm_movelh_ps)
   int8x16 idx0  = _mm_shuffle_epi8(min0, select_index);

   // detect which lane matches
   int8x16 mask0 = _mm_cmpeq_epi16(idx0, index_of_lane);

   // set that lane to the maximum value
   int8x16 v1    = _mm_or_si128(v0,mask0);

   // repeat the above 3 more times
   int8x16 min1  = _mm_minpos_epu16(v1);
   int8x16 idx1  = _mm_shuffle_epi8(min1, select_index);
   int8x16 v2    = _mm_or_si128(v1,_mm_cmpeq_epi16(idx1, index_of_lane));

   int8x16 min2  = _mm_minpos_epu16(v2);
   int8x16 idx2  = _mm_shuffle_epi8(min2, select_index);
   int8x16 v3    = _mm_or_si128(v2,_mm_cmpeq_epi16(idx2, index_of_lane));

   int8x16 min3  = _mm_minpos_epu16(v3);
   int8x16 idx3  = _mm_shuffle_epi8(min3, select_index);
   int8x16 v4    = _mm_or_si128(v3,_mm_cmpeq_epi16(idx3, index_of_lane));

   // the 4 smallest values have been replaced, so the new smallest is sorted_data[4], our required median
   int8x16 min4  = _mm_minpos_epu16(v4);
   return _mm_extract_epi16(min4, 0);
}

static int findnth_of_8(int8x16 value, int n)
{
   int16x8 select_index  = _mm_setr_epi8( 2,3,2,3, 2,3,2,3, 2,3,2,3, 2,3,2,3 );
   int8x16 index_of_lane = _mm_setr_epi16(0,1,2,3,4,5,6,7);

   // force (n-1)th smallest values to max, then do mipos
   int8x16 v0 = value;

   // could use duff's device, since max of 7 steps and each is only a few instructions
   while (n >= 2) {
      int8x16 min0  = _mm_minpos_epu16(v0);
      int8x16 idx0  = _mm_shuffle_epi8(min0, select_index);
      int8x16 v1    = _mm_or_si128(v0,_mm_cmpeq_epi16(idx0, index_of_lane));
      int8x16 min1  = _mm_minpos_epu16(v1);
      int8x16 idx1  = _mm_shuffle_epi8(min1, select_index);
      int8x16 v2    = _mm_or_si128(v1,_mm_cmpeq_epi16(idx1, index_of_lane));
      v0 = v2;
      n -= 2;
   }
   if (n >= 1) {
      int8x16 min3  = _mm_minpos_epu16(v0);
      int8x16 idx3  = _mm_shuffle_epi8(min3, select_index);
      int8x16 v4    = _mm_or_si128(v0,_mm_cmpeq_epi16(idx3, index_of_lane));
      v0 = v4;
   }

   int8x16 min4  = _mm_minpos_epu16(v0);
   return _mm_extract_epi16(min4, 0);
}

int radaudio_sse4_findnth_of_8_or_less(S16 *data, int count, int n)
{
   rrAssert(n < count);

   // load data
   int8x16 vbase = _mm_loadu_si128((int8x16 *) data);

   // generate all 1s in any lane whose lane index is >= count
   int8x16 mask  = _mm_cmpgt_epi16(_mm_setr_epi16(0,1,2,3,4,5,6,7), _mm_set1_epi16((S16) count-1));

   // force all those values to maximum possible, so min will have no effect
   int8x16 v0    = _mm_or_si128(vbase, mask);

   // now we can treat count as 8
   return findnth_of_8(v0, n);
}

//////////////////////////////////////////////////////////////////////////////
//
//  functions requiring POPCNT are below this point
//

#ifdef _MSC_VER
#include <intrin.h> // POPCNT
#else
#define __popcnt __builtin_popcount
#endif

// in the below functions, the term 8th means arr[8], i.e. technically the 9th

int radaudio_sse4popcnt_find_median_8th_of_16(S16 *data)
{
   int n=0;

   int8x16 v0    = _mm_loadu_si128((int8x16 *) (data+0));
   int8x16 v1    = _mm_loadu_si128((int8x16 *) (data+8));

   // we use a different strategy for ties:
   // - as shorthand, replacing the minimum in the current array with 0xffff is called "replacing"
   // - 4th_of_8 always replaces exactly one item every iteration, so it knows exactly how far it has to run
   // - in this code, replacing exactly one item would be painful since we don't know which one had it
   //   - so we replace all duplicate minimums.
   //   - requires us to count how many we replace
   //     - after replacing 8, we return the min
   //     - if we go from e.g. 7 replaced to 9 replaced, then the current value we're replacing is the 8th

   while (n <= 8) {
      // find the smallest item across both
      int8x16 min0   = _mm_minpos_epu16(_mm_min_epu16(v0,v1));

      // broadcast to all lanes
      int8x16 minlo  = _mm_shufflelo_epi16(min0, MY_SHUFFLE(0,0,0,0));
      float4  minlof = _mm_castsi128_ps(minlo);
      int8x16 minall = _mm_castps_si128(_mm_movelh_ps(minlof,minlof));

      // locate that item; it might be in more than one place
      int8x16 mask0 = _mm_cmpeq_epi16(v0, minall);
      int8x16 mask1 = _mm_cmpeq_epi16(v1, minall);

      // count how many of them there are by creating a corresponding byte mask
      int16x8 merged = _mm_or_si128(_mm_and_si128(mask0, _mm_set1_epi16(0x00ff)), _mm_and_si128(mask1, _mm_set1_epi16(0xff00)));

      n += __popcnt(_mm_movemask_epi8(merged));

      // if the total number of min items we've eliminated is 8 or more, then we've found the 8th item (the median)
      if (n > 8) {
         return _mm_extract_epi16(minall, 0);
      }

      // replace all minimum items with max
      v0 = _mm_or_si128(v0, mask0);
      v1 = _mm_or_si128(v1, mask1);
   }
   // we've replaced exactly 8 items
  
   // find the minimum of the remaining items and return that
   return _mm_extract_epi16(_mm_minpos_epu16(_mm_min_epu16(v0,v1)), 0);
}

#endif // #if DO_BUILD_SSE4
