// Copyright Epic Games Tools, LLC. All Rights Reserved.
#ifndef RADAUDIO_ENCODER_SSE_H
#define RADAUDIO_ENCODER_SSE_H

#include "radaudio_common.h"
#include "radaudio_sse.h"

#ifdef RADAUDIO_WRAP
#define WRAPPED_NAME(name) RR_STRING_JOIN(RADAUDIO_WRAP, name##_)

#define radaudio_sse2_compute_best_quantized_coeff16_loop	WRAPPED_NAME(radaudio_sse2_compute_best_quantized_coeff16_loop)
#define radaudio_sse2_compute_best_quantized_coeff8_loop    WRAPPED_NAME(radaudio_sse2_compute_best_quantized_coeff8_loop)
#define radaudio_sse2_fabs_coefficients                 	WRAPPED_NAME(radaudio_sse2_fabs_coefficients)
#define radaudio_sse4_find_median_4th_of_8					WRAPPED_NAME(radaudio_sse4_find_median_4th_of_8)
#define radaudio_sse4_findnth_of_8_or_less                  WRAPPED_NAME(radaudio_sse4_findnth_of_8_or_less)
#define radaudio_sse4popcnt_find_median_8th_of_16			WRAPPED_NAME(radaudio_sse4popcnt_find_median_8th_of_16)
#endif


extern void radaudio_sse2_compute_best_quantized_coeff16_loop(S16 best_quantized[], F32 best_so_far, F32 ncoeff[], F32 quantizer, F32 step_quantizer, int num_quantizers);
extern void radaudio_sse2_compute_best_quantized_coeff8_loop(S16 best_quantized[], F32 best_so_far, F32 ncoeff[], F32 quantizer, F32 step_quantizer, int num_quantizers);

extern void radaudio_sse2_fabs_coefficients(float *absval, float *data, int num_coeff);
extern int radaudio_sse4_find_median_4th_of_8(S16 *data);
extern int radaudio_sse4_findnth_of_8_or_less(S16 *data, int count, int n);
extern int radaudio_sse4popcnt_find_median_8th_of_16(S16 *data);

#endif













