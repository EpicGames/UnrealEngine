// Copyright Epic Games Tools, LLC. All Rights Reserved.

static radaudio_huffman rada_band_exponent_correct_huff =
{
  #ifdef HUFFMAN_ENCODE
  {
    {   0, 2}, {   2, 2}, {   7, 5}, {  15, 6}, {  31, 8}, {  95, 9}, { 223,10}, { 735,10}, 
    { 479,10}, { 991,10}, { 703,11}, {1727,11}, { 447,11}, {1471,11}, { 959,11}, {1983,11}, 
    { 127,11}, {1151,11}, { 639,11}, {1663,11}, { 383,11}, {1407,11}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, { 895,11}, {1919,11}, { 255,11}, {1279,11}, { 767,11}, 
    {1791,11}, { 511,11}, {1535,11}, {1023,11}, {2047,11}, {  63,10}, { 575,10}, { 319,10}, 
    { 831,10}, { 191,10}, { 351, 9}, { 159, 8}, {  47, 6}, {  23, 5}, {   3, 3}, {   1, 2}, 
  },
  #endif
  #ifdef HUFFMAN_DECODE
  {
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,  3}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 8,  4}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,252}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, {10,245}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,  3}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 9,  5}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,252}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, {11, 16}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,  3}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 8,251}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,252}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, {10,249}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,  3}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, {10,  6}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,252}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, {11,237}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,  3}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 8,  4}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,252}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, {10,247}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,  3}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 9,250}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,252}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, {11, 20}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,  3}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 8,251}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,252}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, {11, 12}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,  3}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, {10,  8}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,252}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, {11,241}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,  3}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 8,  4}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,252}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, {10,246}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,  3}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 9,  5}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,252}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, {11, 18}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,  3}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 8,251}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,252}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, {11, 10}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,  3}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, {10,  7}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,252}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, {11,239}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,  3}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 8,  4}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,252}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, {10,248}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,  3}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 9,250}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,252}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, {11,235}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,  3}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 8,251}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,252}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, {11, 14}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,  3}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, {10,  9}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,252}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, {11,243}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,  3}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 8,  4}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,252}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, {10,245}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,  3}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 9,  5}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,252}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, {11, 17}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,  3}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 8,251}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,252}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, {10,249}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,  3}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, {10,  6}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,252}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, {11,238}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,  3}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 8,  4}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,252}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, {10,247}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,  3}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 9,250}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,252}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, {11, 21}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,  3}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 8,251}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,252}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, {11, 13}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,  3}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, {10,  8}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,252}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, {11,242}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,  3}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 8,  4}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,252}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, {10,246}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,  3}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 9,  5}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,252}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, {11, 19}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,  3}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 8,251}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,252}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, {11, 11}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,  3}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, {10,  7}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,252}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, {11,240}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,  3}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 8,  4}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,252}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, {10,248}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,  3}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 9,250}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,252}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, {11,236}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,  3}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 8,251}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,252}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, {11, 15}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,  3}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, {10,  9}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,  2}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 6,252}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, { 5,253}, 
    { 2,  0}, { 2,255}, { 2,  1}, { 3,254}, { 2,  0}, { 2,255}, { 2,  1}, {11,244}, 
  }
  #endif
};

static radaudio_huffman rada_band_exponent_stereo_correct_huff =
{
  #ifdef HUFFMAN_ENCODE
  {
    {   0, 1}, {   1, 2}, {   7, 4}, {  31, 7}, {  63, 8}, { 127, 9}, { 383, 9}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, { 255, 9}, { 511, 9}, { 191, 8}, {  95, 7}, {  15, 5}, {   3, 3}, 
  },
  #endif
  #ifdef HUFFMAN_DECODE
  {
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 7,  3}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 8,  4}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 7,253}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 9,  5}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 7,  3}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 8,252}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 7,253}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 9,250}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 7,  3}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 8,  4}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 7,253}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 9,  6}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 7,  3}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 8,252}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 7,253}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 9,251}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 7,  3}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 8,  4}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 7,253}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 9,  5}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 7,  3}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 8,252}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 7,253}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 9,250}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 7,  3}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 8,  4}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 7,253}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 9,  6}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 7,  3}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 8,252}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 7,253}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 9,251}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 7,  3}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 8,  4}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 7,253}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 9,  5}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 7,  3}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 8,252}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 7,253}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 9,250}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 7,  3}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 8,  4}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 7,253}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 9,  6}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 7,  3}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 8,252}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 7,253}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 9,251}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 7,  3}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 8,  4}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 7,253}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 9,  5}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 7,  3}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 8,252}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 7,253}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 9,250}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 7,  3}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 8,  4}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 7,253}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 9,  6}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 7,  3}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 8,252}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 7,253}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 4,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 9,251}, 
  }
  #endif
};

static radaudio_huffman rada_subband_value_huff =
{
  #ifdef HUFFMAN_ENCODE
  {
    { 991,11}, {2015,11}, {  63,11}, {1087,11}, { 575,11}, {1599,11}, { 319,11}, {1343,11}, 
    { 495,10}, {  23, 6}, {   3, 4}, {   1, 3}, {   0, 2}, {   2, 2}, {   5, 3}, {  11, 4}, 
    {   7, 5}, {  55, 6}, {  15, 6}, {  47, 7}, { 111, 8}, { 239, 9}, {1007,10}, {  31,10}, 
    { 543,10}, { 287,10}, { 799,10}, { 159,10}, { 671,10}, { 415,10}, { 927,10}, {  95,10}, 
    { 607,10}, { 351,10}, { 863,10}, { 223,10}, { 735,10}, { 479,10}, { 831,11}, {1855,11}, 
    { 191,11}, {1215,11}, { 703,11}, {1727,11}, { 447,11}, {1471,11}, { 959,11}, {1983,11}, 
    { 127,11}, {1151,11}, { 639,11}, {1663,11}, { 383,11}, {1407,11}, { 895,11}, {1919,11}, 
    { 255,11}, {1279,11}, { 767,11}, {1791,11}, { 511,11}, {1535,11}, {1023,11}, {2047,11}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
  },
  #endif
  #ifdef HUFFMAN_DECODE
  {
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 18}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6,  9}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {10, 23}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 7, 19}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 17}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {11,  2}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 18}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6,  9}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {10, 31}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 8, 20}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 17}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {11, 48}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 18}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6,  9}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {10, 27}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 7, 19}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 17}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {11, 40}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 18}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6,  9}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {10, 35}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 9, 21}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 17}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {11, 56}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 18}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6,  9}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {10, 25}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 7, 19}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 17}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {11,  6}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 18}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6,  9}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {10, 33}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 8, 20}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 17}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {11, 52}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 18}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6,  9}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {10, 29}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 7, 19}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 17}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {11, 44}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 18}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6,  9}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {10, 37}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {10,  8}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 17}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {11, 60}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 18}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6,  9}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {10, 24}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 7, 19}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 17}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {11,  4}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 18}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6,  9}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {10, 32}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 8, 20}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 17}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {11, 50}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 18}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6,  9}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {10, 28}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 7, 19}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 17}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {11, 42}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 18}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6,  9}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {10, 36}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 9, 21}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 17}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {11, 58}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 18}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6,  9}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {10, 26}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 7, 19}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 17}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {11, 38}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 18}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6,  9}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {10, 34}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 8, 20}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 17}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {11, 54}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 18}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6,  9}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {10, 30}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 7, 19}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 17}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {11, 46}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 18}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6,  9}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {11,  0}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {10, 22}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 17}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {11, 62}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 18}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6,  9}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {10, 23}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 7, 19}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 17}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {11,  3}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 18}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6,  9}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {10, 31}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 8, 20}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 17}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {11, 49}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 18}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6,  9}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {10, 27}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 7, 19}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 17}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {11, 41}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 18}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6,  9}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {10, 35}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 9, 21}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 17}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {11, 57}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 18}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6,  9}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {10, 25}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 7, 19}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 17}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {11,  7}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 18}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6,  9}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {10, 33}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 8, 20}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 17}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {11, 53}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 18}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6,  9}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {10, 29}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 7, 19}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 17}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {11, 45}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 18}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6,  9}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {10, 37}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {10,  8}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 17}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {11, 61}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 18}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6,  9}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {10, 24}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 7, 19}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 17}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {11,  5}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 18}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6,  9}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {10, 32}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 8, 20}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 17}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {11, 51}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 18}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6,  9}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {10, 28}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 7, 19}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 17}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {11, 43}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 18}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6,  9}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {10, 36}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 9, 21}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 17}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {11, 59}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 18}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6,  9}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {10, 26}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 7, 19}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 17}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {11, 39}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 18}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6,  9}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {10, 34}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 8, 20}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 17}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {11, 55}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 18}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6,  9}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {10, 30}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 7, 19}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 17}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {11, 47}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 18}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6,  9}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {11,  1}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 5, 16}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {10, 22}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 10}, { 2, 12}, { 3, 14}, { 2, 13}, { 6, 17}, 
    { 2, 12}, { 3, 11}, { 2, 13}, { 4, 15}, { 2, 12}, { 3, 14}, { 2, 13}, {11, 63}, 
  }
  #endif
};

static radaudio_huffman rada_subband_value_stereo_correct_huff =
{
  #ifdef HUFFMAN_ENCODE
  {
    {   0, 1}, {   1, 2}, {   7, 5}, {  15, 6}, {  31, 8}, { 415,10}, { 927,11}, {1951,11}, 
    {  95,11}, {1119,11}, { 607,11}, {1631,11}, { 351,11}, {1375,11}, { 863,11}, {1887,11}, 
    { 223,11}, {1247,11}, { 735,11}, {1759,11}, { 479,11}, {1503,11}, { 991,11}, {2015,11}, 
    {  63,11}, {1087,11}, { 575,11}, {1599,11}, { 319,11}, {1343,11}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, { 831,11}, {1855,11}, { 191,11}, {1215,11}, { 703,11}, {1727,11}, 
    { 447,11}, {1471,11}, { 959,11}, {1983,11}, { 127,11}, {1151,11}, { 639,11}, {1663,11}, 
    { 383,11}, {1407,11}, { 895,11}, {1919,11}, { 255,11}, {1279,11}, { 767,11}, {1791,11}, 
    { 511,11}, {1535,11}, {1023,11}, {2047,11}, { 159, 9}, {  47, 6}, {  23, 5}, {   3, 3}, 
  },
  #endif
  #ifdef HUFFMAN_DECODE
  {
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,  3}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 8,  4}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, {11, 24}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,  3}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, {11,  8}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, {11,236}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,  3}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 9,252}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, {11,228}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,  3}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, {11, 16}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, {11,244}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,  3}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 8,  4}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, {11, 28}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,  3}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, {11, 12}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, {11,240}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,  3}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, {10,  5}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, {11,232}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,  3}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, {11, 20}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, {11,248}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,  3}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 8,  4}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, {11, 26}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,  3}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, {11, 10}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, {11,238}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,  3}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 9,252}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, {11,230}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,  3}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, {11, 18}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, {11,246}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,  3}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 8,  4}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, {11,226}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,  3}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, {11, 14}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, {11,242}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,  3}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, {11,  6}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, {11,234}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,  3}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, {11, 22}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, {11,250}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,  3}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 8,  4}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, {11, 25}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,  3}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, {11,  9}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, {11,237}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,  3}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 9,252}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, {11,229}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,  3}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, {11, 17}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, {11,245}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,  3}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 8,  4}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, {11, 29}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,  3}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, {11, 13}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, {11,241}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,  3}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, {10,  5}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, {11,233}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,  3}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, {11, 21}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, {11,249}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,  3}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 8,  4}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, {11, 27}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,  3}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, {11, 11}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, {11,239}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,  3}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 9,252}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, {11,231}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,  3}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, {11, 19}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, {11,247}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,  3}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 8,  4}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, {11,227}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,  3}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, {11, 15}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, {11,243}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,  3}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, {11,  7}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, {11,235}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,  3}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, {11, 23}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, { 5,254}, 
    { 1,  0}, { 2,  1}, { 1,  0}, { 3,255}, { 1,  0}, { 2,  1}, { 1,  0}, {11,251}, 
  }
  #endif
};

static radaudio_huffman rada_subband_value_last_in_band_correct_huff =
{
  #ifdef HUFFMAN_ENCODE
  {
    {   0, 1}, {   3, 3}, {  15, 5}, {  63, 8}, { 447,11}, {1471,11}, { 959,11}, {1983,11}, 
    { 127,11}, {1151,11}, { 639,11}, {1663,11}, { 383,11}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {1407,11}, { 895,11}, {1919,11}, { 255,11}, {1279,11}, { 767,11}, {1791,11}, 
    { 511,11}, {1535,11}, {1023,11}, {2047,11}, { 191, 9}, {  31, 6}, {   7, 4}, {   1, 2}, 
  },
  #endif
  #ifdef HUFFMAN_DECODE
  {
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 8,  3}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, {11,  8}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 9,252}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, {11,244}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 8,  3}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, {11, 12}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, {11,  4}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, {11,248}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 8,  3}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, {11, 10}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 9,252}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, {11,246}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 8,  3}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, {11,242}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, {11,  6}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, {11,250}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 8,  3}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, {11,  9}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 9,252}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, {11,245}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 8,  3}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, {11,241}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, {11,  5}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, {11,249}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 8,  3}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, {11, 11}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 9,252}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, {11,247}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 8,  3}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, {11,243}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, {11,  7}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 6,253}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 5,  2}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, { 4,254}, 
    { 1,  0}, { 2,255}, { 1,  0}, { 3,  1}, { 1,  0}, { 2,255}, { 1,  0}, {11,251}, 
  }
  #endif
};

static radaudio_huffman rada_zero_runlength_huff =
{
  #ifdef HUFFMAN_ENCODE
  {
    {   0, 1}, {   1, 3}, {   5, 4}, {  13, 4}, {   3, 5}, {  19, 5}, {  11, 6}, {  43, 6}, 
    {  27, 6}, {  59, 7}, { 123, 7}, {   7, 7}, {  39, 8}, { 167, 8}, { 103, 8}, { 231, 8}, 
    {  23, 8}, { 151, 8}, {  87, 8}, { 215, 8}, {  55, 8}, { 183, 8}, { 119, 9}, { 375, 9}, 
    { 247, 9}, { 503, 9}, {  15, 9}, { 271, 9}, {  47,10}, { 559,10}, { 303,10}, { 815,10}, 
    { 143, 9}, { 399, 9}, {  79, 9}, { 335, 9}, { 175,10}, { 687,10}, { 431,10}, { 943,10}, 
    { 111,10}, { 623,10}, { 367,10}, {  95,11}, {1119,11}, { 607,11}, {1631,11}, { 351,11}, 
    { 879,10}, { 239,10}, { 751,10}, { 495,10}, {1375,11}, { 863,11}, {1887,11}, { 223,11}, 
    {1247,11}, { 735,11}, {1759,11}, { 479,11}, {1007,10}, { 735,11}, {1759,11}, { 479,11}, 
    { 207, 9}, {   0, 0}, {   0, 0}, {   0, 0}, { 463, 9}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {  31,10}, {   0, 0}, {   0, 0}, {   0, 0}, { 543,10}, {   0, 0}, {   0, 0}, {   0, 0}, 
    { 287,10}, {   0, 0}, {   0, 0}, {   0, 0}, { 799,10}, {   0, 0}, {   0, 0}, {   0, 0}, 
    { 159,10}, {   0, 0}, {   0, 0}, {   0, 0}, {1503,11}, {   0, 0}, {   0, 0}, {   0, 0}, 
    { 671,10}, {   0, 0}, {   0, 0}, {   0, 0}, { 415,10}, {   0, 0}, {   0, 0}, {   0, 0}, 
    { 991,11}, {   0, 0}, {   0, 0}, {   0, 0}, {2015,11}, {   0, 0}, {   0, 0}, {   0, 0}, 
    { 927,10}, {   0, 0}, {   0, 0}, {   0, 0}, {  63,11}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {1087,11}, {   0, 0}, {   0, 0}, {   0, 0}, { 575,11}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {1599,11}, {   0, 0}, {   0, 0}, {   0, 0}, { 319,11}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {1343,11}, {   0, 0}, {   0, 0}, {   0, 0}, { 831,11}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {1855,11}, {   0, 0}, {   0, 0}, {   0, 0}, { 191,11}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {1215,11}, {   0, 0}, {   0, 0}, {   0, 0}, { 703,11}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {1727,11}, {   0, 0}, {   0, 0}, {   0, 0}, { 447,11}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {1471,11}, {   0, 0}, {   0, 0}, {   0, 0}, { 959,11}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {1983,11}, {   0, 0}, {   0, 0}, {   0, 0}, { 127,11}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {1151,11}, {   0, 0}, {   0, 0}, {   0, 0}, { 639,11}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {1663,11}, {   0, 0}, {   0, 0}, {   0, 0}, { 383,11}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {1407,11}, {   0, 0}, {   0, 0}, {   0, 0}, { 895,11}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {1919,11}, {   0, 0}, {   0, 0}, {   0, 0}, { 255,11}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {1279,11}, {   0, 0}, {   0, 0}, {   0, 0}, { 767,11}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {1791,11}, {   0, 0}, {   0, 0}, {   0, 0}, { 511,11}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {1535,11}, {   0, 0}, {   0, 0}, {   0, 0}, {1023,11}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {2047,11}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {  71, 7}, 
  },
  #endif
  #ifdef HUFFMAN_DECODE
  {
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 7, 11}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  6}, { 1,  0}, { 4,  3}, { 1,  0}, { 9, 26}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 16}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  8}, { 1,  0}, { 4,  3}, { 1,  0}, {10, 72}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 12}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  7}, { 1,  0}, { 4,  3}, { 1,  0}, {10, 28}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 20}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 7,  9}, { 1,  0}, { 4,  3}, { 1,  0}, {11,116}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 7,255}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  6}, { 1,  0}, { 4,  3}, { 1,  0}, { 9, 34}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 18}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  8}, { 1,  0}, { 4,  3}, { 1,  0}, {11, 43}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 14}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  7}, { 1,  0}, { 4,  3}, { 1,  0}, {10, 40}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 9, 22}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 7, 10}, { 1,  0}, { 4,  3}, { 1,  0}, {11,180}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 7, 11}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  6}, { 1,  0}, { 4,  3}, { 1,  0}, { 9, 32}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 17}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  8}, { 1,  0}, { 4,  3}, { 1,  0}, {10, 88}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 13}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  7}, { 1,  0}, { 4,  3}, { 1,  0}, {10, 36}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 21}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 7,  9}, { 1,  0}, { 4,  3}, { 1,  0}, {11,148}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 7,255}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  6}, { 1,  0}, { 4,  3}, { 1,  0}, { 9, 64}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 19}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  8}, { 1,  0}, { 4,  3}, { 1,  0}, {11, 55}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 15}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  7}, { 1,  0}, { 4,  3}, { 1,  0}, {10, 49}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 9, 24}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 7, 10}, { 1,  0}, { 4,  3}, { 1,  0}, {11,212}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 7, 11}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  6}, { 1,  0}, { 4,  3}, { 1,  0}, { 9, 27}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 16}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  8}, { 1,  0}, { 4,  3}, { 1,  0}, {10, 80}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 12}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  7}, { 1,  0}, { 4,  3}, { 1,  0}, {10, 30}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 20}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 7,  9}, { 1,  0}, { 4,  3}, { 1,  0}, {11,132}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 7,255}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  6}, { 1,  0}, { 4,  3}, { 1,  0}, { 9, 35}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 18}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  8}, { 1,  0}, { 4,  3}, { 1,  0}, {11, 47}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 14}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  7}, { 1,  0}, { 4,  3}, { 1,  0}, {10, 42}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 9, 23}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 7, 10}, { 1,  0}, { 4,  3}, { 1,  0}, {11,196}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 7, 11}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  6}, { 1,  0}, { 4,  3}, { 1,  0}, { 9, 33}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 17}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  8}, { 1,  0}, { 4,  3}, { 1,  0}, {10,100}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 13}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  7}, { 1,  0}, { 4,  3}, { 1,  0}, {10, 38}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 21}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 7,  9}, { 1,  0}, { 4,  3}, { 1,  0}, {11,164}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 7,255}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  6}, { 1,  0}, { 4,  3}, { 1,  0}, { 9, 68}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 19}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  8}, { 1,  0}, { 4,  3}, { 1,  0}, {11, 59}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 15}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  7}, { 1,  0}, { 4,  3}, { 1,  0}, {10, 51}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 9, 25}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 7, 10}, { 1,  0}, { 4,  3}, { 1,  0}, {11,228}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 7, 11}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  6}, { 1,  0}, { 4,  3}, { 1,  0}, { 9, 26}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 16}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  8}, { 1,  0}, { 4,  3}, { 1,  0}, {10, 76}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 12}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  7}, { 1,  0}, { 4,  3}, { 1,  0}, {10, 29}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 20}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 7,  9}, { 1,  0}, { 4,  3}, { 1,  0}, {11,124}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 7,255}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  6}, { 1,  0}, { 4,  3}, { 1,  0}, { 9, 34}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 18}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  8}, { 1,  0}, { 4,  3}, { 1,  0}, {11, 45}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 14}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  7}, { 1,  0}, { 4,  3}, { 1,  0}, {10, 41}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 9, 22}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 7, 10}, { 1,  0}, { 4,  3}, { 1,  0}, {11,188}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 7, 11}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  6}, { 1,  0}, { 4,  3}, { 1,  0}, { 9, 32}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 17}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  8}, { 1,  0}, { 4,  3}, { 1,  0}, {10, 96}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 13}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  7}, { 1,  0}, { 4,  3}, { 1,  0}, {10, 37}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 21}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 7,  9}, { 1,  0}, { 4,  3}, { 1,  0}, {11,156}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 7,255}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  6}, { 1,  0}, { 4,  3}, { 1,  0}, { 9, 64}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 19}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  8}, { 1,  0}, { 4,  3}, { 1,  0}, {11, 57}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 15}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  7}, { 1,  0}, { 4,  3}, { 1,  0}, {10, 50}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 9, 24}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 7, 10}, { 1,  0}, { 4,  3}, { 1,  0}, {11,220}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 7, 11}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  6}, { 1,  0}, { 4,  3}, { 1,  0}, { 9, 27}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 16}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  8}, { 1,  0}, { 4,  3}, { 1,  0}, {10, 84}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 12}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  7}, { 1,  0}, { 4,  3}, { 1,  0}, {10, 31}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 20}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 7,  9}, { 1,  0}, { 4,  3}, { 1,  0}, {11,140}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 7,255}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  6}, { 1,  0}, { 4,  3}, { 1,  0}, { 9, 35}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 18}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  8}, { 1,  0}, { 4,  3}, { 1,  0}, {11, 53}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 14}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  7}, { 1,  0}, { 4,  3}, { 1,  0}, {10, 48}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 9, 23}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 7, 10}, { 1,  0}, { 4,  3}, { 1,  0}, {11,204}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 7, 11}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  6}, { 1,  0}, { 4,  3}, { 1,  0}, { 9, 33}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 17}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  8}, { 1,  0}, { 4,  3}, { 1,  0}, {10,112}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 13}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  7}, { 1,  0}, { 4,  3}, { 1,  0}, {10, 39}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 21}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 7,  9}, { 1,  0}, { 4,  3}, { 1,  0}, {11,172}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 7,255}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  6}, { 1,  0}, { 4,  3}, { 1,  0}, { 9, 68}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 19}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  8}, { 1,  0}, { 4,  3}, { 1,  0}, {11,104}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 15}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  7}, { 1,  0}, { 4,  3}, { 1,  0}, {10, 60}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 9, 25}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 7, 10}, { 1,  0}, { 4,  3}, { 1,  0}, {11,236}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 7, 11}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  6}, { 1,  0}, { 4,  3}, { 1,  0}, { 9, 26}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 16}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  8}, { 1,  0}, { 4,  3}, { 1,  0}, {10, 72}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 12}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  7}, { 1,  0}, { 4,  3}, { 1,  0}, {10, 28}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 20}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 7,  9}, { 1,  0}, { 4,  3}, { 1,  0}, {11,120}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 7,255}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  6}, { 1,  0}, { 4,  3}, { 1,  0}, { 9, 34}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 18}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  8}, { 1,  0}, { 4,  3}, { 1,  0}, {11, 44}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 14}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  7}, { 1,  0}, { 4,  3}, { 1,  0}, {10, 40}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 9, 22}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 7, 10}, { 1,  0}, { 4,  3}, { 1,  0}, {11,184}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 7, 11}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  6}, { 1,  0}, { 4,  3}, { 1,  0}, { 9, 32}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 17}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  8}, { 1,  0}, { 4,  3}, { 1,  0}, {10, 88}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 13}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  7}, { 1,  0}, { 4,  3}, { 1,  0}, {10, 36}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 21}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 7,  9}, { 1,  0}, { 4,  3}, { 1,  0}, {11,152}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 7,255}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  6}, { 1,  0}, { 4,  3}, { 1,  0}, { 9, 64}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 19}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  8}, { 1,  0}, { 4,  3}, { 1,  0}, {11, 56}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 15}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  7}, { 1,  0}, { 4,  3}, { 1,  0}, {10, 49}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 9, 24}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 7, 10}, { 1,  0}, { 4,  3}, { 1,  0}, {11,216}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 7, 11}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  6}, { 1,  0}, { 4,  3}, { 1,  0}, { 9, 27}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 16}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  8}, { 1,  0}, { 4,  3}, { 1,  0}, {10, 80}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 12}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  7}, { 1,  0}, { 4,  3}, { 1,  0}, {10, 30}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 20}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 7,  9}, { 1,  0}, { 4,  3}, { 1,  0}, {11,136}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 7,255}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  6}, { 1,  0}, { 4,  3}, { 1,  0}, { 9, 35}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 18}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  8}, { 1,  0}, { 4,  3}, { 1,  0}, {11, 52}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 14}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  7}, { 1,  0}, { 4,  3}, { 1,  0}, {10, 42}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 9, 23}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 7, 10}, { 1,  0}, { 4,  3}, { 1,  0}, {11,200}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 7, 11}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  6}, { 1,  0}, { 4,  3}, { 1,  0}, { 9, 33}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 17}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  8}, { 1,  0}, { 4,  3}, { 1,  0}, {10,100}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 13}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  7}, { 1,  0}, { 4,  3}, { 1,  0}, {10, 38}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 21}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 7,  9}, { 1,  0}, { 4,  3}, { 1,  0}, {11,168}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 7,255}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  6}, { 1,  0}, { 4,  3}, { 1,  0}, { 9, 68}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 19}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  8}, { 1,  0}, { 4,  3}, { 1,  0}, {11, 92}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 15}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  7}, { 1,  0}, { 4,  3}, { 1,  0}, {10, 51}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 9, 25}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 7, 10}, { 1,  0}, { 4,  3}, { 1,  0}, {11,232}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 7, 11}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  6}, { 1,  0}, { 4,  3}, { 1,  0}, { 9, 26}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 16}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  8}, { 1,  0}, { 4,  3}, { 1,  0}, {10, 76}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 12}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  7}, { 1,  0}, { 4,  3}, { 1,  0}, {10, 29}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 20}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 7,  9}, { 1,  0}, { 4,  3}, { 1,  0}, {11,128}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 7,255}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  6}, { 1,  0}, { 4,  3}, { 1,  0}, { 9, 34}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 18}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  8}, { 1,  0}, { 4,  3}, { 1,  0}, {11, 46}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 14}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  7}, { 1,  0}, { 4,  3}, { 1,  0}, {10, 41}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 9, 22}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 7, 10}, { 1,  0}, { 4,  3}, { 1,  0}, {11,192}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 7, 11}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  6}, { 1,  0}, { 4,  3}, { 1,  0}, { 9, 32}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 17}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  8}, { 1,  0}, { 4,  3}, { 1,  0}, {10, 96}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 13}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  7}, { 1,  0}, { 4,  3}, { 1,  0}, {10, 37}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 21}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 7,  9}, { 1,  0}, { 4,  3}, { 1,  0}, {11,160}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 7,255}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  6}, { 1,  0}, { 4,  3}, { 1,  0}, { 9, 64}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 19}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  8}, { 1,  0}, { 4,  3}, { 1,  0}, {11, 58}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 15}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  7}, { 1,  0}, { 4,  3}, { 1,  0}, {10, 50}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 9, 24}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 7, 10}, { 1,  0}, { 4,  3}, { 1,  0}, {11,224}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 7, 11}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  6}, { 1,  0}, { 4,  3}, { 1,  0}, { 9, 27}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 16}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  8}, { 1,  0}, { 4,  3}, { 1,  0}, {10, 84}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 12}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  7}, { 1,  0}, { 4,  3}, { 1,  0}, {10, 31}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 20}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 7,  9}, { 1,  0}, { 4,  3}, { 1,  0}, {11,144}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 7,255}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  6}, { 1,  0}, { 4,  3}, { 1,  0}, { 9, 35}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 18}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  8}, { 1,  0}, { 4,  3}, { 1,  0}, {11, 54}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 14}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  7}, { 1,  0}, { 4,  3}, { 1,  0}, {10, 48}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 9, 23}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 7, 10}, { 1,  0}, { 4,  3}, { 1,  0}, {11,208}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 7, 11}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  6}, { 1,  0}, { 4,  3}, { 1,  0}, { 9, 33}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 17}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  8}, { 1,  0}, { 4,  3}, { 1,  0}, {10,112}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 13}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  7}, { 1,  0}, { 4,  3}, { 1,  0}, {10, 39}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 21}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 7,  9}, { 1,  0}, { 4,  3}, { 1,  0}, {11,176}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 7,255}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  6}, { 1,  0}, { 4,  3}, { 1,  0}, { 9, 68}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 19}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  8}, { 1,  0}, { 4,  3}, { 1,  0}, {11,108}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  4}, { 1,  0}, { 4,  2}, { 1,  0}, { 8, 15}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 6,  7}, { 1,  0}, { 4,  3}, { 1,  0}, {10, 60}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 5,  5}, { 1,  0}, { 4,  2}, { 1,  0}, { 9, 25}, 
    { 1,  0}, { 3,  1}, { 1,  0}, { 7, 10}, { 1,  0}, { 4,  3}, { 1,  0}, {11,240}, 
  }
  #endif
};

static radaudio_huffman rada_nonzero_bitflags_first64_huff =
{
  #ifdef HUFFMAN_ENCODE
  {
    {   1, 8}, { 101, 9}, { 357, 9}, { 229, 9}, { 485, 9}, {  21, 9}, { 277, 9}, { 149, 9}, 
    { 405, 9}, {  85, 9}, { 341, 9}, { 213, 9}, { 469, 9}, {  53, 9}, { 309, 9}, { 129, 8}, 
    { 181, 9}, { 437, 9}, { 117, 9}, { 373, 9}, { 245, 9}, { 501, 9}, {  13, 9}, { 269, 9}, 
    { 141, 9}, { 397, 9}, {  77, 9}, { 333, 9}, { 205, 9}, { 461, 9}, {  45, 9}, {  18, 7}, 
    { 301, 9}, { 173, 9}, { 429, 9}, { 109, 9}, { 365, 9}, { 237, 9}, { 493, 9}, {  29, 9}, 
    { 285, 9}, { 157, 9}, { 413, 9}, {  93, 9}, { 349, 9}, { 221, 9}, { 477, 9}, {  65, 8}, 
    {  61, 9}, { 317, 9}, { 189, 9}, { 445, 9}, { 125, 9}, { 381, 9}, { 253, 9}, { 193, 8}, 
    { 509, 9}, {   3, 9}, { 259, 9}, {  33, 8}, { 131, 9}, { 161, 8}, {  97, 8}, {  20, 6}, 
    { 387, 9}, {  67, 9}, { 323, 9}, { 195, 9}, { 451, 9}, {  35, 9}, { 291, 9}, { 163, 9}, 
    { 419, 9}, {  99, 9}, { 355, 9}, { 227, 9}, { 483, 9}, {  19, 9}, { 275, 9}, { 225, 8}, 
    { 147, 9}, { 403, 9}, {  83, 9}, { 339, 9}, { 211, 9}, { 467, 9}, {  51, 9}, {  17, 8}, 
    { 307, 9}, { 179, 9}, { 435, 9}, { 115, 9}, { 371, 9}, { 243, 9}, { 145, 8}, {  82, 7}, 
    { 499, 9}, {  11, 9}, { 267, 9}, { 139, 9}, { 395, 9}, {  75, 9}, { 331, 9}, {  81, 8}, 
    { 203, 9}, { 459, 9}, {  43, 9}, { 299, 9}, { 171, 9}, { 427, 9}, { 107, 9}, {  50, 7}, 
    { 363, 9}, { 235, 9}, { 491, 9}, { 209, 8}, {  27, 9}, { 283, 9}, { 155, 9}, { 114, 7}, 
    { 411, 9}, {  49, 8}, { 177, 8}, {  10, 7}, { 113, 8}, {  74, 7}, {  42, 7}, {   4, 5}, 
    {  91, 9}, { 347, 9}, { 219, 9}, { 475, 9}, {  59, 9}, { 315, 9}, { 187, 9}, { 241, 8}, 
    { 443, 9}, { 123, 9}, { 379, 9}, { 251, 9}, { 507, 9}, {   7, 9}, { 263, 9}, {   9, 8}, 
    { 135, 9}, { 391, 9}, {  71, 9}, { 327, 9}, { 199, 9}, { 455, 9}, {  39, 9}, { 137, 8}, 
    { 295, 9}, { 167, 9}, { 423, 9}, { 103, 9}, { 359, 9}, { 231, 9}, {  73, 8}, { 106, 7}, 
    { 487, 9}, {  23, 9}, { 279, 9}, { 151, 9}, { 407, 9}, {  87, 9}, { 343, 9}, { 201, 8}, 
    { 215, 9}, { 471, 9}, {  55, 9}, { 311, 9}, { 183, 9}, { 439, 9}, { 119, 9}, {  26, 7}, 
    { 375, 9}, { 247, 9}, { 503, 9}, {  15, 9}, { 271, 9}, { 143, 9}, { 399, 9}, {  90, 7}, 
    {  79, 9}, {  41, 8}, { 335, 9}, {  58, 7}, { 169, 8}, { 122, 7}, {   6, 7}, {  52, 6}, 
    { 207, 9}, { 463, 9}, {  47, 9}, { 303, 9}, { 175, 9}, { 431, 9}, { 111, 9}, { 105, 8}, 
    { 367, 9}, { 239, 9}, { 495, 9}, { 233, 8}, {  31, 9}, { 287, 9}, {  25, 8}, {  70, 7}, 
    { 159, 9}, { 415, 9}, {  95, 9}, { 153, 8}, { 351, 9}, { 223, 9}, { 479, 9}, {  38, 7}, 
    {  63, 9}, { 319, 9}, { 191, 9}, {  89, 8}, { 217, 8}, { 102, 7}, {  22, 7}, {  12, 6}, 
    { 447, 9}, { 127, 9}, { 383, 9}, {  57, 8}, { 255, 9}, { 185, 8}, { 121, 8}, {  86, 7}, 
    { 511, 9}, { 249, 8}, {   5, 8}, {  54, 7}, { 133, 8}, { 118, 7}, {  14, 7}, {  44, 6}, 
    {  69, 8}, { 197, 8}, {  37, 8}, {  78, 7}, { 165, 8}, {  46, 7}, { 110, 7}, {  28, 6}, 
    {  30, 7}, {  94, 7}, {  62, 7}, {  60, 6}, { 126, 7}, {   2, 6}, {  34, 6}, {   0, 3}, 
  },
  #endif
  #ifdef HUFFMAN_DECODE
  {
    { 3,255}, { 8,  0}, { 6,253}, { 9, 57}, { 5,127}, { 8,234}, { 7,190}, { 9,141}, 
    { 3,255}, { 8,143}, { 7,123}, { 9, 97}, { 6,223}, { 9, 22}, { 7,238}, { 9,179}, 
    { 3,255}, { 8, 87}, { 7, 31}, { 9, 77}, { 6, 63}, { 9,  5}, { 7,222}, { 9,161}, 
    { 3,255}, { 8,206}, { 7,175}, { 9,116}, { 6,247}, { 9, 39}, { 7,248}, { 9,204}, 
    { 3,255}, { 8, 59}, { 6,254}, { 9, 69}, { 5,127}, { 8,242}, { 7,215}, { 9,150}, 
    { 3,255}, { 8,185}, { 7,126}, { 9,106}, { 6,239}, { 9, 30}, { 7,245}, { 9,194}, 
    { 3,255}, { 8,121}, { 7,111}, { 9, 86}, { 6,191}, { 9, 13}, { 7,235}, { 9,170}, 
    { 3,255}, { 8,227}, { 7,187}, { 9,132}, { 6,251}, { 9, 48}, { 7,250}, { 9,216}, 
    { 3,255}, { 8, 47}, { 6,253}, { 9, 65}, { 5,127}, { 8,240}, { 7,207}, { 9,146}, 
    { 3,255}, { 8,158}, { 7,125}, { 9,101}, { 6,223}, { 9, 26}, { 7,243}, { 9,184}, 
    { 3,255}, { 8,103}, { 7, 95}, { 9, 82}, { 6, 63}, { 9,  9}, { 7,231}, { 9,165}, 
    { 3,255}, { 8,219}, { 7,183}, { 9,128}, { 6,247}, { 9, 43}, { 7,249}, { 9,210}, 
    { 3,255}, { 8, 62}, { 6,254}, { 9, 73}, { 5,127}, { 9,  1}, { 7,221}, { 9,155}, 
    { 3,255}, { 8,199}, { 7,159}, { 9,110}, { 6,239}, { 9, 35}, { 7,246}, { 9,198}, 
    { 3,255}, { 8,124}, { 7,119}, { 9, 91}, { 6,191}, { 9, 18}, { 7,237}, { 9,174}, 
    { 3,255}, { 8,230}, { 7,189}, { 9,137}, { 6,251}, { 9, 52}, { 7,252}, { 9,225}, 
    { 3,255}, { 8, 15}, { 6,253}, { 9, 60}, { 5,127}, { 8,236}, { 7,190}, { 9,144}, 
    { 3,255}, { 8,151}, { 7,123}, { 9, 99}, { 6,223}, { 9, 24}, { 7,238}, { 9,181}, 
    { 3,255}, { 8, 94}, { 7, 31}, { 9, 80}, { 6, 63}, { 9,  7}, { 7,222}, { 9,163}, 
    { 3,255}, { 8,211}, { 7,175}, { 9,118}, { 6,247}, { 9, 41}, { 7,248}, { 9,208}, 
    { 3,255}, { 8, 61}, { 6,254}, { 9, 71}, { 5,127}, { 8,244}, { 7,215}, { 9,153}, 
    { 3,255}, { 8,188}, { 7,126}, { 9,108}, { 6,239}, { 9, 33}, { 7,245}, { 9,196}, 
    { 3,255}, { 8,122}, { 7,111}, { 9, 89}, { 6,191}, { 9, 16}, { 7,235}, { 9,172}, 
    { 3,255}, { 8,229}, { 7,187}, { 9,134}, { 6,251}, { 9, 50}, { 7,250}, { 9,218}, 
    { 3,255}, { 8, 55}, { 6,253}, { 9, 67}, { 5,127}, { 8,241}, { 7,207}, { 9,148}, 
    { 3,255}, { 8,167}, { 7,125}, { 9,104}, { 6,223}, { 9, 28}, { 7,243}, { 9,192}, 
    { 3,255}, { 8,115}, { 7, 95}, { 9, 84}, { 6, 63}, { 9, 11}, { 7,231}, { 9,168}, 
    { 3,255}, { 8,220}, { 7,183}, { 9,130}, { 6,247}, { 9, 45}, { 7,249}, { 9,213}, 
    { 3,255}, { 8, 79}, { 6,254}, { 9, 75}, { 5,127}, { 9,  3}, { 7,221}, { 9,157}, 
    { 3,255}, { 8,203}, { 7,159}, { 9,113}, { 6,239}, { 9, 37}, { 7,246}, { 9,201}, 
    { 3,255}, { 8,135}, { 7,119}, { 9, 93}, { 6,191}, { 9, 20}, { 7,237}, { 9,177}, 
    { 3,255}, { 8,233}, { 7,189}, { 9,139}, { 6,251}, { 9, 54}, { 7,252}, { 9,228}, 
    { 3,255}, { 8,  0}, { 6,253}, { 9, 58}, { 5,127}, { 8,234}, { 7,190}, { 9,142}, 
    { 3,255}, { 8,143}, { 7,123}, { 9, 98}, { 6,223}, { 9, 23}, { 7,238}, { 9,180}, 
    { 3,255}, { 8, 87}, { 7, 31}, { 9, 78}, { 6, 63}, { 9,  6}, { 7,222}, { 9,162}, 
    { 3,255}, { 8,206}, { 7,175}, { 9,117}, { 6,247}, { 9, 40}, { 7,248}, { 9,205}, 
    { 3,255}, { 8, 59}, { 6,254}, { 9, 70}, { 5,127}, { 8,242}, { 7,215}, { 9,152}, 
    { 3,255}, { 8,185}, { 7,126}, { 9,107}, { 6,239}, { 9, 32}, { 7,245}, { 9,195}, 
    { 3,255}, { 8,121}, { 7,111}, { 9, 88}, { 6,191}, { 9, 14}, { 7,235}, { 9,171}, 
    { 3,255}, { 8,227}, { 7,187}, { 9,133}, { 6,251}, { 9, 49}, { 7,250}, { 9,217}, 
    { 3,255}, { 8, 47}, { 6,253}, { 9, 66}, { 5,127}, { 8,240}, { 7,207}, { 9,147}, 
    { 3,255}, { 8,158}, { 7,125}, { 9,102}, { 6,223}, { 9, 27}, { 7,243}, { 9,186}, 
    { 3,255}, { 8,103}, { 7, 95}, { 9, 83}, { 6, 63}, { 9, 10}, { 7,231}, { 9,166}, 
    { 3,255}, { 8,219}, { 7,183}, { 9,129}, { 6,247}, { 9, 44}, { 7,249}, { 9,212}, 
    { 3,255}, { 8, 62}, { 6,254}, { 9, 74}, { 5,127}, { 9,  2}, { 7,221}, { 9,156}, 
    { 3,255}, { 8,199}, { 7,159}, { 9,112}, { 6,239}, { 9, 36}, { 7,246}, { 9,200}, 
    { 3,255}, { 8,124}, { 7,119}, { 9, 92}, { 6,191}, { 9, 19}, { 7,237}, { 9,176}, 
    { 3,255}, { 8,230}, { 7,189}, { 9,138}, { 6,251}, { 9, 53}, { 7,252}, { 9,226}, 
    { 3,255}, { 8, 15}, { 6,253}, { 9, 64}, { 5,127}, { 8,236}, { 7,190}, { 9,145}, 
    { 3,255}, { 8,151}, { 7,123}, { 9,100}, { 6,223}, { 9, 25}, { 7,238}, { 9,182}, 
    { 3,255}, { 8, 94}, { 7, 31}, { 9, 81}, { 6, 63}, { 9,  8}, { 7,222}, { 9,164}, 
    { 3,255}, { 8,211}, { 7,175}, { 9,120}, { 6,247}, { 9, 42}, { 7,248}, { 9,209}, 
    { 3,255}, { 8, 61}, { 6,254}, { 9, 72}, { 5,127}, { 8,244}, { 7,215}, { 9,154}, 
    { 3,255}, { 8,188}, { 7,126}, { 9,109}, { 6,239}, { 9, 34}, { 7,245}, { 9,197}, 
    { 3,255}, { 8,122}, { 7,111}, { 9, 90}, { 6,191}, { 9, 17}, { 7,235}, { 9,173}, 
    { 3,255}, { 8,229}, { 7,187}, { 9,136}, { 6,251}, { 9, 51}, { 7,250}, { 9,224}, 
    { 3,255}, { 8, 55}, { 6,253}, { 9, 68}, { 5,127}, { 8,241}, { 7,207}, { 9,149}, 
    { 3,255}, { 8,167}, { 7,125}, { 9,105}, { 6,223}, { 9, 29}, { 7,243}, { 9,193}, 
    { 3,255}, { 8,115}, { 7, 95}, { 9, 85}, { 6, 63}, { 9, 12}, { 7,231}, { 9,169}, 
    { 3,255}, { 8,220}, { 7,183}, { 9,131}, { 6,247}, { 9, 46}, { 7,249}, { 9,214}, 
    { 3,255}, { 8, 79}, { 6,254}, { 9, 76}, { 5,127}, { 9,  4}, { 7,221}, { 9,160}, 
    { 3,255}, { 8,203}, { 7,159}, { 9,114}, { 6,239}, { 9, 38}, { 7,246}, { 9,202}, 
    { 3,255}, { 8,135}, { 7,119}, { 9, 96}, { 6,191}, { 9, 21}, { 7,237}, { 9,178}, 
    { 3,255}, { 8,233}, { 7,189}, { 9,140}, { 6,251}, { 9, 56}, { 7,252}, { 9,232}, 
    { 3,255}, { 8,  0}, { 6,253}, { 9, 57}, { 5,127}, { 8,234}, { 7,190}, { 9,141}, 
    { 3,255}, { 8,143}, { 7,123}, { 9, 97}, { 6,223}, { 9, 22}, { 7,238}, { 9,179}, 
    { 3,255}, { 8, 87}, { 7, 31}, { 9, 77}, { 6, 63}, { 9,  5}, { 7,222}, { 9,161}, 
    { 3,255}, { 8,206}, { 7,175}, { 9,116}, { 6,247}, { 9, 39}, { 7,248}, { 9,204}, 
    { 3,255}, { 8, 59}, { 6,254}, { 9, 69}, { 5,127}, { 8,242}, { 7,215}, { 9,150}, 
    { 3,255}, { 8,185}, { 7,126}, { 9,106}, { 6,239}, { 9, 30}, { 7,245}, { 9,194}, 
    { 3,255}, { 8,121}, { 7,111}, { 9, 86}, { 6,191}, { 9, 13}, { 7,235}, { 9,170}, 
    { 3,255}, { 8,227}, { 7,187}, { 9,132}, { 6,251}, { 9, 48}, { 7,250}, { 9,216}, 
    { 3,255}, { 8, 47}, { 6,253}, { 9, 65}, { 5,127}, { 8,240}, { 7,207}, { 9,146}, 
    { 3,255}, { 8,158}, { 7,125}, { 9,101}, { 6,223}, { 9, 26}, { 7,243}, { 9,184}, 
    { 3,255}, { 8,103}, { 7, 95}, { 9, 82}, { 6, 63}, { 9,  9}, { 7,231}, { 9,165}, 
    { 3,255}, { 8,219}, { 7,183}, { 9,128}, { 6,247}, { 9, 43}, { 7,249}, { 9,210}, 
    { 3,255}, { 8, 62}, { 6,254}, { 9, 73}, { 5,127}, { 9,  1}, { 7,221}, { 9,155}, 
    { 3,255}, { 8,199}, { 7,159}, { 9,110}, { 6,239}, { 9, 35}, { 7,246}, { 9,198}, 
    { 3,255}, { 8,124}, { 7,119}, { 9, 91}, { 6,191}, { 9, 18}, { 7,237}, { 9,174}, 
    { 3,255}, { 8,230}, { 7,189}, { 9,137}, { 6,251}, { 9, 52}, { 7,252}, { 9,225}, 
    { 3,255}, { 8, 15}, { 6,253}, { 9, 60}, { 5,127}, { 8,236}, { 7,190}, { 9,144}, 
    { 3,255}, { 8,151}, { 7,123}, { 9, 99}, { 6,223}, { 9, 24}, { 7,238}, { 9,181}, 
    { 3,255}, { 8, 94}, { 7, 31}, { 9, 80}, { 6, 63}, { 9,  7}, { 7,222}, { 9,163}, 
    { 3,255}, { 8,211}, { 7,175}, { 9,118}, { 6,247}, { 9, 41}, { 7,248}, { 9,208}, 
    { 3,255}, { 8, 61}, { 6,254}, { 9, 71}, { 5,127}, { 8,244}, { 7,215}, { 9,153}, 
    { 3,255}, { 8,188}, { 7,126}, { 9,108}, { 6,239}, { 9, 33}, { 7,245}, { 9,196}, 
    { 3,255}, { 8,122}, { 7,111}, { 9, 89}, { 6,191}, { 9, 16}, { 7,235}, { 9,172}, 
    { 3,255}, { 8,229}, { 7,187}, { 9,134}, { 6,251}, { 9, 50}, { 7,250}, { 9,218}, 
    { 3,255}, { 8, 55}, { 6,253}, { 9, 67}, { 5,127}, { 8,241}, { 7,207}, { 9,148}, 
    { 3,255}, { 8,167}, { 7,125}, { 9,104}, { 6,223}, { 9, 28}, { 7,243}, { 9,192}, 
    { 3,255}, { 8,115}, { 7, 95}, { 9, 84}, { 6, 63}, { 9, 11}, { 7,231}, { 9,168}, 
    { 3,255}, { 8,220}, { 7,183}, { 9,130}, { 6,247}, { 9, 45}, { 7,249}, { 9,213}, 
    { 3,255}, { 8, 79}, { 6,254}, { 9, 75}, { 5,127}, { 9,  3}, { 7,221}, { 9,157}, 
    { 3,255}, { 8,203}, { 7,159}, { 9,113}, { 6,239}, { 9, 37}, { 7,246}, { 9,201}, 
    { 3,255}, { 8,135}, { 7,119}, { 9, 93}, { 6,191}, { 9, 20}, { 7,237}, { 9,177}, 
    { 3,255}, { 8,233}, { 7,189}, { 9,139}, { 6,251}, { 9, 54}, { 7,252}, { 9,228}, 
    { 3,255}, { 8,  0}, { 6,253}, { 9, 58}, { 5,127}, { 8,234}, { 7,190}, { 9,142}, 
    { 3,255}, { 8,143}, { 7,123}, { 9, 98}, { 6,223}, { 9, 23}, { 7,238}, { 9,180}, 
    { 3,255}, { 8, 87}, { 7, 31}, { 9, 78}, { 6, 63}, { 9,  6}, { 7,222}, { 9,162}, 
    { 3,255}, { 8,206}, { 7,175}, { 9,117}, { 6,247}, { 9, 40}, { 7,248}, { 9,205}, 
    { 3,255}, { 8, 59}, { 6,254}, { 9, 70}, { 5,127}, { 8,242}, { 7,215}, { 9,152}, 
    { 3,255}, { 8,185}, { 7,126}, { 9,107}, { 6,239}, { 9, 32}, { 7,245}, { 9,195}, 
    { 3,255}, { 8,121}, { 7,111}, { 9, 88}, { 6,191}, { 9, 14}, { 7,235}, { 9,171}, 
    { 3,255}, { 8,227}, { 7,187}, { 9,133}, { 6,251}, { 9, 49}, { 7,250}, { 9,217}, 
    { 3,255}, { 8, 47}, { 6,253}, { 9, 66}, { 5,127}, { 8,240}, { 7,207}, { 9,147}, 
    { 3,255}, { 8,158}, { 7,125}, { 9,102}, { 6,223}, { 9, 27}, { 7,243}, { 9,186}, 
    { 3,255}, { 8,103}, { 7, 95}, { 9, 83}, { 6, 63}, { 9, 10}, { 7,231}, { 9,166}, 
    { 3,255}, { 8,219}, { 7,183}, { 9,129}, { 6,247}, { 9, 44}, { 7,249}, { 9,212}, 
    { 3,255}, { 8, 62}, { 6,254}, { 9, 74}, { 5,127}, { 9,  2}, { 7,221}, { 9,156}, 
    { 3,255}, { 8,199}, { 7,159}, { 9,112}, { 6,239}, { 9, 36}, { 7,246}, { 9,200}, 
    { 3,255}, { 8,124}, { 7,119}, { 9, 92}, { 6,191}, { 9, 19}, { 7,237}, { 9,176}, 
    { 3,255}, { 8,230}, { 7,189}, { 9,138}, { 6,251}, { 9, 53}, { 7,252}, { 9,226}, 
    { 3,255}, { 8, 15}, { 6,253}, { 9, 64}, { 5,127}, { 8,236}, { 7,190}, { 9,145}, 
    { 3,255}, { 8,151}, { 7,123}, { 9,100}, { 6,223}, { 9, 25}, { 7,238}, { 9,182}, 
    { 3,255}, { 8, 94}, { 7, 31}, { 9, 81}, { 6, 63}, { 9,  8}, { 7,222}, { 9,164}, 
    { 3,255}, { 8,211}, { 7,175}, { 9,120}, { 6,247}, { 9, 42}, { 7,248}, { 9,209}, 
    { 3,255}, { 8, 61}, { 6,254}, { 9, 72}, { 5,127}, { 8,244}, { 7,215}, { 9,154}, 
    { 3,255}, { 8,188}, { 7,126}, { 9,109}, { 6,239}, { 9, 34}, { 7,245}, { 9,197}, 
    { 3,255}, { 8,122}, { 7,111}, { 9, 90}, { 6,191}, { 9, 17}, { 7,235}, { 9,173}, 
    { 3,255}, { 8,229}, { 7,187}, { 9,136}, { 6,251}, { 9, 51}, { 7,250}, { 9,224}, 
    { 3,255}, { 8, 55}, { 6,253}, { 9, 68}, { 5,127}, { 8,241}, { 7,207}, { 9,149}, 
    { 3,255}, { 8,167}, { 7,125}, { 9,105}, { 6,223}, { 9, 29}, { 7,243}, { 9,193}, 
    { 3,255}, { 8,115}, { 7, 95}, { 9, 85}, { 6, 63}, { 9, 12}, { 7,231}, { 9,169}, 
    { 3,255}, { 8,220}, { 7,183}, { 9,131}, { 6,247}, { 9, 46}, { 7,249}, { 9,214}, 
    { 3,255}, { 8, 79}, { 6,254}, { 9, 76}, { 5,127}, { 9,  4}, { 7,221}, { 9,160}, 
    { 3,255}, { 8,203}, { 7,159}, { 9,114}, { 6,239}, { 9, 38}, { 7,246}, { 9,202}, 
    { 3,255}, { 8,135}, { 7,119}, { 9, 96}, { 6,191}, { 9, 21}, { 7,237}, { 9,178}, 
    { 3,255}, { 8,233}, { 7,189}, { 9,140}, { 6,251}, { 9, 56}, { 7,252}, { 9,232}, 
    { 3,255}, { 8,  0}, { 6,253}, { 9, 57}, { 5,127}, { 8,234}, { 7,190}, { 9,141}, 
    { 3,255}, { 8,143}, { 7,123}, { 9, 97}, { 6,223}, { 9, 22}, { 7,238}, { 9,179}, 
    { 3,255}, { 8, 87}, { 7, 31}, { 9, 77}, { 6, 63}, { 9,  5}, { 7,222}, { 9,161}, 
    { 3,255}, { 8,206}, { 7,175}, { 9,116}, { 6,247}, { 9, 39}, { 7,248}, { 9,204}, 
    { 3,255}, { 8, 59}, { 6,254}, { 9, 69}, { 5,127}, { 8,242}, { 7,215}, { 9,150}, 
    { 3,255}, { 8,185}, { 7,126}, { 9,106}, { 6,239}, { 9, 30}, { 7,245}, { 9,194}, 
    { 3,255}, { 8,121}, { 7,111}, { 9, 86}, { 6,191}, { 9, 13}, { 7,235}, { 9,170}, 
    { 3,255}, { 8,227}, { 7,187}, { 9,132}, { 6,251}, { 9, 48}, { 7,250}, { 9,216}, 
    { 3,255}, { 8, 47}, { 6,253}, { 9, 65}, { 5,127}, { 8,240}, { 7,207}, { 9,146}, 
    { 3,255}, { 8,158}, { 7,125}, { 9,101}, { 6,223}, { 9, 26}, { 7,243}, { 9,184}, 
    { 3,255}, { 8,103}, { 7, 95}, { 9, 82}, { 6, 63}, { 9,  9}, { 7,231}, { 9,165}, 
    { 3,255}, { 8,219}, { 7,183}, { 9,128}, { 6,247}, { 9, 43}, { 7,249}, { 9,210}, 
    { 3,255}, { 8, 62}, { 6,254}, { 9, 73}, { 5,127}, { 9,  1}, { 7,221}, { 9,155}, 
    { 3,255}, { 8,199}, { 7,159}, { 9,110}, { 6,239}, { 9, 35}, { 7,246}, { 9,198}, 
    { 3,255}, { 8,124}, { 7,119}, { 9, 91}, { 6,191}, { 9, 18}, { 7,237}, { 9,174}, 
    { 3,255}, { 8,230}, { 7,189}, { 9,137}, { 6,251}, { 9, 52}, { 7,252}, { 9,225}, 
    { 3,255}, { 8, 15}, { 6,253}, { 9, 60}, { 5,127}, { 8,236}, { 7,190}, { 9,144}, 
    { 3,255}, { 8,151}, { 7,123}, { 9, 99}, { 6,223}, { 9, 24}, { 7,238}, { 9,181}, 
    { 3,255}, { 8, 94}, { 7, 31}, { 9, 80}, { 6, 63}, { 9,  7}, { 7,222}, { 9,163}, 
    { 3,255}, { 8,211}, { 7,175}, { 9,118}, { 6,247}, { 9, 41}, { 7,248}, { 9,208}, 
    { 3,255}, { 8, 61}, { 6,254}, { 9, 71}, { 5,127}, { 8,244}, { 7,215}, { 9,153}, 
    { 3,255}, { 8,188}, { 7,126}, { 9,108}, { 6,239}, { 9, 33}, { 7,245}, { 9,196}, 
    { 3,255}, { 8,122}, { 7,111}, { 9, 89}, { 6,191}, { 9, 16}, { 7,235}, { 9,172}, 
    { 3,255}, { 8,229}, { 7,187}, { 9,134}, { 6,251}, { 9, 50}, { 7,250}, { 9,218}, 
    { 3,255}, { 8, 55}, { 6,253}, { 9, 67}, { 5,127}, { 8,241}, { 7,207}, { 9,148}, 
    { 3,255}, { 8,167}, { 7,125}, { 9,104}, { 6,223}, { 9, 28}, { 7,243}, { 9,192}, 
    { 3,255}, { 8,115}, { 7, 95}, { 9, 84}, { 6, 63}, { 9, 11}, { 7,231}, { 9,168}, 
    { 3,255}, { 8,220}, { 7,183}, { 9,130}, { 6,247}, { 9, 45}, { 7,249}, { 9,213}, 
    { 3,255}, { 8, 79}, { 6,254}, { 9, 75}, { 5,127}, { 9,  3}, { 7,221}, { 9,157}, 
    { 3,255}, { 8,203}, { 7,159}, { 9,113}, { 6,239}, { 9, 37}, { 7,246}, { 9,201}, 
    { 3,255}, { 8,135}, { 7,119}, { 9, 93}, { 6,191}, { 9, 20}, { 7,237}, { 9,177}, 
    { 3,255}, { 8,233}, { 7,189}, { 9,139}, { 6,251}, { 9, 54}, { 7,252}, { 9,228}, 
    { 3,255}, { 8,  0}, { 6,253}, { 9, 58}, { 5,127}, { 8,234}, { 7,190}, { 9,142}, 
    { 3,255}, { 8,143}, { 7,123}, { 9, 98}, { 6,223}, { 9, 23}, { 7,238}, { 9,180}, 
    { 3,255}, { 8, 87}, { 7, 31}, { 9, 78}, { 6, 63}, { 9,  6}, { 7,222}, { 9,162}, 
    { 3,255}, { 8,206}, { 7,175}, { 9,117}, { 6,247}, { 9, 40}, { 7,248}, { 9,205}, 
    { 3,255}, { 8, 59}, { 6,254}, { 9, 70}, { 5,127}, { 8,242}, { 7,215}, { 9,152}, 
    { 3,255}, { 8,185}, { 7,126}, { 9,107}, { 6,239}, { 9, 32}, { 7,245}, { 9,195}, 
    { 3,255}, { 8,121}, { 7,111}, { 9, 88}, { 6,191}, { 9, 14}, { 7,235}, { 9,171}, 
    { 3,255}, { 8,227}, { 7,187}, { 9,133}, { 6,251}, { 9, 49}, { 7,250}, { 9,217}, 
    { 3,255}, { 8, 47}, { 6,253}, { 9, 66}, { 5,127}, { 8,240}, { 7,207}, { 9,147}, 
    { 3,255}, { 8,158}, { 7,125}, { 9,102}, { 6,223}, { 9, 27}, { 7,243}, { 9,186}, 
    { 3,255}, { 8,103}, { 7, 95}, { 9, 83}, { 6, 63}, { 9, 10}, { 7,231}, { 9,166}, 
    { 3,255}, { 8,219}, { 7,183}, { 9,129}, { 6,247}, { 9, 44}, { 7,249}, { 9,212}, 
    { 3,255}, { 8, 62}, { 6,254}, { 9, 74}, { 5,127}, { 9,  2}, { 7,221}, { 9,156}, 
    { 3,255}, { 8,199}, { 7,159}, { 9,112}, { 6,239}, { 9, 36}, { 7,246}, { 9,200}, 
    { 3,255}, { 8,124}, { 7,119}, { 9, 92}, { 6,191}, { 9, 19}, { 7,237}, { 9,176}, 
    { 3,255}, { 8,230}, { 7,189}, { 9,138}, { 6,251}, { 9, 53}, { 7,252}, { 9,226}, 
    { 3,255}, { 8, 15}, { 6,253}, { 9, 64}, { 5,127}, { 8,236}, { 7,190}, { 9,145}, 
    { 3,255}, { 8,151}, { 7,123}, { 9,100}, { 6,223}, { 9, 25}, { 7,238}, { 9,182}, 
    { 3,255}, { 8, 94}, { 7, 31}, { 9, 81}, { 6, 63}, { 9,  8}, { 7,222}, { 9,164}, 
    { 3,255}, { 8,211}, { 7,175}, { 9,120}, { 6,247}, { 9, 42}, { 7,248}, { 9,209}, 
    { 3,255}, { 8, 61}, { 6,254}, { 9, 72}, { 5,127}, { 8,244}, { 7,215}, { 9,154}, 
    { 3,255}, { 8,188}, { 7,126}, { 9,109}, { 6,239}, { 9, 34}, { 7,245}, { 9,197}, 
    { 3,255}, { 8,122}, { 7,111}, { 9, 90}, { 6,191}, { 9, 17}, { 7,235}, { 9,173}, 
    { 3,255}, { 8,229}, { 7,187}, { 9,136}, { 6,251}, { 9, 51}, { 7,250}, { 9,224}, 
    { 3,255}, { 8, 55}, { 6,253}, { 9, 68}, { 5,127}, { 8,241}, { 7,207}, { 9,149}, 
    { 3,255}, { 8,167}, { 7,125}, { 9,105}, { 6,223}, { 9, 29}, { 7,243}, { 9,193}, 
    { 3,255}, { 8,115}, { 7, 95}, { 9, 85}, { 6, 63}, { 9, 12}, { 7,231}, { 9,169}, 
    { 3,255}, { 8,220}, { 7,183}, { 9,131}, { 6,247}, { 9, 46}, { 7,249}, { 9,214}, 
    { 3,255}, { 8, 79}, { 6,254}, { 9, 76}, { 5,127}, { 9,  4}, { 7,221}, { 9,160}, 
    { 3,255}, { 8,203}, { 7,159}, { 9,114}, { 6,239}, { 9, 38}, { 7,246}, { 9,202}, 
    { 3,255}, { 8,135}, { 7,119}, { 9, 96}, { 6,191}, { 9, 21}, { 7,237}, { 9,178}, 
    { 3,255}, { 8,233}, { 7,189}, { 9,140}, { 6,251}, { 9, 56}, { 7,252}, { 9,232}, 
    { 3,255}, { 8,  0}, { 6,253}, { 9, 57}, { 5,127}, { 8,234}, { 7,190}, { 9,141}, 
    { 3,255}, { 8,143}, { 7,123}, { 9, 97}, { 6,223}, { 9, 22}, { 7,238}, { 9,179}, 
    { 3,255}, { 8, 87}, { 7, 31}, { 9, 77}, { 6, 63}, { 9,  5}, { 7,222}, { 9,161}, 
    { 3,255}, { 8,206}, { 7,175}, { 9,116}, { 6,247}, { 9, 39}, { 7,248}, { 9,204}, 
    { 3,255}, { 8, 59}, { 6,254}, { 9, 69}, { 5,127}, { 8,242}, { 7,215}, { 9,150}, 
    { 3,255}, { 8,185}, { 7,126}, { 9,106}, { 6,239}, { 9, 30}, { 7,245}, { 9,194}, 
    { 3,255}, { 8,121}, { 7,111}, { 9, 86}, { 6,191}, { 9, 13}, { 7,235}, { 9,170}, 
    { 3,255}, { 8,227}, { 7,187}, { 9,132}, { 6,251}, { 9, 48}, { 7,250}, { 9,216}, 
    { 3,255}, { 8, 47}, { 6,253}, { 9, 65}, { 5,127}, { 8,240}, { 7,207}, { 9,146}, 
    { 3,255}, { 8,158}, { 7,125}, { 9,101}, { 6,223}, { 9, 26}, { 7,243}, { 9,184}, 
    { 3,255}, { 8,103}, { 7, 95}, { 9, 82}, { 6, 63}, { 9,  9}, { 7,231}, { 9,165}, 
    { 3,255}, { 8,219}, { 7,183}, { 9,128}, { 6,247}, { 9, 43}, { 7,249}, { 9,210}, 
    { 3,255}, { 8, 62}, { 6,254}, { 9, 73}, { 5,127}, { 9,  1}, { 7,221}, { 9,155}, 
    { 3,255}, { 8,199}, { 7,159}, { 9,110}, { 6,239}, { 9, 35}, { 7,246}, { 9,198}, 
    { 3,255}, { 8,124}, { 7,119}, { 9, 91}, { 6,191}, { 9, 18}, { 7,237}, { 9,174}, 
    { 3,255}, { 8,230}, { 7,189}, { 9,137}, { 6,251}, { 9, 52}, { 7,252}, { 9,225}, 
    { 3,255}, { 8, 15}, { 6,253}, { 9, 60}, { 5,127}, { 8,236}, { 7,190}, { 9,144}, 
    { 3,255}, { 8,151}, { 7,123}, { 9, 99}, { 6,223}, { 9, 24}, { 7,238}, { 9,181}, 
    { 3,255}, { 8, 94}, { 7, 31}, { 9, 80}, { 6, 63}, { 9,  7}, { 7,222}, { 9,163}, 
    { 3,255}, { 8,211}, { 7,175}, { 9,118}, { 6,247}, { 9, 41}, { 7,248}, { 9,208}, 
    { 3,255}, { 8, 61}, { 6,254}, { 9, 71}, { 5,127}, { 8,244}, { 7,215}, { 9,153}, 
    { 3,255}, { 8,188}, { 7,126}, { 9,108}, { 6,239}, { 9, 33}, { 7,245}, { 9,196}, 
    { 3,255}, { 8,122}, { 7,111}, { 9, 89}, { 6,191}, { 9, 16}, { 7,235}, { 9,172}, 
    { 3,255}, { 8,229}, { 7,187}, { 9,134}, { 6,251}, { 9, 50}, { 7,250}, { 9,218}, 
    { 3,255}, { 8, 55}, { 6,253}, { 9, 67}, { 5,127}, { 8,241}, { 7,207}, { 9,148}, 
    { 3,255}, { 8,167}, { 7,125}, { 9,104}, { 6,223}, { 9, 28}, { 7,243}, { 9,192}, 
    { 3,255}, { 8,115}, { 7, 95}, { 9, 84}, { 6, 63}, { 9, 11}, { 7,231}, { 9,168}, 
    { 3,255}, { 8,220}, { 7,183}, { 9,130}, { 6,247}, { 9, 45}, { 7,249}, { 9,213}, 
    { 3,255}, { 8, 79}, { 6,254}, { 9, 75}, { 5,127}, { 9,  3}, { 7,221}, { 9,157}, 
    { 3,255}, { 8,203}, { 7,159}, { 9,113}, { 6,239}, { 9, 37}, { 7,246}, { 9,201}, 
    { 3,255}, { 8,135}, { 7,119}, { 9, 93}, { 6,191}, { 9, 20}, { 7,237}, { 9,177}, 
    { 3,255}, { 8,233}, { 7,189}, { 9,139}, { 6,251}, { 9, 54}, { 7,252}, { 9,228}, 
    { 3,255}, { 8,  0}, { 6,253}, { 9, 58}, { 5,127}, { 8,234}, { 7,190}, { 9,142}, 
    { 3,255}, { 8,143}, { 7,123}, { 9, 98}, { 6,223}, { 9, 23}, { 7,238}, { 9,180}, 
    { 3,255}, { 8, 87}, { 7, 31}, { 9, 78}, { 6, 63}, { 9,  6}, { 7,222}, { 9,162}, 
    { 3,255}, { 8,206}, { 7,175}, { 9,117}, { 6,247}, { 9, 40}, { 7,248}, { 9,205}, 
    { 3,255}, { 8, 59}, { 6,254}, { 9, 70}, { 5,127}, { 8,242}, { 7,215}, { 9,152}, 
    { 3,255}, { 8,185}, { 7,126}, { 9,107}, { 6,239}, { 9, 32}, { 7,245}, { 9,195}, 
    { 3,255}, { 8,121}, { 7,111}, { 9, 88}, { 6,191}, { 9, 14}, { 7,235}, { 9,171}, 
    { 3,255}, { 8,227}, { 7,187}, { 9,133}, { 6,251}, { 9, 49}, { 7,250}, { 9,217}, 
    { 3,255}, { 8, 47}, { 6,253}, { 9, 66}, { 5,127}, { 8,240}, { 7,207}, { 9,147}, 
    { 3,255}, { 8,158}, { 7,125}, { 9,102}, { 6,223}, { 9, 27}, { 7,243}, { 9,186}, 
    { 3,255}, { 8,103}, { 7, 95}, { 9, 83}, { 6, 63}, { 9, 10}, { 7,231}, { 9,166}, 
    { 3,255}, { 8,219}, { 7,183}, { 9,129}, { 6,247}, { 9, 44}, { 7,249}, { 9,212}, 
    { 3,255}, { 8, 62}, { 6,254}, { 9, 74}, { 5,127}, { 9,  2}, { 7,221}, { 9,156}, 
    { 3,255}, { 8,199}, { 7,159}, { 9,112}, { 6,239}, { 9, 36}, { 7,246}, { 9,200}, 
    { 3,255}, { 8,124}, { 7,119}, { 9, 92}, { 6,191}, { 9, 19}, { 7,237}, { 9,176}, 
    { 3,255}, { 8,230}, { 7,189}, { 9,138}, { 6,251}, { 9, 53}, { 7,252}, { 9,226}, 
    { 3,255}, { 8, 15}, { 6,253}, { 9, 64}, { 5,127}, { 8,236}, { 7,190}, { 9,145}, 
    { 3,255}, { 8,151}, { 7,123}, { 9,100}, { 6,223}, { 9, 25}, { 7,238}, { 9,182}, 
    { 3,255}, { 8, 94}, { 7, 31}, { 9, 81}, { 6, 63}, { 9,  8}, { 7,222}, { 9,164}, 
    { 3,255}, { 8,211}, { 7,175}, { 9,120}, { 6,247}, { 9, 42}, { 7,248}, { 9,209}, 
    { 3,255}, { 8, 61}, { 6,254}, { 9, 72}, { 5,127}, { 8,244}, { 7,215}, { 9,154}, 
    { 3,255}, { 8,188}, { 7,126}, { 9,109}, { 6,239}, { 9, 34}, { 7,245}, { 9,197}, 
    { 3,255}, { 8,122}, { 7,111}, { 9, 90}, { 6,191}, { 9, 17}, { 7,235}, { 9,173}, 
    { 3,255}, { 8,229}, { 7,187}, { 9,136}, { 6,251}, { 9, 51}, { 7,250}, { 9,224}, 
    { 3,255}, { 8, 55}, { 6,253}, { 9, 68}, { 5,127}, { 8,241}, { 7,207}, { 9,149}, 
    { 3,255}, { 8,167}, { 7,125}, { 9,105}, { 6,223}, { 9, 29}, { 7,243}, { 9,193}, 
    { 3,255}, { 8,115}, { 7, 95}, { 9, 85}, { 6, 63}, { 9, 12}, { 7,231}, { 9,169}, 
    { 3,255}, { 8,220}, { 7,183}, { 9,131}, { 6,247}, { 9, 46}, { 7,249}, { 9,214}, 
    { 3,255}, { 8, 79}, { 6,254}, { 9, 76}, { 5,127}, { 9,  4}, { 7,221}, { 9,160}, 
    { 3,255}, { 8,203}, { 7,159}, { 9,114}, { 6,239}, { 9, 38}, { 7,246}, { 9,202}, 
    { 3,255}, { 8,135}, { 7,119}, { 9, 96}, { 6,191}, { 9, 21}, { 7,237}, { 9,178}, 
    { 3,255}, { 8,233}, { 7,189}, { 9,140}, { 6,251}, { 9, 56}, { 7,252}, { 9,232}, 
  }
  #endif
};

static radaudio_huffman rada_nonzero_bitflags_huff0 =
{
  #ifdef HUFFMAN_ENCODE
  {
    { 559,11}, {1583,11}, { 303,11}, {1327,11}, { 815,11}, {1839,11}, { 175,11}, {1199,11}, 
    { 687,11}, {1711,11}, { 431,11}, {1455,11}, { 943,11}, {1967,11}, { 111,11}, { 199,10}, 
    {1135,11}, { 623,11}, {1647,11}, { 367,11}, {1391,11}, { 879,11}, {1903,11}, { 711,10}, 
    { 239,11}, {1263,11}, { 751,11}, { 455,10}, {1775,11}, { 967,10}, {  39,10}, {  53, 8}, 
    { 495,11}, {1519,11}, {1007,11}, {2031,11}, {  31,11}, {1055,11}, { 543,11}, { 551,10}, 
    {1567,11}, { 287,11}, {1311,11}, { 295,10}, { 799,11}, { 807,10}, { 167,10}, { 181, 8}, 
    {1823,11}, { 159,11}, {1183,11}, { 679,10}, { 671,11}, { 423,10}, { 935,10}, { 117, 8}, 
    {1695,11}, { 103,10}, { 615,10}, { 245, 8}, { 359,10}, {  13, 8}, {  71, 9}, {  17, 7}, 
    { 415,11}, {1439,11}, { 927,11}, {1951,11}, {  95,11}, {1119,11}, { 607,11}, { 871,10}, 
    {1631,11}, { 351,11}, {1375,11}, { 231,10}, { 863,11}, { 743,10}, { 487,10}, { 141, 8}, 
    {1887,11}, { 223,11}, {1247,11}, { 999,10}, { 735,11}, {  23,10}, { 535,10}, {  77, 8}, 
    {1759,11}, { 279,10}, { 791,10}, { 205, 8}, { 151,10}, {  45, 8}, { 173, 8}, {  81, 7}, 
    { 479,11}, {1503,11}, { 991,11}, { 663,10}, {2015,11}, { 407,10}, { 919,10}, { 109, 8}, 
    {  63,11}, {  87,10}, { 599,10}, { 237, 8}, { 343,10}, {  29, 8}, { 157, 8}, {   6, 6}, 
    {1087,11}, { 855,10}, { 215,10}, {  93, 8}, { 727,10}, { 221, 8}, {  61, 8}, {  49, 7}, 
    { 471,10}, { 189, 8}, { 125, 8}, {  38, 6}, { 253, 8}, {  22, 6}, { 113, 7}, {   4, 5}, 
    { 575,11}, {1599,11}, { 319,11}, {1343,11}, { 831,11}, {1855,11}, { 191,11}, { 983,10}, 
    {1215,11}, { 703,11}, {1727,11}, {  55,10}, { 447,11}, { 567,10}, { 311,10}, {   3, 8}, 
    {1471,11}, { 959,11}, {1983,11}, { 823,10}, { 127,11}, { 183,10}, { 695,10}, { 131, 8}, 
    {1151,11}, { 439,10}, { 951,10}, {  67, 8}, { 119,10}, { 195, 8}, {  35, 8}, {  54, 6}, 
    { 639,11}, {1663,11}, { 383,11}, { 631,10}, {1407,11}, { 375,10}, { 887,10}, { 163, 8}, 
    { 895,11}, { 247,10}, { 759,10}, {  99, 8}, { 503,10}, { 227, 8}, {  19, 8}, {   9, 7}, 
    {1919,11}, {1015,10}, {  15,10}, { 147, 8}, { 527,10}, {  83, 8}, { 327, 9}, {  73, 7}, 
    { 271,10}, { 211, 8}, {  51, 8}, {  41, 7}, { 179, 8}, {  14, 6}, { 105, 7}, {  20, 5}, 
    { 255,11}, {1279,11}, { 767,11}, { 783,10}, {1791,11}, { 143,10}, { 655,10}, { 115, 8}, 
    { 511,11}, {1535,11}, { 399,10}, { 243, 8}, { 911,10}, {  11, 8}, { 139, 8}, {  25, 7}, 
    {1023,11}, {  79,10}, { 591,10}, {  75, 8}, { 335,10}, { 203, 8}, {  43, 8}, {  46, 6}, 
    { 847,10}, { 171, 8}, { 107, 8}, {  89, 7}, { 235, 8}, {  57, 7}, { 121, 7}, {  12, 5}, 
    {2047,11}, { 207,10}, { 719,10}, {  27, 8}, { 463,10}, { 155, 8}, {  91, 8}, {  30, 6}, 
    { 975,10}, { 219, 8}, {  59, 8}, {  62, 6}, { 187, 8}, {   5, 7}, {   1, 6}, {  28, 5}, 
    {  47,10}, { 123, 8}, { 251, 8}, {  69, 7}, {   7, 8}, {  37, 7}, {  33, 6}, {   2, 5}, 
    { 135, 8}, { 101, 7}, {  21, 7}, {  18, 5}, {  85, 7}, {  10, 5}, {  26, 5}, {   0, 3}, 
  },
  #endif
  #ifdef HUFFMAN_DECODE
  {
    { 3,255}, { 6,238}, { 5,247}, { 8,143}, { 5,127}, { 7,237}, { 6,111}, { 8,244}, 
    { 3,255}, { 7,175}, { 5,253}, { 8,205}, { 5,223}, { 8, 61}, { 6,189}, {10,178}, 
    { 3,255}, { 7, 63}, { 5,251}, { 8,174}, { 5,191}, { 7,250}, { 6,125}, {10, 85}, 
    { 3,255}, { 7,207}, { 5,254}, { 8,227}, { 5,239}, { 8,109}, { 6,231}, {11, 36}, 
    { 3,255}, { 6,246}, { 5,247}, { 8,158}, { 5,127}, { 7,245}, { 6,123}, {10, 30}, 
    { 3,255}, { 7,187}, { 5,253}, { 8,214}, { 5,223}, { 8, 93}, { 6,215}, {10,240}, 
    { 3,255}, { 7,119}, { 5,251}, { 8,186}, { 5,191}, { 8, 31}, { 6,159}, {10,139}, 
    { 3,255}, { 7,221}, { 5,254}, { 8,234}, { 5,239}, { 8,118}, { 6,235}, {11,104}, 
    { 3,255}, { 6,238}, { 5,247}, { 8,155}, { 5,127}, { 7,243}, { 6,111}, { 9, 62}, 
    { 3,255}, { 7,183}, { 5,253}, { 8,211}, { 5,223}, { 8, 87}, { 6,189}, {10,209}, 
    { 3,255}, { 7, 95}, { 5,251}, { 8,181}, { 5,191}, { 7,252}, { 6,125}, {10,105}, 
    { 3,255}, { 7,219}, { 5,254}, { 8,230}, { 5,239}, { 8,115}, { 6,231}, {11, 68}, 
    { 3,255}, { 6,246}, { 5,247}, { 8,171}, { 5,127}, { 7,249}, { 6,123}, {10, 57}, 
    { 3,255}, { 7,190}, { 5,253}, { 8,218}, { 5,223}, { 8,103}, { 6,215}, {11, 14}, 
    { 3,255}, { 7,126}, { 5,251}, { 8,199}, { 5,191}, { 8, 55}, { 6,159}, {10,156}, 
    { 3,255}, { 7,222}, { 5,254}, { 8,241}, { 5,239}, { 8,122}, { 6,235}, {11,148}, 
    { 3,255}, { 6,238}, { 5,247}, { 8,151}, { 5,127}, { 7,237}, { 6,111}, { 8,248}, 
    { 3,255}, { 7,175}, { 5,253}, { 8,206}, { 5,223}, { 8, 79}, { 6,189}, {10,197}, 
    { 3,255}, { 7, 63}, { 5,251}, { 8,179}, { 5,191}, { 7,250}, { 6,125}, {10, 92}, 
    { 3,255}, { 7,207}, { 5,254}, { 8,229}, { 5,239}, { 8,110}, { 6,231}, {11, 49}, 
    { 3,255}, { 6,246}, { 5,247}, { 8,167}, { 5,127}, { 7,245}, { 6,123}, {10, 46}, 
    { 3,255}, { 7,187}, { 5,253}, { 8,217}, { 5,223}, { 8, 94}, { 6,215}, {11,  6}, 
    { 3,255}, { 7,119}, { 5,251}, { 8,188}, { 5,191}, { 8, 47}, { 6,159}, {10,149}, 
    { 3,255}, { 7,221}, { 5,254}, { 8,236}, { 5,239}, { 8,121}, { 6,235}, {11,134}, 
    { 3,255}, { 6,238}, { 5,247}, { 8,157}, { 5,127}, { 7,243}, { 6,111}, {10, 15}, 
    { 3,255}, { 7,183}, { 5,253}, { 8,213}, { 5,223}, { 8, 91}, { 6,189}, {10,225}, 
    { 3,255}, { 7, 95}, { 5,251}, { 8,185}, { 5,191}, { 7,252}, { 6,125}, {10,114}, 
    { 3,255}, { 7,219}, { 5,254}, { 8,233}, { 5,239}, { 8,117}, { 6,231}, {11, 81}, 
    { 3,255}, { 6,246}, { 5,247}, { 8,173}, { 5,127}, { 7,249}, { 6,123}, {10, 75}, 
    { 3,255}, { 7,190}, { 5,253}, { 8,220}, { 5,223}, { 8,107}, { 6,215}, {11, 24}, 
    { 3,255}, { 7,126}, { 5,251}, { 8,203}, { 5,191}, { 8, 59}, { 6,159}, {10,169}, 
    { 3,255}, { 7,222}, { 5,254}, { 8,242}, { 5,239}, { 8,124}, { 6,235}, {11,192}, 
    { 3,255}, { 6,238}, { 5,247}, { 8,143}, { 5,127}, { 7,237}, { 6,111}, { 8,244}, 
    { 3,255}, { 7,175}, { 5,253}, { 8,205}, { 5,223}, { 8, 61}, { 6,189}, {10,184}, 
    { 3,255}, { 7, 63}, { 5,251}, { 8,174}, { 5,191}, { 7,250}, { 6,125}, {10, 89}, 
    { 3,255}, { 7,207}, { 5,254}, { 8,227}, { 5,239}, { 8,109}, { 6,231}, {11, 41}, 
    { 3,255}, { 6,246}, { 5,247}, { 8,158}, { 5,127}, { 7,245}, { 6,123}, {10, 43}, 
    { 3,255}, { 7,187}, { 5,253}, { 8,214}, { 5,223}, { 8, 93}, { 6,215}, {11,  2}, 
    { 3,255}, { 7,119}, { 5,251}, { 8,186}, { 5,191}, { 8, 31}, { 6,159}, {10,142}, 
    { 3,255}, { 7,221}, { 5,254}, { 8,234}, { 5,239}, { 8,118}, { 6,235}, {11,130}, 
    { 3,255}, { 6,238}, { 5,247}, { 8,155}, { 5,127}, { 7,243}, { 6,111}, { 9,182}, 
    { 3,255}, { 7,183}, { 5,253}, { 8,211}, { 5,223}, { 8, 87}, { 6,189}, {10,212}, 
    { 3,255}, { 7, 95}, { 5,251}, { 8,181}, { 5,191}, { 7,252}, { 6,125}, {10,108}, 
    { 3,255}, { 7,219}, { 5,254}, { 8,230}, { 5,239}, { 8,115}, { 6,231}, {11, 73}, 
    { 3,255}, { 6,246}, { 5,247}, { 8,171}, { 5,127}, { 7,249}, { 6,123}, {10, 60}, 
    { 3,255}, { 7,190}, { 5,253}, { 8,218}, { 5,223}, { 8,103}, { 6,215}, {11, 19}, 
    { 3,255}, { 7,126}, { 5,251}, { 8,199}, { 5,191}, { 8, 55}, { 6,159}, {10,165}, 
    { 3,255}, { 7,222}, { 5,254}, { 8,241}, { 5,239}, { 8,122}, { 6,235}, {11,162}, 
    { 3,255}, { 6,238}, { 5,247}, { 8,151}, { 5,127}, { 7,237}, { 6,111}, { 8,248}, 
    { 3,255}, { 7,175}, { 5,253}, { 8,206}, { 5,223}, { 8, 79}, { 6,189}, {10,202}, 
    { 3,255}, { 7, 63}, { 5,251}, { 8,179}, { 5,191}, { 7,250}, { 6,125}, {10,101}, 
    { 3,255}, { 7,207}, { 5,254}, { 8,229}, { 5,239}, { 8,110}, { 6,231}, {11, 64}, 
    { 3,255}, { 6,246}, { 5,247}, { 8,167}, { 5,127}, { 7,245}, { 6,123}, {10, 53}, 
    { 3,255}, { 7,187}, { 5,253}, { 8,217}, { 5,223}, { 8, 94}, { 6,215}, {11, 10}, 
    { 3,255}, { 7,119}, { 5,251}, { 8,188}, { 5,191}, { 8, 47}, { 6,159}, {10,153}, 
    { 3,255}, { 7,221}, { 5,254}, { 8,236}, { 5,239}, { 8,121}, { 6,235}, {11,140}, 
    { 3,255}, { 6,238}, { 5,247}, { 8,157}, { 5,127}, { 7,243}, { 6,111}, {10, 27}, 
    { 3,255}, { 7,183}, { 5,253}, { 8,213}, { 5,223}, { 8, 91}, { 6,189}, {10,228}, 
    { 3,255}, { 7, 95}, { 5,251}, { 8,185}, { 5,191}, { 7,252}, { 6,125}, {10,120}, 
    { 3,255}, { 7,219}, { 5,254}, { 8,233}, { 5,239}, { 8,117}, { 6,231}, {11, 96}, 
    { 3,255}, { 6,246}, { 5,247}, { 8,173}, { 5,127}, { 7,249}, { 6,123}, {10, 78}, 
    { 3,255}, { 7,190}, { 5,253}, { 8,220}, { 5,223}, { 8,107}, { 6,215}, {11, 32}, 
    { 3,255}, { 7,126}, { 5,251}, { 8,203}, { 5,191}, { 8, 59}, { 6,159}, {10,172}, 
    { 3,255}, { 7,222}, { 5,254}, { 8,242}, { 5,239}, { 8,124}, { 6,235}, {11,200}, 
    { 3,255}, { 6,238}, { 5,247}, { 8,143}, { 5,127}, { 7,237}, { 6,111}, { 8,244}, 
    { 3,255}, { 7,175}, { 5,253}, { 8,205}, { 5,223}, { 8, 61}, { 6,189}, {10,180}, 
    { 3,255}, { 7, 63}, { 5,251}, { 8,174}, { 5,191}, { 7,250}, { 6,125}, {10, 86}, 
    { 3,255}, { 7,207}, { 5,254}, { 8,227}, { 5,239}, { 8,109}, { 6,231}, {11, 38}, 
    { 3,255}, { 6,246}, { 5,247}, { 8,158}, { 5,127}, { 7,245}, { 6,123}, {10, 39}, 
    { 3,255}, { 7,187}, { 5,253}, { 8,214}, { 5,223}, { 8, 93}, { 6,215}, {11,  0}, 
    { 3,255}, { 7,119}, { 5,251}, { 8,186}, { 5,191}, { 8, 31}, { 6,159}, {10,141}, 
    { 3,255}, { 7,221}, { 5,254}, { 8,234}, { 5,239}, { 8,118}, { 6,235}, {11,128}, 
    { 3,255}, { 6,238}, { 5,247}, { 8,155}, { 5,127}, { 7,243}, { 6,111}, { 9, 62}, 
    { 3,255}, { 7,183}, { 5,253}, { 8,211}, { 5,223}, { 8, 87}, { 6,189}, {10,210}, 
    { 3,255}, { 7, 95}, { 5,251}, { 8,181}, { 5,191}, { 7,252}, { 6,125}, {10,106}, 
    { 3,255}, { 7,219}, { 5,254}, { 8,230}, { 5,239}, { 8,115}, { 6,231}, {11, 70}, 
    { 3,255}, { 6,246}, { 5,247}, { 8,171}, { 5,127}, { 7,249}, { 6,123}, {10, 58}, 
    { 3,255}, { 7,190}, { 5,253}, { 8,218}, { 5,223}, { 8,103}, { 6,215}, {11, 17}, 
    { 3,255}, { 7,126}, { 5,251}, { 8,199}, { 5,191}, { 8, 55}, { 6,159}, {10,163}, 
    { 3,255}, { 7,222}, { 5,254}, { 8,241}, { 5,239}, { 8,122}, { 6,235}, {11,160}, 
    { 3,255}, { 6,238}, { 5,247}, { 8,151}, { 5,127}, { 7,237}, { 6,111}, { 8,248}, 
    { 3,255}, { 7,175}, { 5,253}, { 8,206}, { 5,223}, { 8, 79}, { 6,189}, {10,198}, 
    { 3,255}, { 7, 63}, { 5,251}, { 8,179}, { 5,191}, { 7,250}, { 6,125}, {10, 99}, 
    { 3,255}, { 7,207}, { 5,254}, { 8,229}, { 5,239}, { 8,110}, { 6,231}, {11, 52}, 
    { 3,255}, { 6,246}, { 5,247}, { 8,167}, { 5,127}, { 7,245}, { 6,123}, {10, 51}, 
    { 3,255}, { 7,187}, { 5,253}, { 8,217}, { 5,223}, { 8, 94}, { 6,215}, {11,  8}, 
    { 3,255}, { 7,119}, { 5,251}, { 8,188}, { 5,191}, { 8, 47}, { 6,159}, {10,150}, 
    { 3,255}, { 7,221}, { 5,254}, { 8,236}, { 5,239}, { 8,121}, { 6,235}, {11,137}, 
    { 3,255}, { 6,238}, { 5,247}, { 8,157}, { 5,127}, { 7,243}, { 6,111}, {10, 23}, 
    { 3,255}, { 7,183}, { 5,253}, { 8,213}, { 5,223}, { 8, 91}, { 6,189}, {10,226}, 
    { 3,255}, { 7, 95}, { 5,251}, { 8,185}, { 5,191}, { 7,252}, { 6,125}, {10,116}, 
    { 3,255}, { 7,219}, { 5,254}, { 8,233}, { 5,239}, { 8,117}, { 6,231}, {11, 84}, 
    { 3,255}, { 6,246}, { 5,247}, { 8,173}, { 5,127}, { 7,249}, { 6,123}, {10, 77}, 
    { 3,255}, { 7,190}, { 5,253}, { 8,220}, { 5,223}, { 8,107}, { 6,215}, {11, 26}, 
    { 3,255}, { 7,126}, { 5,251}, { 8,203}, { 5,191}, { 8, 59}, { 6,159}, {10,170}, 
    { 3,255}, { 7,222}, { 5,254}, { 8,242}, { 5,239}, { 8,124}, { 6,235}, {11,194}, 
    { 3,255}, { 6,238}, { 5,247}, { 8,143}, { 5,127}, { 7,237}, { 6,111}, { 8,244}, 
    { 3,255}, { 7,175}, { 5,253}, { 8,205}, { 5,223}, { 8, 61}, { 6,189}, {10,195}, 
    { 3,255}, { 7, 63}, { 5,251}, { 8,174}, { 5,191}, { 7,250}, { 6,125}, {10, 90}, 
    { 3,255}, { 7,207}, { 5,254}, { 8,227}, { 5,239}, { 8,109}, { 6,231}, {11, 44}, 
    { 3,255}, { 6,246}, { 5,247}, { 8,158}, { 5,127}, { 7,245}, { 6,123}, {10, 45}, 
    { 3,255}, { 7,187}, { 5,253}, { 8,214}, { 5,223}, { 8, 93}, { 6,215}, {11,  4}, 
    { 3,255}, { 7,119}, { 5,251}, { 8,186}, { 5,191}, { 8, 31}, { 6,159}, {10,147}, 
    { 3,255}, { 7,221}, { 5,254}, { 8,234}, { 5,239}, { 8,118}, { 6,235}, {11,132}, 
    { 3,255}, { 6,238}, { 5,247}, { 8,155}, { 5,127}, { 7,243}, { 6,111}, { 9,182}, 
    { 3,255}, { 7,183}, { 5,253}, { 8,211}, { 5,223}, { 8, 87}, { 6,189}, {10,216}, 
    { 3,255}, { 7, 95}, { 5,251}, { 8,181}, { 5,191}, { 7,252}, { 6,125}, {10,113}, 
    { 3,255}, { 7,219}, { 5,254}, { 8,230}, { 5,239}, { 8,115}, { 6,231}, {11, 76}, 
    { 3,255}, { 6,246}, { 5,247}, { 8,171}, { 5,127}, { 7,249}, { 6,123}, {10, 71}, 
    { 3,255}, { 7,190}, { 5,253}, { 8,218}, { 5,223}, { 8,103}, { 6,215}, {11, 21}, 
    { 3,255}, { 7,126}, { 5,251}, { 8,199}, { 5,191}, { 8, 55}, { 6,159}, {10,166}, 
    { 3,255}, { 7,222}, { 5,254}, { 8,241}, { 5,239}, { 8,122}, { 6,235}, {11,168}, 
    { 3,255}, { 6,238}, { 5,247}, { 8,151}, { 5,127}, { 7,237}, { 6,111}, { 8,248}, 
    { 3,255}, { 7,175}, { 5,253}, { 8,206}, { 5,223}, { 8, 79}, { 6,189}, {10,204}, 
    { 3,255}, { 7, 63}, { 5,251}, { 8,179}, { 5,191}, { 7,250}, { 6,125}, {10,102}, 
    { 3,255}, { 7,207}, { 5,254}, { 8,229}, { 5,239}, { 8,110}, { 6,231}, {11, 66}, 
    { 3,255}, { 6,246}, { 5,247}, { 8,167}, { 5,127}, { 7,245}, { 6,123}, {10, 54}, 
    { 3,255}, { 7,187}, { 5,253}, { 8,217}, { 5,223}, { 8, 94}, { 6,215}, {11, 12}, 
    { 3,255}, { 7,119}, { 5,251}, { 8,188}, { 5,191}, { 8, 47}, { 6,159}, {10,154}, 
    { 3,255}, { 7,221}, { 5,254}, { 8,236}, { 5,239}, { 8,121}, { 6,235}, {11,145}, 
    { 3,255}, { 6,238}, { 5,247}, { 8,157}, { 5,127}, { 7,243}, { 6,111}, {10, 29}, 
    { 3,255}, { 7,183}, { 5,253}, { 8,213}, { 5,223}, { 8, 91}, { 6,189}, {10,232}, 
    { 3,255}, { 7, 95}, { 5,251}, { 8,185}, { 5,191}, { 7,252}, { 6,125}, {10,135}, 
    { 3,255}, { 7,219}, { 5,254}, { 8,233}, { 5,239}, { 8,117}, { 6,231}, {11, 98}, 
    { 3,255}, { 6,246}, { 5,247}, { 8,173}, { 5,127}, { 7,249}, { 6,123}, {10, 83}, 
    { 3,255}, { 7,190}, { 5,253}, { 8,220}, { 5,223}, { 8,107}, { 6,215}, {11, 34}, 
    { 3,255}, { 7,126}, { 5,251}, { 8,203}, { 5,191}, { 8, 59}, { 6,159}, {10,177}, 
    { 3,255}, { 7,222}, { 5,254}, { 8,242}, { 5,239}, { 8,124}, { 6,235}, {11,208}, 
    { 3,255}, { 6,238}, { 5,247}, { 8,143}, { 5,127}, { 7,237}, { 6,111}, { 8,244}, 
    { 3,255}, { 7,175}, { 5,253}, { 8,205}, { 5,223}, { 8, 61}, { 6,189}, {10,178}, 
    { 3,255}, { 7, 63}, { 5,251}, { 8,174}, { 5,191}, { 7,250}, { 6,125}, {10, 85}, 
    { 3,255}, { 7,207}, { 5,254}, { 8,227}, { 5,239}, { 8,109}, { 6,231}, {11, 37}, 
    { 3,255}, { 6,246}, { 5,247}, { 8,158}, { 5,127}, { 7,245}, { 6,123}, {10, 30}, 
    { 3,255}, { 7,187}, { 5,253}, { 8,214}, { 5,223}, { 8, 93}, { 6,215}, {10,240}, 
    { 3,255}, { 7,119}, { 5,251}, { 8,186}, { 5,191}, { 8, 31}, { 6,159}, {10,139}, 
    { 3,255}, { 7,221}, { 5,254}, { 8,234}, { 5,239}, { 8,118}, { 6,235}, {11,112}, 
    { 3,255}, { 6,238}, { 5,247}, { 8,155}, { 5,127}, { 7,243}, { 6,111}, { 9, 62}, 
    { 3,255}, { 7,183}, { 5,253}, { 8,211}, { 5,223}, { 8, 87}, { 6,189}, {10,209}, 
    { 3,255}, { 7, 95}, { 5,251}, { 8,181}, { 5,191}, { 7,252}, { 6,125}, {10,105}, 
    { 3,255}, { 7,219}, { 5,254}, { 8,230}, { 5,239}, { 8,115}, { 6,231}, {11, 69}, 
    { 3,255}, { 6,246}, { 5,247}, { 8,171}, { 5,127}, { 7,249}, { 6,123}, {10, 57}, 
    { 3,255}, { 7,190}, { 5,253}, { 8,218}, { 5,223}, { 8,103}, { 6,215}, {11, 16}, 
    { 3,255}, { 7,126}, { 5,251}, { 8,199}, { 5,191}, { 8, 55}, { 6,159}, {10,156}, 
    { 3,255}, { 7,222}, { 5,254}, { 8,241}, { 5,239}, { 8,122}, { 6,235}, {11,152}, 
    { 3,255}, { 6,238}, { 5,247}, { 8,151}, { 5,127}, { 7,237}, { 6,111}, { 8,248}, 
    { 3,255}, { 7,175}, { 5,253}, { 8,206}, { 5,223}, { 8, 79}, { 6,189}, {10,197}, 
    { 3,255}, { 7, 63}, { 5,251}, { 8,179}, { 5,191}, { 7,250}, { 6,125}, {10, 92}, 
    { 3,255}, { 7,207}, { 5,254}, { 8,229}, { 5,239}, { 8,110}, { 6,231}, {11, 50}, 
    { 3,255}, { 6,246}, { 5,247}, { 8,167}, { 5,127}, { 7,245}, { 6,123}, {10, 46}, 
    { 3,255}, { 7,187}, { 5,253}, { 8,217}, { 5,223}, { 8, 94}, { 6,215}, {11,  7}, 
    { 3,255}, { 7,119}, { 5,251}, { 8,188}, { 5,191}, { 8, 47}, { 6,159}, {10,149}, 
    { 3,255}, { 7,221}, { 5,254}, { 8,236}, { 5,239}, { 8,121}, { 6,235}, {11,136}, 
    { 3,255}, { 6,238}, { 5,247}, { 8,157}, { 5,127}, { 7,243}, { 6,111}, {10, 15}, 
    { 3,255}, { 7,183}, { 5,253}, { 8,213}, { 5,223}, { 8, 91}, { 6,189}, {10,225}, 
    { 3,255}, { 7, 95}, { 5,251}, { 8,185}, { 5,191}, { 7,252}, { 6,125}, {10,114}, 
    { 3,255}, { 7,219}, { 5,254}, { 8,233}, { 5,239}, { 8,117}, { 6,231}, {11, 82}, 
    { 3,255}, { 6,246}, { 5,247}, { 8,173}, { 5,127}, { 7,249}, { 6,123}, {10, 75}, 
    { 3,255}, { 7,190}, { 5,253}, { 8,220}, { 5,223}, { 8,107}, { 6,215}, {11, 25}, 
    { 3,255}, { 7,126}, { 5,251}, { 8,203}, { 5,191}, { 8, 59}, { 6,159}, {10,169}, 
    { 3,255}, { 7,222}, { 5,254}, { 8,242}, { 5,239}, { 8,124}, { 6,235}, {11,193}, 
    { 3,255}, { 6,238}, { 5,247}, { 8,143}, { 5,127}, { 7,237}, { 6,111}, { 8,244}, 
    { 3,255}, { 7,175}, { 5,253}, { 8,205}, { 5,223}, { 8, 61}, { 6,189}, {10,184}, 
    { 3,255}, { 7, 63}, { 5,251}, { 8,174}, { 5,191}, { 7,250}, { 6,125}, {10, 89}, 
    { 3,255}, { 7,207}, { 5,254}, { 8,227}, { 5,239}, { 8,109}, { 6,231}, {11, 42}, 
    { 3,255}, { 6,246}, { 5,247}, { 8,158}, { 5,127}, { 7,245}, { 6,123}, {10, 43}, 
    { 3,255}, { 7,187}, { 5,253}, { 8,214}, { 5,223}, { 8, 93}, { 6,215}, {11,  3}, 
    { 3,255}, { 7,119}, { 5,251}, { 8,186}, { 5,191}, { 8, 31}, { 6,159}, {10,142}, 
    { 3,255}, { 7,221}, { 5,254}, { 8,234}, { 5,239}, { 8,118}, { 6,235}, {11,131}, 
    { 3,255}, { 6,238}, { 5,247}, { 8,155}, { 5,127}, { 7,243}, { 6,111}, { 9,182}, 
    { 3,255}, { 7,183}, { 5,253}, { 8,211}, { 5,223}, { 8, 87}, { 6,189}, {10,212}, 
    { 3,255}, { 7, 95}, { 5,251}, { 8,181}, { 5,191}, { 7,252}, { 6,125}, {10,108}, 
    { 3,255}, { 7,219}, { 5,254}, { 8,230}, { 5,239}, { 8,115}, { 6,231}, {11, 74}, 
    { 3,255}, { 6,246}, { 5,247}, { 8,171}, { 5,127}, { 7,249}, { 6,123}, {10, 60}, 
    { 3,255}, { 7,190}, { 5,253}, { 8,218}, { 5,223}, { 8,103}, { 6,215}, {11, 20}, 
    { 3,255}, { 7,126}, { 5,251}, { 8,199}, { 5,191}, { 8, 55}, { 6,159}, {10,165}, 
    { 3,255}, { 7,222}, { 5,254}, { 8,241}, { 5,239}, { 8,122}, { 6,235}, {11,164}, 
    { 3,255}, { 6,238}, { 5,247}, { 8,151}, { 5,127}, { 7,237}, { 6,111}, { 8,248}, 
    { 3,255}, { 7,175}, { 5,253}, { 8,206}, { 5,223}, { 8, 79}, { 6,189}, {10,202}, 
    { 3,255}, { 7, 63}, { 5,251}, { 8,179}, { 5,191}, { 7,250}, { 6,125}, {10,101}, 
    { 3,255}, { 7,207}, { 5,254}, { 8,229}, { 5,239}, { 8,110}, { 6,231}, {11, 65}, 
    { 3,255}, { 6,246}, { 5,247}, { 8,167}, { 5,127}, { 7,245}, { 6,123}, {10, 53}, 
    { 3,255}, { 7,187}, { 5,253}, { 8,217}, { 5,223}, { 8, 94}, { 6,215}, {11, 11}, 
    { 3,255}, { 7,119}, { 5,251}, { 8,188}, { 5,191}, { 8, 47}, { 6,159}, {10,153}, 
    { 3,255}, { 7,221}, { 5,254}, { 8,236}, { 5,239}, { 8,121}, { 6,235}, {11,144}, 
    { 3,255}, { 6,238}, { 5,247}, { 8,157}, { 5,127}, { 7,243}, { 6,111}, {10, 27}, 
    { 3,255}, { 7,183}, { 5,253}, { 8,213}, { 5,223}, { 8, 91}, { 6,189}, {10,228}, 
    { 3,255}, { 7, 95}, { 5,251}, { 8,185}, { 5,191}, { 7,252}, { 6,125}, {10,120}, 
    { 3,255}, { 7,219}, { 5,254}, { 8,233}, { 5,239}, { 8,117}, { 6,231}, {11, 97}, 
    { 3,255}, { 6,246}, { 5,247}, { 8,173}, { 5,127}, { 7,249}, { 6,123}, {10, 78}, 
    { 3,255}, { 7,190}, { 5,253}, { 8,220}, { 5,223}, { 8,107}, { 6,215}, {11, 33}, 
    { 3,255}, { 7,126}, { 5,251}, { 8,203}, { 5,191}, { 8, 59}, { 6,159}, {10,172}, 
    { 3,255}, { 7,222}, { 5,254}, { 8,242}, { 5,239}, { 8,124}, { 6,235}, {11,201}, 
    { 3,255}, { 6,238}, { 5,247}, { 8,143}, { 5,127}, { 7,237}, { 6,111}, { 8,244}, 
    { 3,255}, { 7,175}, { 5,253}, { 8,205}, { 5,223}, { 8, 61}, { 6,189}, {10,180}, 
    { 3,255}, { 7, 63}, { 5,251}, { 8,174}, { 5,191}, { 7,250}, { 6,125}, {10, 86}, 
    { 3,255}, { 7,207}, { 5,254}, { 8,227}, { 5,239}, { 8,109}, { 6,231}, {11, 40}, 
    { 3,255}, { 6,246}, { 5,247}, { 8,158}, { 5,127}, { 7,245}, { 6,123}, {10, 39}, 
    { 3,255}, { 7,187}, { 5,253}, { 8,214}, { 5,223}, { 8, 93}, { 6,215}, {11,  1}, 
    { 3,255}, { 7,119}, { 5,251}, { 8,186}, { 5,191}, { 8, 31}, { 6,159}, {10,141}, 
    { 3,255}, { 7,221}, { 5,254}, { 8,234}, { 5,239}, { 8,118}, { 6,235}, {11,129}, 
    { 3,255}, { 6,238}, { 5,247}, { 8,155}, { 5,127}, { 7,243}, { 6,111}, { 9, 62}, 
    { 3,255}, { 7,183}, { 5,253}, { 8,211}, { 5,223}, { 8, 87}, { 6,189}, {10,210}, 
    { 3,255}, { 7, 95}, { 5,251}, { 8,181}, { 5,191}, { 7,252}, { 6,125}, {10,106}, 
    { 3,255}, { 7,219}, { 5,254}, { 8,230}, { 5,239}, { 8,115}, { 6,231}, {11, 72}, 
    { 3,255}, { 6,246}, { 5,247}, { 8,171}, { 5,127}, { 7,249}, { 6,123}, {10, 58}, 
    { 3,255}, { 7,190}, { 5,253}, { 8,218}, { 5,223}, { 8,103}, { 6,215}, {11, 18}, 
    { 3,255}, { 7,126}, { 5,251}, { 8,199}, { 5,191}, { 8, 55}, { 6,159}, {10,163}, 
    { 3,255}, { 7,222}, { 5,254}, { 8,241}, { 5,239}, { 8,122}, { 6,235}, {11,161}, 
    { 3,255}, { 6,238}, { 5,247}, { 8,151}, { 5,127}, { 7,237}, { 6,111}, { 8,248}, 
    { 3,255}, { 7,175}, { 5,253}, { 8,206}, { 5,223}, { 8, 79}, { 6,189}, {10,198}, 
    { 3,255}, { 7, 63}, { 5,251}, { 8,179}, { 5,191}, { 7,250}, { 6,125}, {10, 99}, 
    { 3,255}, { 7,207}, { 5,254}, { 8,229}, { 5,239}, { 8,110}, { 6,231}, {11, 56}, 
    { 3,255}, { 6,246}, { 5,247}, { 8,167}, { 5,127}, { 7,245}, { 6,123}, {10, 51}, 
    { 3,255}, { 7,187}, { 5,253}, { 8,217}, { 5,223}, { 8, 94}, { 6,215}, {11,  9}, 
    { 3,255}, { 7,119}, { 5,251}, { 8,188}, { 5,191}, { 8, 47}, { 6,159}, {10,150}, 
    { 3,255}, { 7,221}, { 5,254}, { 8,236}, { 5,239}, { 8,121}, { 6,235}, {11,138}, 
    { 3,255}, { 6,238}, { 5,247}, { 8,157}, { 5,127}, { 7,243}, { 6,111}, {10, 23}, 
    { 3,255}, { 7,183}, { 5,253}, { 8,213}, { 5,223}, { 8, 91}, { 6,189}, {10,226}, 
    { 3,255}, { 7, 95}, { 5,251}, { 8,185}, { 5,191}, { 7,252}, { 6,125}, {10,116}, 
    { 3,255}, { 7,219}, { 5,254}, { 8,233}, { 5,239}, { 8,117}, { 6,231}, {11, 88}, 
    { 3,255}, { 6,246}, { 5,247}, { 8,173}, { 5,127}, { 7,249}, { 6,123}, {10, 77}, 
    { 3,255}, { 7,190}, { 5,253}, { 8,220}, { 5,223}, { 8,107}, { 6,215}, {11, 28}, 
    { 3,255}, { 7,126}, { 5,251}, { 8,203}, { 5,191}, { 8, 59}, { 6,159}, {10,170}, 
    { 3,255}, { 7,222}, { 5,254}, { 8,242}, { 5,239}, { 8,124}, { 6,235}, {11,196}, 
    { 3,255}, { 6,238}, { 5,247}, { 8,143}, { 5,127}, { 7,237}, { 6,111}, { 8,244}, 
    { 3,255}, { 7,175}, { 5,253}, { 8,205}, { 5,223}, { 8, 61}, { 6,189}, {10,195}, 
    { 3,255}, { 7, 63}, { 5,251}, { 8,174}, { 5,191}, { 7,250}, { 6,125}, {10, 90}, 
    { 3,255}, { 7,207}, { 5,254}, { 8,227}, { 5,239}, { 8,109}, { 6,231}, {11, 48}, 
    { 3,255}, { 6,246}, { 5,247}, { 8,158}, { 5,127}, { 7,245}, { 6,123}, {10, 45}, 
    { 3,255}, { 7,187}, { 5,253}, { 8,214}, { 5,223}, { 8, 93}, { 6,215}, {11,  5}, 
    { 3,255}, { 7,119}, { 5,251}, { 8,186}, { 5,191}, { 8, 31}, { 6,159}, {10,147}, 
    { 3,255}, { 7,221}, { 5,254}, { 8,234}, { 5,239}, { 8,118}, { 6,235}, {11,133}, 
    { 3,255}, { 6,238}, { 5,247}, { 8,155}, { 5,127}, { 7,243}, { 6,111}, { 9,182}, 
    { 3,255}, { 7,183}, { 5,253}, { 8,211}, { 5,223}, { 8, 87}, { 6,189}, {10,216}, 
    { 3,255}, { 7, 95}, { 5,251}, { 8,181}, { 5,191}, { 7,252}, { 6,125}, {10,113}, 
    { 3,255}, { 7,219}, { 5,254}, { 8,230}, { 5,239}, { 8,115}, { 6,231}, {11, 80}, 
    { 3,255}, { 6,246}, { 5,247}, { 8,171}, { 5,127}, { 7,249}, { 6,123}, {10, 71}, 
    { 3,255}, { 7,190}, { 5,253}, { 8,218}, { 5,223}, { 8,103}, { 6,215}, {11, 22}, 
    { 3,255}, { 7,126}, { 5,251}, { 8,199}, { 5,191}, { 8, 55}, { 6,159}, {10,166}, 
    { 3,255}, { 7,222}, { 5,254}, { 8,241}, { 5,239}, { 8,122}, { 6,235}, {11,176}, 
    { 3,255}, { 6,238}, { 5,247}, { 8,151}, { 5,127}, { 7,237}, { 6,111}, { 8,248}, 
    { 3,255}, { 7,175}, { 5,253}, { 8,206}, { 5,223}, { 8, 79}, { 6,189}, {10,204}, 
    { 3,255}, { 7, 63}, { 5,251}, { 8,179}, { 5,191}, { 7,250}, { 6,125}, {10,102}, 
    { 3,255}, { 7,207}, { 5,254}, { 8,229}, { 5,239}, { 8,110}, { 6,231}, {11, 67}, 
    { 3,255}, { 6,246}, { 5,247}, { 8,167}, { 5,127}, { 7,245}, { 6,123}, {10, 54}, 
    { 3,255}, { 7,187}, { 5,253}, { 8,217}, { 5,223}, { 8, 94}, { 6,215}, {11, 13}, 
    { 3,255}, { 7,119}, { 5,251}, { 8,188}, { 5,191}, { 8, 47}, { 6,159}, {10,154}, 
    { 3,255}, { 7,221}, { 5,254}, { 8,236}, { 5,239}, { 8,121}, { 6,235}, {11,146}, 
    { 3,255}, { 6,238}, { 5,247}, { 8,157}, { 5,127}, { 7,243}, { 6,111}, {10, 29}, 
    { 3,255}, { 7,183}, { 5,253}, { 8,213}, { 5,223}, { 8, 91}, { 6,189}, {10,232}, 
    { 3,255}, { 7, 95}, { 5,251}, { 8,185}, { 5,191}, { 7,252}, { 6,125}, {10,135}, 
    { 3,255}, { 7,219}, { 5,254}, { 8,233}, { 5,239}, { 8,117}, { 6,231}, {11,100}, 
    { 3,255}, { 6,246}, { 5,247}, { 8,173}, { 5,127}, { 7,249}, { 6,123}, {10, 83}, 
    { 3,255}, { 7,190}, { 5,253}, { 8,220}, { 5,223}, { 8,107}, { 6,215}, {11, 35}, 
    { 3,255}, { 7,126}, { 5,251}, { 8,203}, { 5,191}, { 8, 59}, { 6,159}, {10,177}, 
    { 3,255}, { 7,222}, { 5,254}, { 8,242}, { 5,239}, { 8,124}, { 6,235}, {11,224}, 
  }
  #endif
};

static radaudio_huffman rada_nonzero_bitflags_huff1 =
{
  #ifdef HUFFMAN_ENCODE
  {
    {   0, 2}, {  42, 7}, { 106, 7}, {  54, 8}, {   2, 6}, { 182, 8}, { 118, 8}, { 101, 9}, 
    {  34, 6}, { 246, 8}, {  14, 8}, { 357, 9}, { 142, 8}, { 229, 9}, { 485, 9}, {  21, 9}, 
    {  26, 7}, {  78, 8}, { 206, 8}, { 277, 9}, {  46, 8}, { 149, 9}, { 405, 9}, {  85, 9}, 
    { 174, 8}, { 341, 9}, { 213, 9}, { 469, 9}, {  53, 9}, { 309, 9}, { 181, 9}, { 437, 9}, 
    {  90, 7}, { 110, 8}, { 238, 8}, { 117, 9}, {  30, 8}, { 373, 9}, { 245, 9}, { 501, 9}, 
    { 158, 8}, {  13, 9}, { 269, 9}, { 141, 9}, { 397, 9}, {  77, 9}, { 333, 9}, { 205, 9}, 
    {  94, 8}, { 461, 9}, {  45, 9}, { 301, 9}, { 173, 9}, { 429, 9}, { 109, 9}, { 365, 9}, 
    { 237, 9}, { 493, 9}, {  29, 9}, { 285, 9}, { 157, 9}, { 413, 9}, {  93, 9}, { 222, 8}, 
    {  18, 6}, {  62, 8}, { 190, 8}, { 349, 9}, { 126, 8}, { 221, 9}, { 477, 9}, {  61, 9}, 
    { 254, 8}, { 317, 9}, { 189, 9}, { 445, 9}, { 125, 9}, { 381, 9}, { 253, 9}, { 509, 9}, 
    {   1, 8}, {   3, 9}, { 259, 9}, { 131, 9}, { 387, 9}, {  67, 9}, { 323, 9}, { 195, 9}, 
    { 451, 9}, {  35, 9}, { 291, 9}, { 163, 9}, { 419, 9}, {  99, 9}, { 355, 9}, { 129, 8}, 
    {  65, 8}, { 227, 9}, { 483, 9}, {  19, 9}, { 275, 9}, { 147, 9}, { 403, 9}, {  83, 9}, 
    { 339, 9}, { 211, 9}, { 467, 9}, {  51, 9}, { 307, 9}, { 179, 9}, { 435, 9}, { 193, 8}, 
    { 115, 9}, { 371, 9}, { 243, 9}, { 499, 9}, {  11, 9}, { 267, 9}, { 139, 9}, {  33, 8}, 
    { 395, 9}, {  75, 9}, { 331, 9}, { 161, 8}, { 203, 9}, {  97, 8}, { 225, 8}, {  58, 7}, 
    {  50, 6}, {  17, 8}, { 145, 8}, { 459, 9}, {  81, 8}, {  43, 9}, { 299, 9}, { 171, 9}, 
    { 209, 8}, { 427, 9}, { 107, 9}, { 363, 9}, { 235, 9}, { 491, 9}, {  27, 9}, { 283, 9}, 
    {  49, 8}, { 155, 9}, { 411, 9}, {  91, 9}, { 347, 9}, { 219, 9}, { 475, 9}, {  59, 9}, 
    { 315, 9}, { 187, 9}, { 443, 9}, { 123, 9}, { 379, 9}, { 251, 9}, { 507, 9}, { 177, 8}, 
    { 113, 8}, {   7, 9}, { 263, 9}, { 135, 9}, { 391, 9}, {  71, 9}, { 327, 9}, { 199, 9}, 
    { 455, 9}, {  39, 9}, { 295, 9}, { 167, 9}, { 423, 9}, { 103, 9}, { 359, 9}, { 241, 8}, 
    { 231, 9}, { 487, 9}, {  23, 9}, { 279, 9}, { 151, 9}, { 407, 9}, {  87, 9}, {   9, 8}, 
    { 343, 9}, { 215, 9}, { 471, 9}, { 137, 8}, {  55, 9}, {  73, 8}, { 201, 8}, { 122, 7}, 
    {  41, 8}, { 311, 9}, { 183, 9}, { 439, 9}, { 119, 9}, { 375, 9}, { 247, 9}, { 503, 9}, 
    {  15, 9}, { 271, 9}, { 143, 9}, { 399, 9}, {  79, 9}, { 335, 9}, { 207, 9}, { 169, 8}, 
    { 463, 9}, {  47, 9}, { 303, 9}, { 175, 9}, { 431, 9}, { 111, 9}, { 367, 9}, { 105, 8}, 
    { 239, 9}, { 495, 9}, {  31, 9}, { 233, 8}, { 287, 9}, {  25, 8}, { 153, 8}, {   6, 7}, 
    { 159, 9}, { 415, 9}, {  95, 9}, { 351, 9}, { 223, 9}, { 479, 9}, {  63, 9}, {  89, 8}, 
    { 319, 9}, { 217, 8}, {  57, 8}, { 185, 8}, { 191, 9}, { 121, 8}, { 249, 8}, {  70, 7}, 
    { 447, 9}, { 127, 9}, { 383, 9}, {   5, 8}, { 255, 9}, { 133, 8}, {  69, 8}, {  38, 7}, 
    { 511, 9}, { 197, 8}, {  37, 8}, { 102, 7}, { 165, 8}, {  22, 7}, {  86, 7}, {  10, 6}, 
  },
  #endif
  #ifdef HUFFMAN_DECODE
  {
    { 2,  0}, { 8, 80}, { 6,  4}, { 9, 81}, { 2,  0}, { 8,243}, { 7,223}, { 9,161}, 
    { 2,  0}, { 8,183}, { 6,255}, { 9,116}, { 2,  0}, { 9, 41}, { 8, 10}, { 9,200}, 
    { 2,  0}, { 8,129}, { 6, 64}, { 9, 99}, { 2,  0}, { 9, 15}, { 7,253}, { 9,178}, 
    { 2,  0}, { 8,221}, { 7, 16}, { 9,142}, { 2,  0}, { 9, 58}, { 8, 36}, { 9,218}, 
    { 2,  0}, { 8,119}, { 6,  8}, { 9, 89}, { 2,  0}, { 8,250}, { 7,247}, { 9,169}, 
    { 2,  0}, { 8,192}, { 7,  1}, { 9,133}, { 2,  0}, { 9, 50}, { 8, 20}, { 9,209}, 
    { 2,  0}, { 8,144}, { 6,128}, { 9,107}, { 2,  0}, { 9, 28}, { 8,  3}, { 9,188}, 
    { 2,  0}, { 8,234}, { 7,127}, { 9,151}, { 2,  0}, { 9, 71}, { 8, 65}, { 9,230}, 
    { 2,  0}, { 8, 96}, { 6,  4}, { 9, 85}, { 2,  0}, { 8,246}, { 7,239}, { 9,165}, 
    { 2,  0}, { 8,189}, { 6,255}, { 9,121}, { 2,  0}, { 9, 45}, { 8, 17}, { 9,204}, 
    { 2,  0}, { 8,132}, { 6, 64}, { 9,103}, { 2,  0}, { 9, 23}, { 7,254}, { 9,182}, 
    { 2,  0}, { 8,231}, { 7, 32}, { 9,147}, { 2,  0}, { 9, 62}, { 8, 48}, { 9,226}, 
    { 2,  0}, { 8,125}, { 6,  8}, { 9, 93}, { 2,  0}, { 9,  7}, { 7,251}, { 9,173}, 
    { 2,  0}, { 8,215}, { 7,  2}, { 9,138}, { 2,  0}, { 9, 54}, { 8, 33}, { 9,213}, 
    { 2,  0}, { 8,160}, { 6,128}, { 9,112}, { 2,  0}, { 9, 35}, { 8,  6}, { 9,196}, 
    { 2,  0}, { 8,237}, { 7,191}, { 9,155}, { 2,  0}, { 9, 76}, { 8, 68}, { 9,241}, 
    { 2,  0}, { 8, 95}, { 6,  4}, { 9, 83}, { 2,  0}, { 8,245}, { 7,223}, { 9,163}, 
    { 2,  0}, { 8,187}, { 6,255}, { 9,118}, { 2,  0}, { 9, 43}, { 8, 12}, { 9,202}, 
    { 2,  0}, { 8,130}, { 6, 64}, { 9,101}, { 2,  0}, { 9, 21}, { 7,253}, { 9,180}, 
    { 2,  0}, { 8,222}, { 7, 16}, { 9,145}, { 2,  0}, { 9, 60}, { 8, 40}, { 9,224}, 
    { 2,  0}, { 8,123}, { 6,  8}, { 9, 91}, { 2,  0}, { 8,252}, { 7,247}, { 9,171}, 
    { 2,  0}, { 8,207}, { 7,  1}, { 9,135}, { 2,  0}, { 9, 52}, { 8, 24}, { 9,211}, 
    { 2,  0}, { 8,159}, { 6,128}, { 9,109}, { 2,  0}, { 9, 30}, { 8,  5}, { 9,194}, 
    { 2,  0}, { 8,235}, { 7,127}, { 9,153}, { 2,  0}, { 9, 74}, { 8, 66}, { 9,236}, 
    { 2,  0}, { 8,111}, { 6,  4}, { 9, 87}, { 2,  0}, { 8,249}, { 7,239}, { 9,167}, 
    { 2,  0}, { 8,190}, { 6,255}, { 9,124}, { 2,  0}, { 9, 47}, { 8, 18}, { 9,206}, 
    { 2,  0}, { 8,136}, { 6, 64}, { 9,105}, { 2,  0}, { 9, 26}, { 7,254}, { 9,185}, 
    { 2,  0}, { 8,233}, { 7, 32}, { 9,149}, { 2,  0}, { 9, 69}, { 8, 63}, { 9,228}, 
    { 2,  0}, { 8,126}, { 6,  8}, { 9, 97}, { 2,  0}, { 9, 13}, { 7,251}, { 9,176}, 
    { 2,  0}, { 8,219}, { 7,  2}, { 9,140}, { 2,  0}, { 9, 56}, { 8, 34}, { 9,216}, 
    { 2,  0}, { 8,175}, { 6,128}, { 9,114}, { 2,  0}, { 9, 38}, { 8,  9}, { 9,198}, 
    { 2,  0}, { 8,238}, { 7,191}, { 9,157}, { 2,  0}, { 9, 78}, { 8, 72}, { 9,244}, 
    { 2,  0}, { 8, 80}, { 6,  4}, { 9, 82}, { 2,  0}, { 8,243}, { 7,223}, { 9,162}, 
    { 2,  0}, { 8,183}, { 6,255}, { 9,117}, { 2,  0}, { 9, 42}, { 8, 10}, { 9,201}, 
    { 2,  0}, { 8,129}, { 6, 64}, { 9,100}, { 2,  0}, { 9, 19}, { 7,253}, { 9,179}, 
    { 2,  0}, { 8,221}, { 7, 16}, { 9,143}, { 2,  0}, { 9, 59}, { 8, 36}, { 9,220}, 
    { 2,  0}, { 8,119}, { 6,  8}, { 9, 90}, { 2,  0}, { 8,250}, { 7,247}, { 9,170}, 
    { 2,  0}, { 8,192}, { 7,  1}, { 9,134}, { 2,  0}, { 9, 51}, { 8, 20}, { 9,210}, 
    { 2,  0}, { 8,144}, { 6,128}, { 9,108}, { 2,  0}, { 9, 29}, { 8,  3}, { 9,193}, 
    { 2,  0}, { 8,234}, { 7,127}, { 9,152}, { 2,  0}, { 9, 73}, { 8, 65}, { 9,232}, 
    { 2,  0}, { 8, 96}, { 6,  4}, { 9, 86}, { 2,  0}, { 8,246}, { 7,239}, { 9,166}, 
    { 2,  0}, { 8,189}, { 6,255}, { 9,122}, { 2,  0}, { 9, 46}, { 8, 17}, { 9,205}, 
    { 2,  0}, { 8,132}, { 6, 64}, { 9,104}, { 2,  0}, { 9, 25}, { 7,254}, { 9,184}, 
    { 2,  0}, { 8,231}, { 7, 32}, { 9,148}, { 2,  0}, { 9, 67}, { 8, 48}, { 9,227}, 
    { 2,  0}, { 8,125}, { 6,  8}, { 9, 94}, { 2,  0}, { 9, 11}, { 7,251}, { 9,174}, 
    { 2,  0}, { 8,215}, { 7,  2}, { 9,139}, { 2,  0}, { 9, 55}, { 8, 33}, { 9,214}, 
    { 2,  0}, { 8,160}, { 6,128}, { 9,113}, { 2,  0}, { 9, 37}, { 8,  6}, { 9,197}, 
    { 2,  0}, { 8,237}, { 7,191}, { 9,156}, { 2,  0}, { 9, 77}, { 8, 68}, { 9,242}, 
    { 2,  0}, { 8, 95}, { 6,  4}, { 9, 84}, { 2,  0}, { 8,245}, { 7,223}, { 9,164}, 
    { 2,  0}, { 8,187}, { 6,255}, { 9,120}, { 2,  0}, { 9, 44}, { 8, 12}, { 9,203}, 
    { 2,  0}, { 8,130}, { 6, 64}, { 9,102}, { 2,  0}, { 9, 22}, { 7,253}, { 9,181}, 
    { 2,  0}, { 8,222}, { 7, 16}, { 9,146}, { 2,  0}, { 9, 61}, { 8, 40}, { 9,225}, 
    { 2,  0}, { 8,123}, { 6,  8}, { 9, 92}, { 2,  0}, { 8,252}, { 7,247}, { 9,172}, 
    { 2,  0}, { 8,207}, { 7,  1}, { 9,137}, { 2,  0}, { 9, 53}, { 8, 24}, { 9,212}, 
    { 2,  0}, { 8,159}, { 6,128}, { 9,110}, { 2,  0}, { 9, 31}, { 8,  5}, { 9,195}, 
    { 2,  0}, { 8,235}, { 7,127}, { 9,154}, { 2,  0}, { 9, 75}, { 8, 66}, { 9,240}, 
    { 2,  0}, { 8,111}, { 6,  4}, { 9, 88}, { 2,  0}, { 8,249}, { 7,239}, { 9,168}, 
    { 2,  0}, { 8,190}, { 6,255}, { 9,131}, { 2,  0}, { 9, 49}, { 8, 18}, { 9,208}, 
    { 2,  0}, { 8,136}, { 6, 64}, { 9,106}, { 2,  0}, { 9, 27}, { 7,254}, { 9,186}, 
    { 2,  0}, { 8,233}, { 7, 32}, { 9,150}, { 2,  0}, { 9, 70}, { 8, 63}, { 9,229}, 
    { 2,  0}, { 8,126}, { 6,  8}, { 9, 98}, { 2,  0}, { 9, 14}, { 7,251}, { 9,177}, 
    { 2,  0}, { 8,219}, { 7,  2}, { 9,141}, { 2,  0}, { 9, 57}, { 8, 34}, { 9,217}, 
    { 2,  0}, { 8,175}, { 6,128}, { 9,115}, { 2,  0}, { 9, 39}, { 8,  9}, { 9,199}, 
    { 2,  0}, { 8,238}, { 7,191}, { 9,158}, { 2,  0}, { 9, 79}, { 8, 72}, { 9,248}, 
    { 2,  0}, { 8, 80}, { 6,  4}, { 9, 81}, { 2,  0}, { 8,243}, { 7,223}, { 9,161}, 
    { 2,  0}, { 8,183}, { 6,255}, { 9,116}, { 2,  0}, { 9, 41}, { 8, 10}, { 9,200}, 
    { 2,  0}, { 8,129}, { 6, 64}, { 9, 99}, { 2,  0}, { 9, 15}, { 7,253}, { 9,178}, 
    { 2,  0}, { 8,221}, { 7, 16}, { 9,142}, { 2,  0}, { 9, 58}, { 8, 36}, { 9,218}, 
    { 2,  0}, { 8,119}, { 6,  8}, { 9, 89}, { 2,  0}, { 8,250}, { 7,247}, { 9,169}, 
    { 2,  0}, { 8,192}, { 7,  1}, { 9,133}, { 2,  0}, { 9, 50}, { 8, 20}, { 9,209}, 
    { 2,  0}, { 8,144}, { 6,128}, { 9,107}, { 2,  0}, { 9, 28}, { 8,  3}, { 9,188}, 
    { 2,  0}, { 8,234}, { 7,127}, { 9,151}, { 2,  0}, { 9, 71}, { 8, 65}, { 9,230}, 
    { 2,  0}, { 8, 96}, { 6,  4}, { 9, 85}, { 2,  0}, { 8,246}, { 7,239}, { 9,165}, 
    { 2,  0}, { 8,189}, { 6,255}, { 9,121}, { 2,  0}, { 9, 45}, { 8, 17}, { 9,204}, 
    { 2,  0}, { 8,132}, { 6, 64}, { 9,103}, { 2,  0}, { 9, 23}, { 7,254}, { 9,182}, 
    { 2,  0}, { 8,231}, { 7, 32}, { 9,147}, { 2,  0}, { 9, 62}, { 8, 48}, { 9,226}, 
    { 2,  0}, { 8,125}, { 6,  8}, { 9, 93}, { 2,  0}, { 9,  7}, { 7,251}, { 9,173}, 
    { 2,  0}, { 8,215}, { 7,  2}, { 9,138}, { 2,  0}, { 9, 54}, { 8, 33}, { 9,213}, 
    { 2,  0}, { 8,160}, { 6,128}, { 9,112}, { 2,  0}, { 9, 35}, { 8,  6}, { 9,196}, 
    { 2,  0}, { 8,237}, { 7,191}, { 9,155}, { 2,  0}, { 9, 76}, { 8, 68}, { 9,241}, 
    { 2,  0}, { 8, 95}, { 6,  4}, { 9, 83}, { 2,  0}, { 8,245}, { 7,223}, { 9,163}, 
    { 2,  0}, { 8,187}, { 6,255}, { 9,118}, { 2,  0}, { 9, 43}, { 8, 12}, { 9,202}, 
    { 2,  0}, { 8,130}, { 6, 64}, { 9,101}, { 2,  0}, { 9, 21}, { 7,253}, { 9,180}, 
    { 2,  0}, { 8,222}, { 7, 16}, { 9,145}, { 2,  0}, { 9, 60}, { 8, 40}, { 9,224}, 
    { 2,  0}, { 8,123}, { 6,  8}, { 9, 91}, { 2,  0}, { 8,252}, { 7,247}, { 9,171}, 
    { 2,  0}, { 8,207}, { 7,  1}, { 9,135}, { 2,  0}, { 9, 52}, { 8, 24}, { 9,211}, 
    { 2,  0}, { 8,159}, { 6,128}, { 9,109}, { 2,  0}, { 9, 30}, { 8,  5}, { 9,194}, 
    { 2,  0}, { 8,235}, { 7,127}, { 9,153}, { 2,  0}, { 9, 74}, { 8, 66}, { 9,236}, 
    { 2,  0}, { 8,111}, { 6,  4}, { 9, 87}, { 2,  0}, { 8,249}, { 7,239}, { 9,167}, 
    { 2,  0}, { 8,190}, { 6,255}, { 9,124}, { 2,  0}, { 9, 47}, { 8, 18}, { 9,206}, 
    { 2,  0}, { 8,136}, { 6, 64}, { 9,105}, { 2,  0}, { 9, 26}, { 7,254}, { 9,185}, 
    { 2,  0}, { 8,233}, { 7, 32}, { 9,149}, { 2,  0}, { 9, 69}, { 8, 63}, { 9,228}, 
    { 2,  0}, { 8,126}, { 6,  8}, { 9, 97}, { 2,  0}, { 9, 13}, { 7,251}, { 9,176}, 
    { 2,  0}, { 8,219}, { 7,  2}, { 9,140}, { 2,  0}, { 9, 56}, { 8, 34}, { 9,216}, 
    { 2,  0}, { 8,175}, { 6,128}, { 9,114}, { 2,  0}, { 9, 38}, { 8,  9}, { 9,198}, 
    { 2,  0}, { 8,238}, { 7,191}, { 9,157}, { 2,  0}, { 9, 78}, { 8, 72}, { 9,244}, 
    { 2,  0}, { 8, 80}, { 6,  4}, { 9, 82}, { 2,  0}, { 8,243}, { 7,223}, { 9,162}, 
    { 2,  0}, { 8,183}, { 6,255}, { 9,117}, { 2,  0}, { 9, 42}, { 8, 10}, { 9,201}, 
    { 2,  0}, { 8,129}, { 6, 64}, { 9,100}, { 2,  0}, { 9, 19}, { 7,253}, { 9,179}, 
    { 2,  0}, { 8,221}, { 7, 16}, { 9,143}, { 2,  0}, { 9, 59}, { 8, 36}, { 9,220}, 
    { 2,  0}, { 8,119}, { 6,  8}, { 9, 90}, { 2,  0}, { 8,250}, { 7,247}, { 9,170}, 
    { 2,  0}, { 8,192}, { 7,  1}, { 9,134}, { 2,  0}, { 9, 51}, { 8, 20}, { 9,210}, 
    { 2,  0}, { 8,144}, { 6,128}, { 9,108}, { 2,  0}, { 9, 29}, { 8,  3}, { 9,193}, 
    { 2,  0}, { 8,234}, { 7,127}, { 9,152}, { 2,  0}, { 9, 73}, { 8, 65}, { 9,232}, 
    { 2,  0}, { 8, 96}, { 6,  4}, { 9, 86}, { 2,  0}, { 8,246}, { 7,239}, { 9,166}, 
    { 2,  0}, { 8,189}, { 6,255}, { 9,122}, { 2,  0}, { 9, 46}, { 8, 17}, { 9,205}, 
    { 2,  0}, { 8,132}, { 6, 64}, { 9,104}, { 2,  0}, { 9, 25}, { 7,254}, { 9,184}, 
    { 2,  0}, { 8,231}, { 7, 32}, { 9,148}, { 2,  0}, { 9, 67}, { 8, 48}, { 9,227}, 
    { 2,  0}, { 8,125}, { 6,  8}, { 9, 94}, { 2,  0}, { 9, 11}, { 7,251}, { 9,174}, 
    { 2,  0}, { 8,215}, { 7,  2}, { 9,139}, { 2,  0}, { 9, 55}, { 8, 33}, { 9,214}, 
    { 2,  0}, { 8,160}, { 6,128}, { 9,113}, { 2,  0}, { 9, 37}, { 8,  6}, { 9,197}, 
    { 2,  0}, { 8,237}, { 7,191}, { 9,156}, { 2,  0}, { 9, 77}, { 8, 68}, { 9,242}, 
    { 2,  0}, { 8, 95}, { 6,  4}, { 9, 84}, { 2,  0}, { 8,245}, { 7,223}, { 9,164}, 
    { 2,  0}, { 8,187}, { 6,255}, { 9,120}, { 2,  0}, { 9, 44}, { 8, 12}, { 9,203}, 
    { 2,  0}, { 8,130}, { 6, 64}, { 9,102}, { 2,  0}, { 9, 22}, { 7,253}, { 9,181}, 
    { 2,  0}, { 8,222}, { 7, 16}, { 9,146}, { 2,  0}, { 9, 61}, { 8, 40}, { 9,225}, 
    { 2,  0}, { 8,123}, { 6,  8}, { 9, 92}, { 2,  0}, { 8,252}, { 7,247}, { 9,172}, 
    { 2,  0}, { 8,207}, { 7,  1}, { 9,137}, { 2,  0}, { 9, 53}, { 8, 24}, { 9,212}, 
    { 2,  0}, { 8,159}, { 6,128}, { 9,110}, { 2,  0}, { 9, 31}, { 8,  5}, { 9,195}, 
    { 2,  0}, { 8,235}, { 7,127}, { 9,154}, { 2,  0}, { 9, 75}, { 8, 66}, { 9,240}, 
    { 2,  0}, { 8,111}, { 6,  4}, { 9, 88}, { 2,  0}, { 8,249}, { 7,239}, { 9,168}, 
    { 2,  0}, { 8,190}, { 6,255}, { 9,131}, { 2,  0}, { 9, 49}, { 8, 18}, { 9,208}, 
    { 2,  0}, { 8,136}, { 6, 64}, { 9,106}, { 2,  0}, { 9, 27}, { 7,254}, { 9,186}, 
    { 2,  0}, { 8,233}, { 7, 32}, { 9,150}, { 2,  0}, { 9, 70}, { 8, 63}, { 9,229}, 
    { 2,  0}, { 8,126}, { 6,  8}, { 9, 98}, { 2,  0}, { 9, 14}, { 7,251}, { 9,177}, 
    { 2,  0}, { 8,219}, { 7,  2}, { 9,141}, { 2,  0}, { 9, 57}, { 8, 34}, { 9,217}, 
    { 2,  0}, { 8,175}, { 6,128}, { 9,115}, { 2,  0}, { 9, 39}, { 8,  9}, { 9,199}, 
    { 2,  0}, { 8,238}, { 7,191}, { 9,158}, { 2,  0}, { 9, 79}, { 8, 72}, { 9,248}, 
    { 2,  0}, { 8, 80}, { 6,  4}, { 9, 81}, { 2,  0}, { 8,243}, { 7,223}, { 9,161}, 
    { 2,  0}, { 8,183}, { 6,255}, { 9,116}, { 2,  0}, { 9, 41}, { 8, 10}, { 9,200}, 
    { 2,  0}, { 8,129}, { 6, 64}, { 9, 99}, { 2,  0}, { 9, 15}, { 7,253}, { 9,178}, 
    { 2,  0}, { 8,221}, { 7, 16}, { 9,142}, { 2,  0}, { 9, 58}, { 8, 36}, { 9,218}, 
    { 2,  0}, { 8,119}, { 6,  8}, { 9, 89}, { 2,  0}, { 8,250}, { 7,247}, { 9,169}, 
    { 2,  0}, { 8,192}, { 7,  1}, { 9,133}, { 2,  0}, { 9, 50}, { 8, 20}, { 9,209}, 
    { 2,  0}, { 8,144}, { 6,128}, { 9,107}, { 2,  0}, { 9, 28}, { 8,  3}, { 9,188}, 
    { 2,  0}, { 8,234}, { 7,127}, { 9,151}, { 2,  0}, { 9, 71}, { 8, 65}, { 9,230}, 
    { 2,  0}, { 8, 96}, { 6,  4}, { 9, 85}, { 2,  0}, { 8,246}, { 7,239}, { 9,165}, 
    { 2,  0}, { 8,189}, { 6,255}, { 9,121}, { 2,  0}, { 9, 45}, { 8, 17}, { 9,204}, 
    { 2,  0}, { 8,132}, { 6, 64}, { 9,103}, { 2,  0}, { 9, 23}, { 7,254}, { 9,182}, 
    { 2,  0}, { 8,231}, { 7, 32}, { 9,147}, { 2,  0}, { 9, 62}, { 8, 48}, { 9,226}, 
    { 2,  0}, { 8,125}, { 6,  8}, { 9, 93}, { 2,  0}, { 9,  7}, { 7,251}, { 9,173}, 
    { 2,  0}, { 8,215}, { 7,  2}, { 9,138}, { 2,  0}, { 9, 54}, { 8, 33}, { 9,213}, 
    { 2,  0}, { 8,160}, { 6,128}, { 9,112}, { 2,  0}, { 9, 35}, { 8,  6}, { 9,196}, 
    { 2,  0}, { 8,237}, { 7,191}, { 9,155}, { 2,  0}, { 9, 76}, { 8, 68}, { 9,241}, 
    { 2,  0}, { 8, 95}, { 6,  4}, { 9, 83}, { 2,  0}, { 8,245}, { 7,223}, { 9,163}, 
    { 2,  0}, { 8,187}, { 6,255}, { 9,118}, { 2,  0}, { 9, 43}, { 8, 12}, { 9,202}, 
    { 2,  0}, { 8,130}, { 6, 64}, { 9,101}, { 2,  0}, { 9, 21}, { 7,253}, { 9,180}, 
    { 2,  0}, { 8,222}, { 7, 16}, { 9,145}, { 2,  0}, { 9, 60}, { 8, 40}, { 9,224}, 
    { 2,  0}, { 8,123}, { 6,  8}, { 9, 91}, { 2,  0}, { 8,252}, { 7,247}, { 9,171}, 
    { 2,  0}, { 8,207}, { 7,  1}, { 9,135}, { 2,  0}, { 9, 52}, { 8, 24}, { 9,211}, 
    { 2,  0}, { 8,159}, { 6,128}, { 9,109}, { 2,  0}, { 9, 30}, { 8,  5}, { 9,194}, 
    { 2,  0}, { 8,235}, { 7,127}, { 9,153}, { 2,  0}, { 9, 74}, { 8, 66}, { 9,236}, 
    { 2,  0}, { 8,111}, { 6,  4}, { 9, 87}, { 2,  0}, { 8,249}, { 7,239}, { 9,167}, 
    { 2,  0}, { 8,190}, { 6,255}, { 9,124}, { 2,  0}, { 9, 47}, { 8, 18}, { 9,206}, 
    { 2,  0}, { 8,136}, { 6, 64}, { 9,105}, { 2,  0}, { 9, 26}, { 7,254}, { 9,185}, 
    { 2,  0}, { 8,233}, { 7, 32}, { 9,149}, { 2,  0}, { 9, 69}, { 8, 63}, { 9,228}, 
    { 2,  0}, { 8,126}, { 6,  8}, { 9, 97}, { 2,  0}, { 9, 13}, { 7,251}, { 9,176}, 
    { 2,  0}, { 8,219}, { 7,  2}, { 9,140}, { 2,  0}, { 9, 56}, { 8, 34}, { 9,216}, 
    { 2,  0}, { 8,175}, { 6,128}, { 9,114}, { 2,  0}, { 9, 38}, { 8,  9}, { 9,198}, 
    { 2,  0}, { 8,238}, { 7,191}, { 9,157}, { 2,  0}, { 9, 78}, { 8, 72}, { 9,244}, 
    { 2,  0}, { 8, 80}, { 6,  4}, { 9, 82}, { 2,  0}, { 8,243}, { 7,223}, { 9,162}, 
    { 2,  0}, { 8,183}, { 6,255}, { 9,117}, { 2,  0}, { 9, 42}, { 8, 10}, { 9,201}, 
    { 2,  0}, { 8,129}, { 6, 64}, { 9,100}, { 2,  0}, { 9, 19}, { 7,253}, { 9,179}, 
    { 2,  0}, { 8,221}, { 7, 16}, { 9,143}, { 2,  0}, { 9, 59}, { 8, 36}, { 9,220}, 
    { 2,  0}, { 8,119}, { 6,  8}, { 9, 90}, { 2,  0}, { 8,250}, { 7,247}, { 9,170}, 
    { 2,  0}, { 8,192}, { 7,  1}, { 9,134}, { 2,  0}, { 9, 51}, { 8, 20}, { 9,210}, 
    { 2,  0}, { 8,144}, { 6,128}, { 9,108}, { 2,  0}, { 9, 29}, { 8,  3}, { 9,193}, 
    { 2,  0}, { 8,234}, { 7,127}, { 9,152}, { 2,  0}, { 9, 73}, { 8, 65}, { 9,232}, 
    { 2,  0}, { 8, 96}, { 6,  4}, { 9, 86}, { 2,  0}, { 8,246}, { 7,239}, { 9,166}, 
    { 2,  0}, { 8,189}, { 6,255}, { 9,122}, { 2,  0}, { 9, 46}, { 8, 17}, { 9,205}, 
    { 2,  0}, { 8,132}, { 6, 64}, { 9,104}, { 2,  0}, { 9, 25}, { 7,254}, { 9,184}, 
    { 2,  0}, { 8,231}, { 7, 32}, { 9,148}, { 2,  0}, { 9, 67}, { 8, 48}, { 9,227}, 
    { 2,  0}, { 8,125}, { 6,  8}, { 9, 94}, { 2,  0}, { 9, 11}, { 7,251}, { 9,174}, 
    { 2,  0}, { 8,215}, { 7,  2}, { 9,139}, { 2,  0}, { 9, 55}, { 8, 33}, { 9,214}, 
    { 2,  0}, { 8,160}, { 6,128}, { 9,113}, { 2,  0}, { 9, 37}, { 8,  6}, { 9,197}, 
    { 2,  0}, { 8,237}, { 7,191}, { 9,156}, { 2,  0}, { 9, 77}, { 8, 68}, { 9,242}, 
    { 2,  0}, { 8, 95}, { 6,  4}, { 9, 84}, { 2,  0}, { 8,245}, { 7,223}, { 9,164}, 
    { 2,  0}, { 8,187}, { 6,255}, { 9,120}, { 2,  0}, { 9, 44}, { 8, 12}, { 9,203}, 
    { 2,  0}, { 8,130}, { 6, 64}, { 9,102}, { 2,  0}, { 9, 22}, { 7,253}, { 9,181}, 
    { 2,  0}, { 8,222}, { 7, 16}, { 9,146}, { 2,  0}, { 9, 61}, { 8, 40}, { 9,225}, 
    { 2,  0}, { 8,123}, { 6,  8}, { 9, 92}, { 2,  0}, { 8,252}, { 7,247}, { 9,172}, 
    { 2,  0}, { 8,207}, { 7,  1}, { 9,137}, { 2,  0}, { 9, 53}, { 8, 24}, { 9,212}, 
    { 2,  0}, { 8,159}, { 6,128}, { 9,110}, { 2,  0}, { 9, 31}, { 8,  5}, { 9,195}, 
    { 2,  0}, { 8,235}, { 7,127}, { 9,154}, { 2,  0}, { 9, 75}, { 8, 66}, { 9,240}, 
    { 2,  0}, { 8,111}, { 6,  4}, { 9, 88}, { 2,  0}, { 8,249}, { 7,239}, { 9,168}, 
    { 2,  0}, { 8,190}, { 6,255}, { 9,131}, { 2,  0}, { 9, 49}, { 8, 18}, { 9,208}, 
    { 2,  0}, { 8,136}, { 6, 64}, { 9,106}, { 2,  0}, { 9, 27}, { 7,254}, { 9,186}, 
    { 2,  0}, { 8,233}, { 7, 32}, { 9,150}, { 2,  0}, { 9, 70}, { 8, 63}, { 9,229}, 
    { 2,  0}, { 8,126}, { 6,  8}, { 9, 98}, { 2,  0}, { 9, 14}, { 7,251}, { 9,177}, 
    { 2,  0}, { 8,219}, { 7,  2}, { 9,141}, { 2,  0}, { 9, 57}, { 8, 34}, { 9,217}, 
    { 2,  0}, { 8,175}, { 6,128}, { 9,115}, { 2,  0}, { 9, 39}, { 8,  9}, { 9,199}, 
    { 2,  0}, { 8,238}, { 7,191}, { 9,158}, { 2,  0}, { 9, 79}, { 8, 72}, { 9,248}, 
    { 2,  0}, { 8, 80}, { 6,  4}, { 9, 81}, { 2,  0}, { 8,243}, { 7,223}, { 9,161}, 
    { 2,  0}, { 8,183}, { 6,255}, { 9,116}, { 2,  0}, { 9, 41}, { 8, 10}, { 9,200}, 
    { 2,  0}, { 8,129}, { 6, 64}, { 9, 99}, { 2,  0}, { 9, 15}, { 7,253}, { 9,178}, 
    { 2,  0}, { 8,221}, { 7, 16}, { 9,142}, { 2,  0}, { 9, 58}, { 8, 36}, { 9,218}, 
    { 2,  0}, { 8,119}, { 6,  8}, { 9, 89}, { 2,  0}, { 8,250}, { 7,247}, { 9,169}, 
    { 2,  0}, { 8,192}, { 7,  1}, { 9,133}, { 2,  0}, { 9, 50}, { 8, 20}, { 9,209}, 
    { 2,  0}, { 8,144}, { 6,128}, { 9,107}, { 2,  0}, { 9, 28}, { 8,  3}, { 9,188}, 
    { 2,  0}, { 8,234}, { 7,127}, { 9,151}, { 2,  0}, { 9, 71}, { 8, 65}, { 9,230}, 
    { 2,  0}, { 8, 96}, { 6,  4}, { 9, 85}, { 2,  0}, { 8,246}, { 7,239}, { 9,165}, 
    { 2,  0}, { 8,189}, { 6,255}, { 9,121}, { 2,  0}, { 9, 45}, { 8, 17}, { 9,204}, 
    { 2,  0}, { 8,132}, { 6, 64}, { 9,103}, { 2,  0}, { 9, 23}, { 7,254}, { 9,182}, 
    { 2,  0}, { 8,231}, { 7, 32}, { 9,147}, { 2,  0}, { 9, 62}, { 8, 48}, { 9,226}, 
    { 2,  0}, { 8,125}, { 6,  8}, { 9, 93}, { 2,  0}, { 9,  7}, { 7,251}, { 9,173}, 
    { 2,  0}, { 8,215}, { 7,  2}, { 9,138}, { 2,  0}, { 9, 54}, { 8, 33}, { 9,213}, 
    { 2,  0}, { 8,160}, { 6,128}, { 9,112}, { 2,  0}, { 9, 35}, { 8,  6}, { 9,196}, 
    { 2,  0}, { 8,237}, { 7,191}, { 9,155}, { 2,  0}, { 9, 76}, { 8, 68}, { 9,241}, 
    { 2,  0}, { 8, 95}, { 6,  4}, { 9, 83}, { 2,  0}, { 8,245}, { 7,223}, { 9,163}, 
    { 2,  0}, { 8,187}, { 6,255}, { 9,118}, { 2,  0}, { 9, 43}, { 8, 12}, { 9,202}, 
    { 2,  0}, { 8,130}, { 6, 64}, { 9,101}, { 2,  0}, { 9, 21}, { 7,253}, { 9,180}, 
    { 2,  0}, { 8,222}, { 7, 16}, { 9,145}, { 2,  0}, { 9, 60}, { 8, 40}, { 9,224}, 
    { 2,  0}, { 8,123}, { 6,  8}, { 9, 91}, { 2,  0}, { 8,252}, { 7,247}, { 9,171}, 
    { 2,  0}, { 8,207}, { 7,  1}, { 9,135}, { 2,  0}, { 9, 52}, { 8, 24}, { 9,211}, 
    { 2,  0}, { 8,159}, { 6,128}, { 9,109}, { 2,  0}, { 9, 30}, { 8,  5}, { 9,194}, 
    { 2,  0}, { 8,235}, { 7,127}, { 9,153}, { 2,  0}, { 9, 74}, { 8, 66}, { 9,236}, 
    { 2,  0}, { 8,111}, { 6,  4}, { 9, 87}, { 2,  0}, { 8,249}, { 7,239}, { 9,167}, 
    { 2,  0}, { 8,190}, { 6,255}, { 9,124}, { 2,  0}, { 9, 47}, { 8, 18}, { 9,206}, 
    { 2,  0}, { 8,136}, { 6, 64}, { 9,105}, { 2,  0}, { 9, 26}, { 7,254}, { 9,185}, 
    { 2,  0}, { 8,233}, { 7, 32}, { 9,149}, { 2,  0}, { 9, 69}, { 8, 63}, { 9,228}, 
    { 2,  0}, { 8,126}, { 6,  8}, { 9, 97}, { 2,  0}, { 9, 13}, { 7,251}, { 9,176}, 
    { 2,  0}, { 8,219}, { 7,  2}, { 9,140}, { 2,  0}, { 9, 56}, { 8, 34}, { 9,216}, 
    { 2,  0}, { 8,175}, { 6,128}, { 9,114}, { 2,  0}, { 9, 38}, { 8,  9}, { 9,198}, 
    { 2,  0}, { 8,238}, { 7,191}, { 9,157}, { 2,  0}, { 9, 78}, { 8, 72}, { 9,244}, 
    { 2,  0}, { 8, 80}, { 6,  4}, { 9, 82}, { 2,  0}, { 8,243}, { 7,223}, { 9,162}, 
    { 2,  0}, { 8,183}, { 6,255}, { 9,117}, { 2,  0}, { 9, 42}, { 8, 10}, { 9,201}, 
    { 2,  0}, { 8,129}, { 6, 64}, { 9,100}, { 2,  0}, { 9, 19}, { 7,253}, { 9,179}, 
    { 2,  0}, { 8,221}, { 7, 16}, { 9,143}, { 2,  0}, { 9, 59}, { 8, 36}, { 9,220}, 
    { 2,  0}, { 8,119}, { 6,  8}, { 9, 90}, { 2,  0}, { 8,250}, { 7,247}, { 9,170}, 
    { 2,  0}, { 8,192}, { 7,  1}, { 9,134}, { 2,  0}, { 9, 51}, { 8, 20}, { 9,210}, 
    { 2,  0}, { 8,144}, { 6,128}, { 9,108}, { 2,  0}, { 9, 29}, { 8,  3}, { 9,193}, 
    { 2,  0}, { 8,234}, { 7,127}, { 9,152}, { 2,  0}, { 9, 73}, { 8, 65}, { 9,232}, 
    { 2,  0}, { 8, 96}, { 6,  4}, { 9, 86}, { 2,  0}, { 8,246}, { 7,239}, { 9,166}, 
    { 2,  0}, { 8,189}, { 6,255}, { 9,122}, { 2,  0}, { 9, 46}, { 8, 17}, { 9,205}, 
    { 2,  0}, { 8,132}, { 6, 64}, { 9,104}, { 2,  0}, { 9, 25}, { 7,254}, { 9,184}, 
    { 2,  0}, { 8,231}, { 7, 32}, { 9,148}, { 2,  0}, { 9, 67}, { 8, 48}, { 9,227}, 
    { 2,  0}, { 8,125}, { 6,  8}, { 9, 94}, { 2,  0}, { 9, 11}, { 7,251}, { 9,174}, 
    { 2,  0}, { 8,215}, { 7,  2}, { 9,139}, { 2,  0}, { 9, 55}, { 8, 33}, { 9,214}, 
    { 2,  0}, { 8,160}, { 6,128}, { 9,113}, { 2,  0}, { 9, 37}, { 8,  6}, { 9,197}, 
    { 2,  0}, { 8,237}, { 7,191}, { 9,156}, { 2,  0}, { 9, 77}, { 8, 68}, { 9,242}, 
    { 2,  0}, { 8, 95}, { 6,  4}, { 9, 84}, { 2,  0}, { 8,245}, { 7,223}, { 9,164}, 
    { 2,  0}, { 8,187}, { 6,255}, { 9,120}, { 2,  0}, { 9, 44}, { 8, 12}, { 9,203}, 
    { 2,  0}, { 8,130}, { 6, 64}, { 9,102}, { 2,  0}, { 9, 22}, { 7,253}, { 9,181}, 
    { 2,  0}, { 8,222}, { 7, 16}, { 9,146}, { 2,  0}, { 9, 61}, { 8, 40}, { 9,225}, 
    { 2,  0}, { 8,123}, { 6,  8}, { 9, 92}, { 2,  0}, { 8,252}, { 7,247}, { 9,172}, 
    { 2,  0}, { 8,207}, { 7,  1}, { 9,137}, { 2,  0}, { 9, 53}, { 8, 24}, { 9,212}, 
    { 2,  0}, { 8,159}, { 6,128}, { 9,110}, { 2,  0}, { 9, 31}, { 8,  5}, { 9,195}, 
    { 2,  0}, { 8,235}, { 7,127}, { 9,154}, { 2,  0}, { 9, 75}, { 8, 66}, { 9,240}, 
    { 2,  0}, { 8,111}, { 6,  4}, { 9, 88}, { 2,  0}, { 8,249}, { 7,239}, { 9,168}, 
    { 2,  0}, { 8,190}, { 6,255}, { 9,131}, { 2,  0}, { 9, 49}, { 8, 18}, { 9,208}, 
    { 2,  0}, { 8,136}, { 6, 64}, { 9,106}, { 2,  0}, { 9, 27}, { 7,254}, { 9,186}, 
    { 2,  0}, { 8,233}, { 7, 32}, { 9,150}, { 2,  0}, { 9, 70}, { 8, 63}, { 9,229}, 
    { 2,  0}, { 8,126}, { 6,  8}, { 9, 98}, { 2,  0}, { 9, 14}, { 7,251}, { 9,177}, 
    { 2,  0}, { 8,219}, { 7,  2}, { 9,141}, { 2,  0}, { 9, 57}, { 8, 34}, { 9,217}, 
    { 2,  0}, { 8,175}, { 6,128}, { 9,115}, { 2,  0}, { 9, 39}, { 8,  9}, { 9,199}, 
    { 2,  0}, { 8,238}, { 7,191}, { 9,158}, { 2,  0}, { 9, 79}, { 8, 72}, { 9,248}, 
  }
  #endif
};

static radaudio_huffman rada_nonzero_bitflags_huff2 =
{
  #ifdef HUFFMAN_ENCODE
  {
    { 223,10}, { 991,11}, {2015,11}, {  63,11}, {1087,11}, { 575,11}, {1599,11}, {  61, 9}, 
    { 319,11}, {1343,11}, { 831,11}, { 317, 9}, {1855,11}, { 189, 9}, { 445, 9}, { 125, 9}, 
    { 191,11}, {1215,11}, { 735,10}, { 381, 9}, { 703,11}, { 253, 9}, { 509, 9}, {   3, 9}, 
    {1727,11}, { 259, 9}, { 131, 9}, { 387, 9}, {  67, 9}, { 323, 9}, { 195, 9}, {  33, 8}, 
    { 447,11}, {1471,11}, { 959,11}, { 451, 9}, {1983,11}, {  35, 9}, { 291, 9}, { 163, 9}, 
    { 127,11}, { 419, 9}, {  99, 9}, { 355, 9}, { 227, 9}, { 483, 9}, {  19, 9}, { 161, 8}, 
    {1151,11}, { 275, 9}, { 147, 9}, { 403, 9}, {  83, 9}, { 339, 9}, { 211, 9}, {  97, 8}, 
    { 467, 9}, {  51, 9}, { 307, 9}, { 225, 8}, { 179, 9}, {  17, 8}, { 145, 8}, {  50, 7}, 
    { 639,11}, {1663,11}, { 383,11}, { 435, 9}, {1407,11}, { 115, 9}, { 371, 9}, { 243, 9}, 
    { 895,11}, { 499, 9}, {  11, 9}, { 267, 9}, { 139, 9}, { 395, 9}, {  75, 9}, {  81, 8}, 
    {1919,11}, { 331, 9}, { 203, 9}, { 459, 9}, {  43, 9}, { 299, 9}, { 171, 9}, { 209, 8}, 
    { 427, 9}, { 107, 9}, { 363, 9}, {  49, 8}, { 235, 9}, { 177, 8}, { 113, 8}, { 114, 7}, 
    { 255,11}, { 491, 9}, {  27, 9}, { 283, 9}, { 155, 9}, { 411, 9}, {  91, 9}, { 241, 8}, 
    { 347, 9}, { 219, 9}, { 475, 9}, {   9, 8}, {  59, 9}, { 137, 8}, {  73, 8}, {  10, 7}, 
    { 315, 9}, { 187, 9}, { 443, 9}, { 201, 8}, { 123, 9}, {  41, 8}, { 169, 8}, {  74, 7}, 
    { 379, 9}, { 105, 8}, { 233, 8}, {  42, 7}, {  25, 8}, { 106, 7}, {  26, 7}, {   8, 5}, 
    {1279,11}, { 767,11}, {1791,11}, { 251, 9}, { 479,10}, { 507, 9}, {   7, 9}, { 263, 9}, 
    { 511,11}, { 135, 9}, { 391, 9}, {  71, 9}, { 327, 9}, { 199, 9}, { 455, 9}, { 153, 8}, 
    {1535,11}, {  39, 9}, { 295, 9}, { 167, 9}, { 423, 9}, { 103, 9}, { 359, 9}, {  89, 8}, 
    { 231, 9}, { 487, 9}, {  23, 9}, { 217, 8}, { 279, 9}, {  57, 8}, { 185, 8}, {  90, 7}, 
    {1023,11}, { 151, 9}, { 407, 9}, {  87, 9}, { 343, 9}, { 215, 9}, { 471, 9}, { 121, 8}, 
    {  55, 9}, { 311, 9}, { 183, 9}, { 249, 8}, { 439, 9}, {   5, 8}, { 133, 8}, {  58, 7}, 
    { 119, 9}, { 375, 9}, { 247, 9}, {  69, 8}, { 503, 9}, { 197, 8}, {  37, 8}, { 122, 7}, 
    {  15, 9}, { 165, 8}, { 101, 8}, {   6, 7}, { 229, 8}, {  70, 7}, {  38, 7}, {  24, 5}, 
    {2047,11}, { 271, 9}, { 143, 9}, { 399, 9}, {  79, 9}, { 335, 9}, { 207, 9}, {  21, 8}, 
    { 463, 9}, {  47, 9}, { 303, 9}, { 149, 8}, { 175, 9}, {  85, 8}, { 213, 8}, { 102, 7}, 
    { 431, 9}, { 111, 9}, { 367, 9}, {  53, 8}, { 239, 9}, { 181, 8}, { 117, 8}, {  22, 7}, 
    { 495, 9}, { 245, 8}, {  13, 8}, {  86, 7}, { 141, 8}, {  54, 7}, { 118, 7}, {   4, 5}, 
    {  31, 9}, { 287, 9}, { 159, 9}, {  77, 8}, { 415, 9}, { 205, 8}, {  45, 8}, {  14, 7}, 
    {  95, 9}, { 173, 8}, { 109, 8}, {  78, 7}, { 237, 8}, {  46, 7}, { 110, 7}, {  20, 5}, 
    { 351, 9}, {  29, 8}, { 157, 8}, {  30, 7}, {  93, 8}, {  94, 7}, {  62, 7}, {  12, 5}, 
    { 221, 8}, { 126, 7}, {   1, 7}, {  18, 6}, {  65, 7}, {  28, 5}, {   2, 5}, {   0, 4}, 
  },
  #endif
  #ifdef HUFFMAN_DECODE
  {
    { 4,255}, { 7,250}, { 5,254}, { 9, 23}, { 5,223}, { 8,173}, { 7,187}, { 9,134}, 
    { 5,127}, { 8,107}, { 7,111}, { 9, 74}, { 5,247}, { 8,218}, { 7,231}, { 9,184}, 
    { 4,255}, { 8, 61}, { 6,251}, { 9, 46}, { 5,239}, { 8,199}, { 7,215}, { 9,154}, 
    { 5,191}, { 8,124}, { 7,126}, { 9, 98}, { 5,253}, { 8,241}, { 7,243}, { 9,224}, 
    { 4,255}, { 8, 31}, { 5,254}, { 9, 37}, { 5,223}, { 8,182}, { 7,190}, { 9,145}, 
    { 5,127}, { 8,117}, { 7,123}, { 9, 84}, { 5,247}, { 8,230}, { 7,237}, { 9,201}, 
    { 4,255}, { 8, 91}, { 7, 63}, { 9, 57}, { 5,239}, { 8,211}, { 7,221}, { 9,168}, 
    { 5,191}, { 8,157}, { 7,175}, { 9,108}, { 5,253}, { 9,  7}, { 7,246}, {11,  3}, 
    { 4,255}, { 7,252}, { 5,254}, { 9, 28}, { 5,223}, { 8,179}, { 7,189}, { 9,139}, 
    { 5,127}, { 8,110}, { 7,119}, { 9, 78}, { 5,247}, { 8,227}, { 7,235}, { 9,196}, 
    { 4,255}, { 8, 79}, { 6,251}, { 9, 52}, { 5,239}, { 8,205}, { 7,219}, { 9,163}, 
    { 5,191}, { 8,151}, { 7,159}, { 9,102}, { 5,253}, { 8,244}, { 7,245}, { 9,232}, 
    { 4,255}, { 8, 55}, { 5,254}, { 9, 42}, { 5,223}, { 8,186}, { 7,207}, { 9,149}, 
    { 5,127}, { 8,121}, { 7,125}, { 9, 89}, { 5,247}, { 8,234}, { 7,238}, { 9,209}, 
    { 4,255}, { 8, 94}, { 7, 95}, { 9, 69}, { 5,239}, { 8,214}, { 7,222}, { 9,176}, 
    { 5,191}, { 8,167}, { 7,183}, { 9,116}, { 5,253}, { 9, 15}, { 7,249}, {11, 40}, 
    { 4,255}, { 7,250}, { 5,254}, { 9, 26}, { 5,223}, { 8,174}, { 7,187}, { 9,137}, 
    { 5,127}, { 8,109}, { 7,111}, { 9, 76}, { 5,247}, { 8,220}, { 7,231}, { 9,194}, 
    { 4,255}, { 8, 62}, { 6,251}, { 9, 50}, { 5,239}, { 8,203}, { 7,215}, { 9,161}, 
    { 5,191}, { 8,143}, { 7,126}, { 9,100}, { 5,253}, { 8,242}, { 7,243}, { 9,226}, 
    { 4,255}, { 8, 47}, { 5,254}, { 9, 39}, { 5,223}, { 8,185}, { 7,190}, { 9,147}, 
    { 5,127}, { 8,118}, { 7,123}, { 9, 86}, { 5,247}, { 8,233}, { 7,237}, { 9,204}, 
    { 4,255}, { 8, 93}, { 7, 63}, { 9, 60}, { 5,239}, { 8,213}, { 7,221}, { 9,170}, 
    { 5,191}, { 8,158}, { 7,175}, { 9,113}, { 5,253}, { 9, 13}, { 7,246}, {11, 16}, 
    { 4,255}, { 7,252}, { 5,254}, { 9, 30}, { 5,223}, { 8,181}, { 7,189}, { 9,141}, 
    { 5,127}, { 8,115}, { 7,119}, { 9, 82}, { 5,247}, { 8,229}, { 7,235}, { 9,198}, 
    { 4,255}, { 8, 87}, { 6,251}, { 9, 54}, { 5,239}, { 8,206}, { 7,219}, { 9,165}, 
    { 5,191}, { 8,155}, { 7,159}, { 9,105}, { 5,253}, { 8,248}, { 7,245}, {10,  0}, 
    { 4,255}, { 8, 59}, { 5,254}, { 9, 44}, { 5,223}, { 8,188}, { 7,207}, { 9,152}, 
    { 5,127}, { 8,122}, { 7,125}, { 9, 92}, { 5,247}, { 8,236}, { 7,238}, { 9,212}, 
    { 4,255}, { 8,103}, { 7, 95}, { 9, 71}, { 5,239}, { 8,217}, { 7,222}, { 9,178}, 
    { 5,191}, { 8,171}, { 7,183}, { 9,131}, { 5,253}, { 9, 21}, { 7,249}, {11, 96}, 
    { 4,255}, { 7,250}, { 5,254}, { 9, 25}, { 5,223}, { 8,173}, { 7,187}, { 9,135}, 
    { 5,127}, { 8,107}, { 7,111}, { 9, 75}, { 5,247}, { 8,218}, { 7,231}, { 9,193}, 
    { 4,255}, { 8, 61}, { 6,251}, { 9, 49}, { 5,239}, { 8,199}, { 7,215}, { 9,156}, 
    { 5,191}, { 8,124}, { 7,126}, { 9, 99}, { 5,253}, { 8,241}, { 7,243}, { 9,225}, 
    { 4,255}, { 8, 31}, { 5,254}, { 9, 38}, { 5,223}, { 8,182}, { 7,190}, { 9,146}, 
    { 5,127}, { 8,117}, { 7,123}, { 9, 85}, { 5,247}, { 8,230}, { 7,237}, { 9,202}, 
    { 4,255}, { 8, 91}, { 7, 63}, { 9, 58}, { 5,239}, { 8,211}, { 7,221}, { 9,169}, 
    { 5,191}, { 8,157}, { 7,175}, { 9,112}, { 5,253}, { 9, 11}, { 7,246}, {11,  8}, 
    { 4,255}, { 7,252}, { 5,254}, { 9, 29}, { 5,223}, { 8,179}, { 7,189}, { 9,140}, 
    { 5,127}, { 8,110}, { 7,119}, { 9, 81}, { 5,247}, { 8,227}, { 7,235}, { 9,197}, 
    { 4,255}, { 8, 79}, { 6,251}, { 9, 53}, { 5,239}, { 8,205}, { 7,219}, { 9,164}, 
    { 5,191}, { 8,151}, { 7,159}, { 9,104}, { 5,253}, { 8,244}, { 7,245}, { 9,240}, 
    { 4,255}, { 8, 55}, { 5,254}, { 9, 43}, { 5,223}, { 8,186}, { 7,207}, { 9,150}, 
    { 5,127}, { 8,121}, { 7,125}, { 9, 90}, { 5,247}, { 8,234}, { 7,238}, { 9,210}, 
    { 4,255}, { 8, 94}, { 7, 95}, { 9, 70}, { 5,239}, { 8,214}, { 7,222}, { 9,177}, 
    { 5,191}, { 8,167}, { 7,183}, { 9,120}, { 5,253}, { 9, 19}, { 7,249}, {11, 66}, 
    { 4,255}, { 7,250}, { 5,254}, { 9, 27}, { 5,223}, { 8,174}, { 7,187}, { 9,138}, 
    { 5,127}, { 8,109}, { 7,111}, { 9, 77}, { 5,247}, { 8,220}, { 7,231}, { 9,195}, 
    { 4,255}, { 8, 62}, { 6,251}, { 9, 51}, { 5,239}, { 8,203}, { 7,215}, { 9,162}, 
    { 5,191}, { 8,143}, { 7,126}, { 9,101}, { 5,253}, { 8,242}, { 7,243}, { 9,228}, 
    { 4,255}, { 8, 47}, { 5,254}, { 9, 41}, { 5,223}, { 8,185}, { 7,190}, { 9,148}, 
    { 5,127}, { 8,118}, { 7,123}, { 9, 88}, { 5,247}, { 8,233}, { 7,237}, { 9,208}, 
    { 4,255}, { 8, 93}, { 7, 63}, { 9, 67}, { 5,239}, { 8,213}, { 7,221}, { 9,172}, 
    { 5,191}, { 8,158}, { 7,175}, { 9,114}, { 5,253}, { 9, 14}, { 7,246}, {11, 32}, 
    { 4,255}, { 7,252}, { 5,254}, { 9, 35}, { 5,223}, { 8,181}, { 7,189}, { 9,142}, 
    { 5,127}, { 8,115}, { 7,119}, { 9, 83}, { 5,247}, { 8,229}, { 7,235}, { 9,200}, 
    { 4,255}, { 8, 87}, { 6,251}, { 9, 56}, { 5,239}, { 8,206}, { 7,219}, { 9,166}, 
    { 5,191}, { 8,155}, { 7,159}, { 9,106}, { 5,253}, { 8,248}, { 7,245}, {10,132}, 
    { 4,255}, { 8, 59}, { 5,254}, { 9, 45}, { 5,223}, { 8,188}, { 7,207}, { 9,153}, 
    { 5,127}, { 8,122}, { 7,125}, { 9, 97}, { 5,247}, { 8,236}, { 7,238}, { 9,216}, 
    { 4,255}, { 8,103}, { 7, 95}, { 9, 73}, { 5,239}, { 8,217}, { 7,222}, { 9,180}, 
    { 5,191}, { 8,171}, { 7,183}, { 9,133}, { 5,253}, { 9, 22}, { 7,249}, {11,136}, 
    { 4,255}, { 7,250}, { 5,254}, { 9, 23}, { 5,223}, { 8,173}, { 7,187}, { 9,134}, 
    { 5,127}, { 8,107}, { 7,111}, { 9, 74}, { 5,247}, { 8,218}, { 7,231}, { 9,184}, 
    { 4,255}, { 8, 61}, { 6,251}, { 9, 46}, { 5,239}, { 8,199}, { 7,215}, { 9,154}, 
    { 5,191}, { 8,124}, { 7,126}, { 9, 98}, { 5,253}, { 8,241}, { 7,243}, { 9,224}, 
    { 4,255}, { 8, 31}, { 5,254}, { 9, 37}, { 5,223}, { 8,182}, { 7,190}, { 9,145}, 
    { 5,127}, { 8,117}, { 7,123}, { 9, 84}, { 5,247}, { 8,230}, { 7,237}, { 9,201}, 
    { 4,255}, { 8, 91}, { 7, 63}, { 9, 57}, { 5,239}, { 8,211}, { 7,221}, { 9,168}, 
    { 5,191}, { 8,157}, { 7,175}, { 9,108}, { 5,253}, { 9,  7}, { 7,246}, {11,  5}, 
    { 4,255}, { 7,252}, { 5,254}, { 9, 28}, { 5,223}, { 8,179}, { 7,189}, { 9,139}, 
    { 5,127}, { 8,110}, { 7,119}, { 9, 78}, { 5,247}, { 8,227}, { 7,235}, { 9,196}, 
    { 4,255}, { 8, 79}, { 6,251}, { 9, 52}, { 5,239}, { 8,205}, { 7,219}, { 9,163}, 
    { 5,191}, { 8,151}, { 7,159}, { 9,102}, { 5,253}, { 8,244}, { 7,245}, { 9,232}, 
    { 4,255}, { 8, 55}, { 5,254}, { 9, 42}, { 5,223}, { 8,186}, { 7,207}, { 9,149}, 
    { 5,127}, { 8,121}, { 7,125}, { 9, 89}, { 5,247}, { 8,234}, { 7,238}, { 9,209}, 
    { 4,255}, { 8, 94}, { 7, 95}, { 9, 69}, { 5,239}, { 8,214}, { 7,222}, { 9,176}, 
    { 5,191}, { 8,167}, { 7,183}, { 9,116}, { 5,253}, { 9, 15}, { 7,249}, {11, 64}, 
    { 4,255}, { 7,250}, { 5,254}, { 9, 26}, { 5,223}, { 8,174}, { 7,187}, { 9,137}, 
    { 5,127}, { 8,109}, { 7,111}, { 9, 76}, { 5,247}, { 8,220}, { 7,231}, { 9,194}, 
    { 4,255}, { 8, 62}, { 6,251}, { 9, 50}, { 5,239}, { 8,203}, { 7,215}, { 9,161}, 
    { 5,191}, { 8,143}, { 7,126}, { 9,100}, { 5,253}, { 8,242}, { 7,243}, { 9,226}, 
    { 4,255}, { 8, 47}, { 5,254}, { 9, 39}, { 5,223}, { 8,185}, { 7,190}, { 9,147}, 
    { 5,127}, { 8,118}, { 7,123}, { 9, 86}, { 5,247}, { 8,233}, { 7,237}, { 9,204}, 
    { 4,255}, { 8, 93}, { 7, 63}, { 9, 60}, { 5,239}, { 8,213}, { 7,221}, { 9,170}, 
    { 5,191}, { 8,158}, { 7,175}, { 9,113}, { 5,253}, { 9, 13}, { 7,246}, {11, 20}, 
    { 4,255}, { 7,252}, { 5,254}, { 9, 30}, { 5,223}, { 8,181}, { 7,189}, { 9,141}, 
    { 5,127}, { 8,115}, { 7,119}, { 9, 82}, { 5,247}, { 8,229}, { 7,235}, { 9,198}, 
    { 4,255}, { 8, 87}, { 6,251}, { 9, 54}, { 5,239}, { 8,206}, { 7,219}, { 9,165}, 
    { 5,191}, { 8,155}, { 7,159}, { 9,105}, { 5,253}, { 8,248}, { 7,245}, {10, 18}, 
    { 4,255}, { 8, 59}, { 5,254}, { 9, 44}, { 5,223}, { 8,188}, { 7,207}, { 9,152}, 
    { 5,127}, { 8,122}, { 7,125}, { 9, 92}, { 5,247}, { 8,236}, { 7,238}, { 9,212}, 
    { 4,255}, { 8,103}, { 7, 95}, { 9, 71}, { 5,239}, { 8,217}, { 7,222}, { 9,178}, 
    { 5,191}, { 8,171}, { 7,183}, { 9,131}, { 5,253}, { 9, 21}, { 7,249}, {11,129}, 
    { 4,255}, { 7,250}, { 5,254}, { 9, 25}, { 5,223}, { 8,173}, { 7,187}, { 9,135}, 
    { 5,127}, { 8,107}, { 7,111}, { 9, 75}, { 5,247}, { 8,218}, { 7,231}, { 9,193}, 
    { 4,255}, { 8, 61}, { 6,251}, { 9, 49}, { 5,239}, { 8,199}, { 7,215}, { 9,156}, 
    { 5,191}, { 8,124}, { 7,126}, { 9, 99}, { 5,253}, { 8,241}, { 7,243}, { 9,225}, 
    { 4,255}, { 8, 31}, { 5,254}, { 9, 38}, { 5,223}, { 8,182}, { 7,190}, { 9,146}, 
    { 5,127}, { 8,117}, { 7,123}, { 9, 85}, { 5,247}, { 8,230}, { 7,237}, { 9,202}, 
    { 4,255}, { 8, 91}, { 7, 63}, { 9, 58}, { 5,239}, { 8,211}, { 7,221}, { 9,169}, 
    { 5,191}, { 8,157}, { 7,175}, { 9,112}, { 5,253}, { 9, 11}, { 7,246}, {11, 10}, 
    { 4,255}, { 7,252}, { 5,254}, { 9, 29}, { 5,223}, { 8,179}, { 7,189}, { 9,140}, 
    { 5,127}, { 8,110}, { 7,119}, { 9, 81}, { 5,247}, { 8,227}, { 7,235}, { 9,197}, 
    { 4,255}, { 8, 79}, { 6,251}, { 9, 53}, { 5,239}, { 8,205}, { 7,219}, { 9,164}, 
    { 5,191}, { 8,151}, { 7,159}, { 9,104}, { 5,253}, { 8,244}, { 7,245}, { 9,240}, 
    { 4,255}, { 8, 55}, { 5,254}, { 9, 43}, { 5,223}, { 8,186}, { 7,207}, { 9,150}, 
    { 5,127}, { 8,121}, { 7,125}, { 9, 90}, { 5,247}, { 8,234}, { 7,238}, { 9,210}, 
    { 4,255}, { 8, 94}, { 7, 95}, { 9, 70}, { 5,239}, { 8,214}, { 7,222}, { 9,177}, 
    { 5,191}, { 8,167}, { 7,183}, { 9,120}, { 5,253}, { 9, 19}, { 7,249}, {11, 72}, 
    { 4,255}, { 7,250}, { 5,254}, { 9, 27}, { 5,223}, { 8,174}, { 7,187}, { 9,138}, 
    { 5,127}, { 8,109}, { 7,111}, { 9, 77}, { 5,247}, { 8,220}, { 7,231}, { 9,195}, 
    { 4,255}, { 8, 62}, { 6,251}, { 9, 51}, { 5,239}, { 8,203}, { 7,215}, { 9,162}, 
    { 5,191}, { 8,143}, { 7,126}, { 9,101}, { 5,253}, { 8,242}, { 7,243}, { 9,228}, 
    { 4,255}, { 8, 47}, { 5,254}, { 9, 41}, { 5,223}, { 8,185}, { 7,190}, { 9,148}, 
    { 5,127}, { 8,118}, { 7,123}, { 9, 88}, { 5,247}, { 8,233}, { 7,237}, { 9,208}, 
    { 4,255}, { 8, 93}, { 7, 63}, { 9, 67}, { 5,239}, { 8,213}, { 7,221}, { 9,172}, 
    { 5,191}, { 8,158}, { 7,175}, { 9,114}, { 5,253}, { 9, 14}, { 7,246}, {11, 34}, 
    { 4,255}, { 7,252}, { 5,254}, { 9, 35}, { 5,223}, { 8,181}, { 7,189}, { 9,142}, 
    { 5,127}, { 8,115}, { 7,119}, { 9, 83}, { 5,247}, { 8,229}, { 7,235}, { 9,200}, 
    { 4,255}, { 8, 87}, { 6,251}, { 9, 56}, { 5,239}, { 8,206}, { 7,219}, { 9,166}, 
    { 5,191}, { 8,155}, { 7,159}, { 9,106}, { 5,253}, { 8,248}, { 7,245}, {11,  1}, 
    { 4,255}, { 8, 59}, { 5,254}, { 9, 45}, { 5,223}, { 8,188}, { 7,207}, { 9,153}, 
    { 5,127}, { 8,122}, { 7,125}, { 9, 97}, { 5,247}, { 8,236}, { 7,238}, { 9,216}, 
    { 4,255}, { 8,103}, { 7, 95}, { 9, 73}, { 5,239}, { 8,217}, { 7,222}, { 9,180}, 
    { 5,191}, { 8,171}, { 7,183}, { 9,133}, { 5,253}, { 9, 22}, { 7,249}, {11,160}, 
    { 4,255}, { 7,250}, { 5,254}, { 9, 23}, { 5,223}, { 8,173}, { 7,187}, { 9,134}, 
    { 5,127}, { 8,107}, { 7,111}, { 9, 74}, { 5,247}, { 8,218}, { 7,231}, { 9,184}, 
    { 4,255}, { 8, 61}, { 6,251}, { 9, 46}, { 5,239}, { 8,199}, { 7,215}, { 9,154}, 
    { 5,191}, { 8,124}, { 7,126}, { 9, 98}, { 5,253}, { 8,241}, { 7,243}, { 9,224}, 
    { 4,255}, { 8, 31}, { 5,254}, { 9, 37}, { 5,223}, { 8,182}, { 7,190}, { 9,145}, 
    { 5,127}, { 8,117}, { 7,123}, { 9, 84}, { 5,247}, { 8,230}, { 7,237}, { 9,201}, 
    { 4,255}, { 8, 91}, { 7, 63}, { 9, 57}, { 5,239}, { 8,211}, { 7,221}, { 9,168}, 
    { 5,191}, { 8,157}, { 7,175}, { 9,108}, { 5,253}, { 9,  7}, { 7,246}, {11,  4}, 
    { 4,255}, { 7,252}, { 5,254}, { 9, 28}, { 5,223}, { 8,179}, { 7,189}, { 9,139}, 
    { 5,127}, { 8,110}, { 7,119}, { 9, 78}, { 5,247}, { 8,227}, { 7,235}, { 9,196}, 
    { 4,255}, { 8, 79}, { 6,251}, { 9, 52}, { 5,239}, { 8,205}, { 7,219}, { 9,163}, 
    { 5,191}, { 8,151}, { 7,159}, { 9,102}, { 5,253}, { 8,244}, { 7,245}, { 9,232}, 
    { 4,255}, { 8, 55}, { 5,254}, { 9, 42}, { 5,223}, { 8,186}, { 7,207}, { 9,149}, 
    { 5,127}, { 8,121}, { 7,125}, { 9, 89}, { 5,247}, { 8,234}, { 7,238}, { 9,209}, 
    { 4,255}, { 8, 94}, { 7, 95}, { 9, 69}, { 5,239}, { 8,214}, { 7,222}, { 9,176}, 
    { 5,191}, { 8,167}, { 7,183}, { 9,116}, { 5,253}, { 9, 15}, { 7,249}, {11, 48}, 
    { 4,255}, { 7,250}, { 5,254}, { 9, 26}, { 5,223}, { 8,174}, { 7,187}, { 9,137}, 
    { 5,127}, { 8,109}, { 7,111}, { 9, 76}, { 5,247}, { 8,220}, { 7,231}, { 9,194}, 
    { 4,255}, { 8, 62}, { 6,251}, { 9, 50}, { 5,239}, { 8,203}, { 7,215}, { 9,161}, 
    { 5,191}, { 8,143}, { 7,126}, { 9,100}, { 5,253}, { 8,242}, { 7,243}, { 9,226}, 
    { 4,255}, { 8, 47}, { 5,254}, { 9, 39}, { 5,223}, { 8,185}, { 7,190}, { 9,147}, 
    { 5,127}, { 8,118}, { 7,123}, { 9, 86}, { 5,247}, { 8,233}, { 7,237}, { 9,204}, 
    { 4,255}, { 8, 93}, { 7, 63}, { 9, 60}, { 5,239}, { 8,213}, { 7,221}, { 9,170}, 
    { 5,191}, { 8,158}, { 7,175}, { 9,113}, { 5,253}, { 9, 13}, { 7,246}, {11, 17}, 
    { 4,255}, { 7,252}, { 5,254}, { 9, 30}, { 5,223}, { 8,181}, { 7,189}, { 9,141}, 
    { 5,127}, { 8,115}, { 7,119}, { 9, 82}, { 5,247}, { 8,229}, { 7,235}, { 9,198}, 
    { 4,255}, { 8, 87}, { 6,251}, { 9, 54}, { 5,239}, { 8,206}, { 7,219}, { 9,165}, 
    { 5,191}, { 8,155}, { 7,159}, { 9,105}, { 5,253}, { 8,248}, { 7,245}, {10,  0}, 
    { 4,255}, { 8, 59}, { 5,254}, { 9, 44}, { 5,223}, { 8,188}, { 7,207}, { 9,152}, 
    { 5,127}, { 8,122}, { 7,125}, { 9, 92}, { 5,247}, { 8,236}, { 7,238}, { 9,212}, 
    { 4,255}, { 8,103}, { 7, 95}, { 9, 71}, { 5,239}, { 8,217}, { 7,222}, { 9,178}, 
    { 5,191}, { 8,171}, { 7,183}, { 9,131}, { 5,253}, { 9, 21}, { 7,249}, {11,128}, 
    { 4,255}, { 7,250}, { 5,254}, { 9, 25}, { 5,223}, { 8,173}, { 7,187}, { 9,135}, 
    { 5,127}, { 8,107}, { 7,111}, { 9, 75}, { 5,247}, { 8,218}, { 7,231}, { 9,193}, 
    { 4,255}, { 8, 61}, { 6,251}, { 9, 49}, { 5,239}, { 8,199}, { 7,215}, { 9,156}, 
    { 5,191}, { 8,124}, { 7,126}, { 9, 99}, { 5,253}, { 8,241}, { 7,243}, { 9,225}, 
    { 4,255}, { 8, 31}, { 5,254}, { 9, 38}, { 5,223}, { 8,182}, { 7,190}, { 9,146}, 
    { 5,127}, { 8,117}, { 7,123}, { 9, 85}, { 5,247}, { 8,230}, { 7,237}, { 9,202}, 
    { 4,255}, { 8, 91}, { 7, 63}, { 9, 58}, { 5,239}, { 8,211}, { 7,221}, { 9,169}, 
    { 5,191}, { 8,157}, { 7,175}, { 9,112}, { 5,253}, { 9, 11}, { 7,246}, {11,  9}, 
    { 4,255}, { 7,252}, { 5,254}, { 9, 29}, { 5,223}, { 8,179}, { 7,189}, { 9,140}, 
    { 5,127}, { 8,110}, { 7,119}, { 9, 81}, { 5,247}, { 8,227}, { 7,235}, { 9,197}, 
    { 4,255}, { 8, 79}, { 6,251}, { 9, 53}, { 5,239}, { 8,205}, { 7,219}, { 9,164}, 
    { 5,191}, { 8,151}, { 7,159}, { 9,104}, { 5,253}, { 8,244}, { 7,245}, { 9,240}, 
    { 4,255}, { 8, 55}, { 5,254}, { 9, 43}, { 5,223}, { 8,186}, { 7,207}, { 9,150}, 
    { 5,127}, { 8,121}, { 7,125}, { 9, 90}, { 5,247}, { 8,234}, { 7,238}, { 9,210}, 
    { 4,255}, { 8, 94}, { 7, 95}, { 9, 70}, { 5,239}, { 8,214}, { 7,222}, { 9,177}, 
    { 5,191}, { 8,167}, { 7,183}, { 9,120}, { 5,253}, { 9, 19}, { 7,249}, {11, 68}, 
    { 4,255}, { 7,250}, { 5,254}, { 9, 27}, { 5,223}, { 8,174}, { 7,187}, { 9,138}, 
    { 5,127}, { 8,109}, { 7,111}, { 9, 77}, { 5,247}, { 8,220}, { 7,231}, { 9,195}, 
    { 4,255}, { 8, 62}, { 6,251}, { 9, 51}, { 5,239}, { 8,203}, { 7,215}, { 9,162}, 
    { 5,191}, { 8,143}, { 7,126}, { 9,101}, { 5,253}, { 8,242}, { 7,243}, { 9,228}, 
    { 4,255}, { 8, 47}, { 5,254}, { 9, 41}, { 5,223}, { 8,185}, { 7,190}, { 9,148}, 
    { 5,127}, { 8,118}, { 7,123}, { 9, 88}, { 5,247}, { 8,233}, { 7,237}, { 9,208}, 
    { 4,255}, { 8, 93}, { 7, 63}, { 9, 67}, { 5,239}, { 8,213}, { 7,221}, { 9,172}, 
    { 5,191}, { 8,158}, { 7,175}, { 9,114}, { 5,253}, { 9, 14}, { 7,246}, {11, 33}, 
    { 4,255}, { 7,252}, { 5,254}, { 9, 35}, { 5,223}, { 8,181}, { 7,189}, { 9,142}, 
    { 5,127}, { 8,115}, { 7,119}, { 9, 83}, { 5,247}, { 8,229}, { 7,235}, { 9,200}, 
    { 4,255}, { 8, 87}, { 6,251}, { 9, 56}, { 5,239}, { 8,206}, { 7,219}, { 9,166}, 
    { 5,191}, { 8,155}, { 7,159}, { 9,106}, { 5,253}, { 8,248}, { 7,245}, {10,132}, 
    { 4,255}, { 8, 59}, { 5,254}, { 9, 45}, { 5,223}, { 8,188}, { 7,207}, { 9,153}, 
    { 5,127}, { 8,122}, { 7,125}, { 9, 97}, { 5,247}, { 8,236}, { 7,238}, { 9,216}, 
    { 4,255}, { 8,103}, { 7, 95}, { 9, 73}, { 5,239}, { 8,217}, { 7,222}, { 9,180}, 
    { 5,191}, { 8,171}, { 7,183}, { 9,133}, { 5,253}, { 9, 22}, { 7,249}, {11,144}, 
    { 4,255}, { 7,250}, { 5,254}, { 9, 23}, { 5,223}, { 8,173}, { 7,187}, { 9,134}, 
    { 5,127}, { 8,107}, { 7,111}, { 9, 74}, { 5,247}, { 8,218}, { 7,231}, { 9,184}, 
    { 4,255}, { 8, 61}, { 6,251}, { 9, 46}, { 5,239}, { 8,199}, { 7,215}, { 9,154}, 
    { 5,191}, { 8,124}, { 7,126}, { 9, 98}, { 5,253}, { 8,241}, { 7,243}, { 9,224}, 
    { 4,255}, { 8, 31}, { 5,254}, { 9, 37}, { 5,223}, { 8,182}, { 7,190}, { 9,145}, 
    { 5,127}, { 8,117}, { 7,123}, { 9, 84}, { 5,247}, { 8,230}, { 7,237}, { 9,201}, 
    { 4,255}, { 8, 91}, { 7, 63}, { 9, 57}, { 5,239}, { 8,211}, { 7,221}, { 9,168}, 
    { 5,191}, { 8,157}, { 7,175}, { 9,108}, { 5,253}, { 9,  7}, { 7,246}, {11,  6}, 
    { 4,255}, { 7,252}, { 5,254}, { 9, 28}, { 5,223}, { 8,179}, { 7,189}, { 9,139}, 
    { 5,127}, { 8,110}, { 7,119}, { 9, 78}, { 5,247}, { 8,227}, { 7,235}, { 9,196}, 
    { 4,255}, { 8, 79}, { 6,251}, { 9, 52}, { 5,239}, { 8,205}, { 7,219}, { 9,163}, 
    { 5,191}, { 8,151}, { 7,159}, { 9,102}, { 5,253}, { 8,244}, { 7,245}, { 9,232}, 
    { 4,255}, { 8, 55}, { 5,254}, { 9, 42}, { 5,223}, { 8,186}, { 7,207}, { 9,149}, 
    { 5,127}, { 8,121}, { 7,125}, { 9, 89}, { 5,247}, { 8,234}, { 7,238}, { 9,209}, 
    { 4,255}, { 8, 94}, { 7, 95}, { 9, 69}, { 5,239}, { 8,214}, { 7,222}, { 9,176}, 
    { 5,191}, { 8,167}, { 7,183}, { 9,116}, { 5,253}, { 9, 15}, { 7,249}, {11, 65}, 
    { 4,255}, { 7,250}, { 5,254}, { 9, 26}, { 5,223}, { 8,174}, { 7,187}, { 9,137}, 
    { 5,127}, { 8,109}, { 7,111}, { 9, 76}, { 5,247}, { 8,220}, { 7,231}, { 9,194}, 
    { 4,255}, { 8, 62}, { 6,251}, { 9, 50}, { 5,239}, { 8,203}, { 7,215}, { 9,161}, 
    { 5,191}, { 8,143}, { 7,126}, { 9,100}, { 5,253}, { 8,242}, { 7,243}, { 9,226}, 
    { 4,255}, { 8, 47}, { 5,254}, { 9, 39}, { 5,223}, { 8,185}, { 7,190}, { 9,147}, 
    { 5,127}, { 8,118}, { 7,123}, { 9, 86}, { 5,247}, { 8,233}, { 7,237}, { 9,204}, 
    { 4,255}, { 8, 93}, { 7, 63}, { 9, 60}, { 5,239}, { 8,213}, { 7,221}, { 9,170}, 
    { 5,191}, { 8,158}, { 7,175}, { 9,113}, { 5,253}, { 9, 13}, { 7,246}, {11, 24}, 
    { 4,255}, { 7,252}, { 5,254}, { 9, 30}, { 5,223}, { 8,181}, { 7,189}, { 9,141}, 
    { 5,127}, { 8,115}, { 7,119}, { 9, 82}, { 5,247}, { 8,229}, { 7,235}, { 9,198}, 
    { 4,255}, { 8, 87}, { 6,251}, { 9, 54}, { 5,239}, { 8,206}, { 7,219}, { 9,165}, 
    { 5,191}, { 8,155}, { 7,159}, { 9,105}, { 5,253}, { 8,248}, { 7,245}, {10, 18}, 
    { 4,255}, { 8, 59}, { 5,254}, { 9, 44}, { 5,223}, { 8,188}, { 7,207}, { 9,152}, 
    { 5,127}, { 8,122}, { 7,125}, { 9, 92}, { 5,247}, { 8,236}, { 7,238}, { 9,212}, 
    { 4,255}, { 8,103}, { 7, 95}, { 9, 71}, { 5,239}, { 8,217}, { 7,222}, { 9,178}, 
    { 5,191}, { 8,171}, { 7,183}, { 9,131}, { 5,253}, { 9, 21}, { 7,249}, {11,130}, 
    { 4,255}, { 7,250}, { 5,254}, { 9, 25}, { 5,223}, { 8,173}, { 7,187}, { 9,135}, 
    { 5,127}, { 8,107}, { 7,111}, { 9, 75}, { 5,247}, { 8,218}, { 7,231}, { 9,193}, 
    { 4,255}, { 8, 61}, { 6,251}, { 9, 49}, { 5,239}, { 8,199}, { 7,215}, { 9,156}, 
    { 5,191}, { 8,124}, { 7,126}, { 9, 99}, { 5,253}, { 8,241}, { 7,243}, { 9,225}, 
    { 4,255}, { 8, 31}, { 5,254}, { 9, 38}, { 5,223}, { 8,182}, { 7,190}, { 9,146}, 
    { 5,127}, { 8,117}, { 7,123}, { 9, 85}, { 5,247}, { 8,230}, { 7,237}, { 9,202}, 
    { 4,255}, { 8, 91}, { 7, 63}, { 9, 58}, { 5,239}, { 8,211}, { 7,221}, { 9,169}, 
    { 5,191}, { 8,157}, { 7,175}, { 9,112}, { 5,253}, { 9, 11}, { 7,246}, {11, 12}, 
    { 4,255}, { 7,252}, { 5,254}, { 9, 29}, { 5,223}, { 8,179}, { 7,189}, { 9,140}, 
    { 5,127}, { 8,110}, { 7,119}, { 9, 81}, { 5,247}, { 8,227}, { 7,235}, { 9,197}, 
    { 4,255}, { 8, 79}, { 6,251}, { 9, 53}, { 5,239}, { 8,205}, { 7,219}, { 9,164}, 
    { 5,191}, { 8,151}, { 7,159}, { 9,104}, { 5,253}, { 8,244}, { 7,245}, { 9,240}, 
    { 4,255}, { 8, 55}, { 5,254}, { 9, 43}, { 5,223}, { 8,186}, { 7,207}, { 9,150}, 
    { 5,127}, { 8,121}, { 7,125}, { 9, 90}, { 5,247}, { 8,234}, { 7,238}, { 9,210}, 
    { 4,255}, { 8, 94}, { 7, 95}, { 9, 70}, { 5,239}, { 8,214}, { 7,222}, { 9,177}, 
    { 5,191}, { 8,167}, { 7,183}, { 9,120}, { 5,253}, { 9, 19}, { 7,249}, {11, 80}, 
    { 4,255}, { 7,250}, { 5,254}, { 9, 27}, { 5,223}, { 8,174}, { 7,187}, { 9,138}, 
    { 5,127}, { 8,109}, { 7,111}, { 9, 77}, { 5,247}, { 8,220}, { 7,231}, { 9,195}, 
    { 4,255}, { 8, 62}, { 6,251}, { 9, 51}, { 5,239}, { 8,203}, { 7,215}, { 9,162}, 
    { 5,191}, { 8,143}, { 7,126}, { 9,101}, { 5,253}, { 8,242}, { 7,243}, { 9,228}, 
    { 4,255}, { 8, 47}, { 5,254}, { 9, 41}, { 5,223}, { 8,185}, { 7,190}, { 9,148}, 
    { 5,127}, { 8,118}, { 7,123}, { 9, 88}, { 5,247}, { 8,233}, { 7,237}, { 9,208}, 
    { 4,255}, { 8, 93}, { 7, 63}, { 9, 67}, { 5,239}, { 8,213}, { 7,221}, { 9,172}, 
    { 5,191}, { 8,158}, { 7,175}, { 9,114}, { 5,253}, { 9, 14}, { 7,246}, {11, 36}, 
    { 4,255}, { 7,252}, { 5,254}, { 9, 35}, { 5,223}, { 8,181}, { 7,189}, { 9,142}, 
    { 5,127}, { 8,115}, { 7,119}, { 9, 83}, { 5,247}, { 8,229}, { 7,235}, { 9,200}, 
    { 4,255}, { 8, 87}, { 6,251}, { 9, 56}, { 5,239}, { 8,206}, { 7,219}, { 9,166}, 
    { 5,191}, { 8,155}, { 7,159}, { 9,106}, { 5,253}, { 8,248}, { 7,245}, {11,  2}, 
    { 4,255}, { 8, 59}, { 5,254}, { 9, 45}, { 5,223}, { 8,188}, { 7,207}, { 9,153}, 
    { 5,127}, { 8,122}, { 7,125}, { 9, 97}, { 5,247}, { 8,236}, { 7,238}, { 9,216}, 
    { 4,255}, { 8,103}, { 7, 95}, { 9, 73}, { 5,239}, { 8,217}, { 7,222}, { 9,180}, 
    { 5,191}, { 8,171}, { 7,183}, { 9,133}, { 5,253}, { 9, 22}, { 7,249}, {11,192}, 
  }
  #endif
};

static radaudio_huffman rada_nonzero_bitflags_huff3 =
{
  #ifdef HUFFMAN_ENCODE
  {
    { 679,11}, {1703,11}, { 423,11}, {1447,11}, { 935,11}, {1959,11}, { 103,11}, {  39,10}, 
    {1127,11}, { 615,11}, {1639,11}, { 359,11}, {1383,11}, { 871,11}, {1895,11}, { 231,11}, 
    {1255,11}, { 743,11}, {1767,11}, { 487,11}, {1511,11}, { 999,11}, {2023,11}, {  23,11}, 
    {1047,11}, { 535,11}, {1559,11}, { 279,11}, {1303,11}, { 791,11}, {1815,11}, { 151,11}, 
    {1175,11}, { 663,11}, {1687,11}, { 407,11}, {1431,11}, { 919,11}, {1943,11}, {  87,11}, 
    {1111,11}, { 599,11}, {1623,11}, { 343,11}, {1367,11}, { 855,11}, {1879,11}, { 215,11}, 
    {1239,11}, { 727,11}, {1751,11}, { 471,11}, {1495,11}, { 983,11}, {2007,11}, {  55,11}, 
    {1079,11}, { 567,11}, {1591,11}, { 311,11}, {1335,11}, { 823,11}, {1847,11}, {  19, 8}, 
    { 183,11}, {1207,11}, { 695,11}, {1719,11}, { 439,11}, {1463,11}, { 951,11}, {1975,11}, 
    { 119,11}, {1143,11}, { 631,11}, {1655,11}, { 375,11}, {1399,11}, { 887,11}, {1911,11}, 
    { 247,11}, {1271,11}, { 759,11}, {1783,11}, { 503,11}, {1527,11}, {1015,11}, {2039,11}, 
    {  15,11}, {1039,11}, { 527,11}, {1551,11}, { 271,11}, {1295,11}, { 783,11}, { 147, 8}, 
    {1807,11}, { 143,11}, {1167,11}, { 655,11}, {1679,11}, { 399,11}, { 551,10}, {1423,11}, 
    { 911,11}, {1935,11}, {  79,11}, {1103,11}, { 591,11}, {1615,11}, { 295,10}, {  83, 8}, 
    { 335,11}, {1359,11}, { 847,11}, {1871,11}, { 207,11}, {1231,11}, { 719,11}, { 211, 8}, 
    {1743,11}, { 463,11}, {1487,11}, {  51, 8}, { 975,11}, { 179, 8}, { 115, 8}, {   9, 5}, 
    {1999,11}, {  47,11}, {1071,11}, { 559,11}, {1583,11}, { 303,11}, {1327,11}, { 815,11}, 
    {1839,11}, { 175,11}, {1199,11}, { 687,11}, {1711,11}, { 431,11}, {1455,11}, { 943,11}, 
    {1967,11}, { 111,11}, {1135,11}, { 623,11}, {1647,11}, { 367,11}, { 807,10}, {1391,11}, 
    { 879,11}, {1903,11}, { 239,11}, {1263,11}, { 751,11}, {1775,11}, { 495,11}, { 243, 8}, 
    { 167,10}, {1519,11}, {1007,11}, {2031,11}, {  31,11}, {1055,11}, { 543,11}, {1567,11}, 
    { 287,11}, {1311,11}, { 799,11}, {1823,11}, { 159,11}, {1183,11}, { 671,11}, {  11, 8}, 
    {1695,11}, { 415,11}, {1439,11}, { 927,11}, {1951,11}, {  95,11}, {1119,11}, { 139, 8}, 
    { 607,11}, {1631,11}, { 351,11}, {  75, 8}, {1375,11}, { 203, 8}, {  43, 8}, {  25, 5}, 
    { 863,11}, {1887,11}, { 223,11}, {1247,11}, { 735,11}, {1759,11}, { 479,11}, {1503,11}, 
    { 991,11}, {2015,11}, {  63,11}, {1087,11}, { 575,11}, {1599,11}, { 319,11}, { 171, 8}, 
    {1343,11}, { 831,11}, {1855,11}, { 191,11}, {1215,11}, { 703,11}, {1727,11}, { 107, 8}, 
    { 447,11}, {1471,11}, { 959,11}, { 235, 8}, {1983,11}, {  27, 8}, { 155, 8}, {   5, 5}, 
    { 127,11}, {1151,11}, { 639,11}, {1663,11}, { 383,11}, {1407,11}, { 895,11}, {  91, 8}, 
    {1919,11}, { 255,11}, {1279,11}, { 219, 8}, { 767,11}, {  59, 8}, { 187, 8}, {  21, 5}, 
    {1791,11}, { 511,11}, {1535,11}, { 123, 8}, {1023,11}, { 251, 8}, {   7, 8}, {   1, 4}, 
    {2047,11}, { 135, 8}, {  71, 8}, {  13, 5}, { 199, 8}, {  29, 5}, {   3, 5}, {   0, 1}, 
  },
  #endif
  #ifdef HUFFMAN_DECODE
  {
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, { 8,246}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,175}, { 1,255}, { 5,251}, { 1,255}, {11, 88}, 
    { 1,255}, { 4,247}, { 1,255}, { 8, 63}, { 1,255}, { 5,239}, { 1,255}, {11, 23}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,221}, { 1,255}, { 5,253}, { 1,255}, {11,164}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, {10,  7}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,190}, { 1,255}, { 5,251}, { 1,255}, {11,129}, 
    { 1,255}, { 4,247}, { 1,255}, { 8,123}, { 1,255}, { 5,239}, { 1,255}, {11, 55}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,237}, { 1,255}, { 5,253}, { 1,255}, {11,202}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, { 8,250}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,187}, { 1,255}, { 5,251}, { 1,255}, {11,106}, 
    { 1,255}, { 4,247}, { 1,255}, { 8,111}, { 1,255}, { 5,239}, { 1,255}, {11, 39}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,231}, { 1,255}, { 5,253}, { 1,255}, {11,181}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, {11,  6}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,215}, { 1,255}, { 5,251}, { 1,255}, {11,145}, 
    { 1,255}, { 4,247}, { 1,255}, { 8,126}, { 1,255}, { 5,239}, { 1,255}, {11, 72}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,243}, { 1,255}, { 5,253}, { 1,255}, {11,224}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, { 8,249}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,183}, { 1,255}, { 5,251}, { 1,255}, {11, 97}, 
    { 1,255}, { 4,247}, { 1,255}, { 8, 95}, { 1,255}, { 5,239}, { 1,255}, {11, 31}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,222}, { 1,255}, { 5,253}, { 1,255}, {11,172}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, {10,160}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,207}, { 1,255}, { 5,251}, { 1,255}, {11,137}, 
    { 1,255}, { 4,247}, { 1,255}, { 8,125}, { 1,255}, { 5,239}, { 1,255}, {11, 64}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,238}, { 1,255}, { 5,253}, { 1,255}, {11,211}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, { 8,252}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,189}, { 1,255}, { 5,251}, { 1,255}, {11,116}, 
    { 1,255}, { 4,247}, { 1,255}, { 8,119}, { 1,255}, { 5,239}, { 1,255}, {11, 47}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,235}, { 1,255}, { 5,253}, { 1,255}, {11,194}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, {11, 15}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,219}, { 1,255}, { 5,251}, { 1,255}, {11,154}, 
    { 1,255}, { 4,247}, { 1,255}, { 8,159}, { 1,255}, { 5,239}, { 1,255}, {11, 80}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,245}, { 1,255}, { 5,253}, { 1,255}, {11,233}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, { 8,246}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,175}, { 1,255}, { 5,251}, { 1,255}, {11, 92}, 
    { 1,255}, { 4,247}, { 1,255}, { 8, 63}, { 1,255}, { 5,239}, { 1,255}, {11, 27}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,221}, { 1,255}, { 5,253}, { 1,255}, {11,168}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, {10,110}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,190}, { 1,255}, { 5,251}, { 1,255}, {11,133}, 
    { 1,255}, { 4,247}, { 1,255}, { 8,123}, { 1,255}, { 5,239}, { 1,255}, {11, 59}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,237}, { 1,255}, { 5,253}, { 1,255}, {11,206}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, { 8,250}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,187}, { 1,255}, { 5,251}, { 1,255}, {11,112}, 
    { 1,255}, { 4,247}, { 1,255}, { 8,111}, { 1,255}, { 5,239}, { 1,255}, {11, 43}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,231}, { 1,255}, { 5,253}, { 1,255}, {11,186}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, {11, 11}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,215}, { 1,255}, { 5,251}, { 1,255}, {11,149}, 
    { 1,255}, { 4,247}, { 1,255}, { 8,126}, { 1,255}, { 5,239}, { 1,255}, {11, 76}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,243}, { 1,255}, { 5,253}, { 1,255}, {11,228}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, { 8,249}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,183}, { 1,255}, { 5,251}, { 1,255}, {11,101}, 
    { 1,255}, { 4,247}, { 1,255}, { 8, 95}, { 1,255}, { 5,239}, { 1,255}, {11, 35}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,222}, { 1,255}, { 5,253}, { 1,255}, {11,177}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, {11,  2}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,207}, { 1,255}, { 5,251}, { 1,255}, {11,141}, 
    { 1,255}, { 4,247}, { 1,255}, { 8,125}, { 1,255}, { 5,239}, { 1,255}, {11, 68}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,238}, { 1,255}, { 5,253}, { 1,255}, {11,216}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, { 8,252}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,189}, { 1,255}, { 5,251}, { 1,255}, {11,121}, 
    { 1,255}, { 4,247}, { 1,255}, { 8,119}, { 1,255}, { 5,239}, { 1,255}, {11, 51}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,235}, { 1,255}, { 5,253}, { 1,255}, {11,198}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, {11, 19}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,219}, { 1,255}, { 5,251}, { 1,255}, {11,158}, 
    { 1,255}, { 4,247}, { 1,255}, { 8,159}, { 1,255}, { 5,239}, { 1,255}, {11, 84}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,245}, { 1,255}, { 5,253}, { 1,255}, {11,241}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, { 8,246}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,175}, { 1,255}, { 5,251}, { 1,255}, {11, 90}, 
    { 1,255}, { 4,247}, { 1,255}, { 8, 63}, { 1,255}, { 5,239}, { 1,255}, {11, 25}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,221}, { 1,255}, { 5,253}, { 1,255}, {11,166}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, {10,102}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,190}, { 1,255}, { 5,251}, { 1,255}, {11,131}, 
    { 1,255}, { 4,247}, { 1,255}, { 8,123}, { 1,255}, { 5,239}, { 1,255}, {11, 57}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,237}, { 1,255}, { 5,253}, { 1,255}, {11,204}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, { 8,250}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,187}, { 1,255}, { 5,251}, { 1,255}, {11,108}, 
    { 1,255}, { 4,247}, { 1,255}, { 8,111}, { 1,255}, { 5,239}, { 1,255}, {11, 41}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,231}, { 1,255}, { 5,253}, { 1,255}, {11,184}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, {11,  9}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,215}, { 1,255}, { 5,251}, { 1,255}, {11,147}, 
    { 1,255}, { 4,247}, { 1,255}, { 8,126}, { 1,255}, { 5,239}, { 1,255}, {11, 74}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,243}, { 1,255}, { 5,253}, { 1,255}, {11,226}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, { 8,249}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,183}, { 1,255}, { 5,251}, { 1,255}, {11, 99}, 
    { 1,255}, { 4,247}, { 1,255}, { 8, 95}, { 1,255}, { 5,239}, { 1,255}, {11, 33}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,222}, { 1,255}, { 5,253}, { 1,255}, {11,174}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, {11,  0}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,207}, { 1,255}, { 5,251}, { 1,255}, {11,139}, 
    { 1,255}, { 4,247}, { 1,255}, { 8,125}, { 1,255}, { 5,239}, { 1,255}, {11, 66}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,238}, { 1,255}, { 5,253}, { 1,255}, {11,213}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, { 8,252}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,189}, { 1,255}, { 5,251}, { 1,255}, {11,118}, 
    { 1,255}, { 4,247}, { 1,255}, { 8,119}, { 1,255}, { 5,239}, { 1,255}, {11, 49}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,235}, { 1,255}, { 5,253}, { 1,255}, {11,196}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, {11, 17}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,219}, { 1,255}, { 5,251}, { 1,255}, {11,156}, 
    { 1,255}, { 4,247}, { 1,255}, { 8,159}, { 1,255}, { 5,239}, { 1,255}, {11, 82}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,245}, { 1,255}, { 5,253}, { 1,255}, {11,236}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, { 8,246}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,175}, { 1,255}, { 5,251}, { 1,255}, {11, 94}, 
    { 1,255}, { 4,247}, { 1,255}, { 8, 63}, { 1,255}, { 5,239}, { 1,255}, {11, 29}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,221}, { 1,255}, { 5,253}, { 1,255}, {11,170}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, {10,150}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,190}, { 1,255}, { 5,251}, { 1,255}, {11,135}, 
    { 1,255}, { 4,247}, { 1,255}, { 8,123}, { 1,255}, { 5,239}, { 1,255}, {11, 61}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,237}, { 1,255}, { 5,253}, { 1,255}, {11,209}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, { 8,250}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,187}, { 1,255}, { 5,251}, { 1,255}, {11,114}, 
    { 1,255}, { 4,247}, { 1,255}, { 8,111}, { 1,255}, { 5,239}, { 1,255}, {11, 45}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,231}, { 1,255}, { 5,253}, { 1,255}, {11,192}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, {11, 13}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,215}, { 1,255}, { 5,251}, { 1,255}, {11,152}, 
    { 1,255}, { 4,247}, { 1,255}, { 8,126}, { 1,255}, { 5,239}, { 1,255}, {11, 78}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,243}, { 1,255}, { 5,253}, { 1,255}, {11,230}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, { 8,249}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,183}, { 1,255}, { 5,251}, { 1,255}, {11,104}, 
    { 1,255}, { 4,247}, { 1,255}, { 8, 95}, { 1,255}, { 5,239}, { 1,255}, {11, 37}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,222}, { 1,255}, { 5,253}, { 1,255}, {11,179}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, {11,  4}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,207}, { 1,255}, { 5,251}, { 1,255}, {11,143}, 
    { 1,255}, { 4,247}, { 1,255}, { 8,125}, { 1,255}, { 5,239}, { 1,255}, {11, 70}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,238}, { 1,255}, { 5,253}, { 1,255}, {11,218}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, { 8,252}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,189}, { 1,255}, { 5,251}, { 1,255}, {11,124}, 
    { 1,255}, { 4,247}, { 1,255}, { 8,119}, { 1,255}, { 5,239}, { 1,255}, {11, 53}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,235}, { 1,255}, { 5,253}, { 1,255}, {11,200}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, {11, 21}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,219}, { 1,255}, { 5,251}, { 1,255}, {11,162}, 
    { 1,255}, { 4,247}, { 1,255}, { 8,159}, { 1,255}, { 5,239}, { 1,255}, {11, 86}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,245}, { 1,255}, { 5,253}, { 1,255}, {11,244}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, { 8,246}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,175}, { 1,255}, { 5,251}, { 1,255}, {11, 89}, 
    { 1,255}, { 4,247}, { 1,255}, { 8, 63}, { 1,255}, { 5,239}, { 1,255}, {11, 24}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,221}, { 1,255}, { 5,253}, { 1,255}, {11,165}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, {10,  7}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,190}, { 1,255}, { 5,251}, { 1,255}, {11,130}, 
    { 1,255}, { 4,247}, { 1,255}, { 8,123}, { 1,255}, { 5,239}, { 1,255}, {11, 56}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,237}, { 1,255}, { 5,253}, { 1,255}, {11,203}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, { 8,250}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,187}, { 1,255}, { 5,251}, { 1,255}, {11,107}, 
    { 1,255}, { 4,247}, { 1,255}, { 8,111}, { 1,255}, { 5,239}, { 1,255}, {11, 40}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,231}, { 1,255}, { 5,253}, { 1,255}, {11,182}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, {11,  8}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,215}, { 1,255}, { 5,251}, { 1,255}, {11,146}, 
    { 1,255}, { 4,247}, { 1,255}, { 8,126}, { 1,255}, { 5,239}, { 1,255}, {11, 73}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,243}, { 1,255}, { 5,253}, { 1,255}, {11,225}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, { 8,249}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,183}, { 1,255}, { 5,251}, { 1,255}, {11, 98}, 
    { 1,255}, { 4,247}, { 1,255}, { 8, 95}, { 1,255}, { 5,239}, { 1,255}, {11, 32}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,222}, { 1,255}, { 5,253}, { 1,255}, {11,173}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, {10,160}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,207}, { 1,255}, { 5,251}, { 1,255}, {11,138}, 
    { 1,255}, { 4,247}, { 1,255}, { 8,125}, { 1,255}, { 5,239}, { 1,255}, {11, 65}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,238}, { 1,255}, { 5,253}, { 1,255}, {11,212}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, { 8,252}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,189}, { 1,255}, { 5,251}, { 1,255}, {11,117}, 
    { 1,255}, { 4,247}, { 1,255}, { 8,119}, { 1,255}, { 5,239}, { 1,255}, {11, 48}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,235}, { 1,255}, { 5,253}, { 1,255}, {11,195}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, {11, 16}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,219}, { 1,255}, { 5,251}, { 1,255}, {11,155}, 
    { 1,255}, { 4,247}, { 1,255}, { 8,159}, { 1,255}, { 5,239}, { 1,255}, {11, 81}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,245}, { 1,255}, { 5,253}, { 1,255}, {11,234}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, { 8,246}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,175}, { 1,255}, { 5,251}, { 1,255}, {11, 93}, 
    { 1,255}, { 4,247}, { 1,255}, { 8, 63}, { 1,255}, { 5,239}, { 1,255}, {11, 28}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,221}, { 1,255}, { 5,253}, { 1,255}, {11,169}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, {10,110}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,190}, { 1,255}, { 5,251}, { 1,255}, {11,134}, 
    { 1,255}, { 4,247}, { 1,255}, { 8,123}, { 1,255}, { 5,239}, { 1,255}, {11, 60}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,237}, { 1,255}, { 5,253}, { 1,255}, {11,208}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, { 8,250}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,187}, { 1,255}, { 5,251}, { 1,255}, {11,113}, 
    { 1,255}, { 4,247}, { 1,255}, { 8,111}, { 1,255}, { 5,239}, { 1,255}, {11, 44}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,231}, { 1,255}, { 5,253}, { 1,255}, {11,188}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, {11, 12}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,215}, { 1,255}, { 5,251}, { 1,255}, {11,151}, 
    { 1,255}, { 4,247}, { 1,255}, { 8,126}, { 1,255}, { 5,239}, { 1,255}, {11, 77}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,243}, { 1,255}, { 5,253}, { 1,255}, {11,229}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, { 8,249}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,183}, { 1,255}, { 5,251}, { 1,255}, {11,103}, 
    { 1,255}, { 4,247}, { 1,255}, { 8, 95}, { 1,255}, { 5,239}, { 1,255}, {11, 36}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,222}, { 1,255}, { 5,253}, { 1,255}, {11,178}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, {11,  3}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,207}, { 1,255}, { 5,251}, { 1,255}, {11,142}, 
    { 1,255}, { 4,247}, { 1,255}, { 8,125}, { 1,255}, { 5,239}, { 1,255}, {11, 69}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,238}, { 1,255}, { 5,253}, { 1,255}, {11,217}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, { 8,252}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,189}, { 1,255}, { 5,251}, { 1,255}, {11,122}, 
    { 1,255}, { 4,247}, { 1,255}, { 8,119}, { 1,255}, { 5,239}, { 1,255}, {11, 52}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,235}, { 1,255}, { 5,253}, { 1,255}, {11,199}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, {11, 20}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,219}, { 1,255}, { 5,251}, { 1,255}, {11,161}, 
    { 1,255}, { 4,247}, { 1,255}, { 8,159}, { 1,255}, { 5,239}, { 1,255}, {11, 85}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,245}, { 1,255}, { 5,253}, { 1,255}, {11,242}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, { 8,246}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,175}, { 1,255}, { 5,251}, { 1,255}, {11, 91}, 
    { 1,255}, { 4,247}, { 1,255}, { 8, 63}, { 1,255}, { 5,239}, { 1,255}, {11, 26}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,221}, { 1,255}, { 5,253}, { 1,255}, {11,167}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, {10,102}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,190}, { 1,255}, { 5,251}, { 1,255}, {11,132}, 
    { 1,255}, { 4,247}, { 1,255}, { 8,123}, { 1,255}, { 5,239}, { 1,255}, {11, 58}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,237}, { 1,255}, { 5,253}, { 1,255}, {11,205}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, { 8,250}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,187}, { 1,255}, { 5,251}, { 1,255}, {11,109}, 
    { 1,255}, { 4,247}, { 1,255}, { 8,111}, { 1,255}, { 5,239}, { 1,255}, {11, 42}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,231}, { 1,255}, { 5,253}, { 1,255}, {11,185}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, {11, 10}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,215}, { 1,255}, { 5,251}, { 1,255}, {11,148}, 
    { 1,255}, { 4,247}, { 1,255}, { 8,126}, { 1,255}, { 5,239}, { 1,255}, {11, 75}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,243}, { 1,255}, { 5,253}, { 1,255}, {11,227}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, { 8,249}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,183}, { 1,255}, { 5,251}, { 1,255}, {11,100}, 
    { 1,255}, { 4,247}, { 1,255}, { 8, 95}, { 1,255}, { 5,239}, { 1,255}, {11, 34}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,222}, { 1,255}, { 5,253}, { 1,255}, {11,176}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, {11,  1}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,207}, { 1,255}, { 5,251}, { 1,255}, {11,140}, 
    { 1,255}, { 4,247}, { 1,255}, { 8,125}, { 1,255}, { 5,239}, { 1,255}, {11, 67}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,238}, { 1,255}, { 5,253}, { 1,255}, {11,214}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, { 8,252}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,189}, { 1,255}, { 5,251}, { 1,255}, {11,120}, 
    { 1,255}, { 4,247}, { 1,255}, { 8,119}, { 1,255}, { 5,239}, { 1,255}, {11, 50}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,235}, { 1,255}, { 5,253}, { 1,255}, {11,197}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, {11, 18}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,219}, { 1,255}, { 5,251}, { 1,255}, {11,157}, 
    { 1,255}, { 4,247}, { 1,255}, { 8,159}, { 1,255}, { 5,239}, { 1,255}, {11, 83}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,245}, { 1,255}, { 5,253}, { 1,255}, {11,240}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, { 8,246}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,175}, { 1,255}, { 5,251}, { 1,255}, {11, 96}, 
    { 1,255}, { 4,247}, { 1,255}, { 8, 63}, { 1,255}, { 5,239}, { 1,255}, {11, 30}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,221}, { 1,255}, { 5,253}, { 1,255}, {11,171}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, {10,150}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,190}, { 1,255}, { 5,251}, { 1,255}, {11,136}, 
    { 1,255}, { 4,247}, { 1,255}, { 8,123}, { 1,255}, { 5,239}, { 1,255}, {11, 62}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,237}, { 1,255}, { 5,253}, { 1,255}, {11,210}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, { 8,250}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,187}, { 1,255}, { 5,251}, { 1,255}, {11,115}, 
    { 1,255}, { 4,247}, { 1,255}, { 8,111}, { 1,255}, { 5,239}, { 1,255}, {11, 46}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,231}, { 1,255}, { 5,253}, { 1,255}, {11,193}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, {11, 14}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,215}, { 1,255}, { 5,251}, { 1,255}, {11,153}, 
    { 1,255}, { 4,247}, { 1,255}, { 8,126}, { 1,255}, { 5,239}, { 1,255}, {11, 79}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,243}, { 1,255}, { 5,253}, { 1,255}, {11,232}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, { 8,249}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,183}, { 1,255}, { 5,251}, { 1,255}, {11,105}, 
    { 1,255}, { 4,247}, { 1,255}, { 8, 95}, { 1,255}, { 5,239}, { 1,255}, {11, 38}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,222}, { 1,255}, { 5,253}, { 1,255}, {11,180}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, {11,  5}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,207}, { 1,255}, { 5,251}, { 1,255}, {11,144}, 
    { 1,255}, { 4,247}, { 1,255}, { 8,125}, { 1,255}, { 5,239}, { 1,255}, {11, 71}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,238}, { 1,255}, { 5,253}, { 1,255}, {11,220}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, { 8,252}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,189}, { 1,255}, { 5,251}, { 1,255}, {11,128}, 
    { 1,255}, { 4,247}, { 1,255}, { 8,119}, { 1,255}, { 5,239}, { 1,255}, {11, 54}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,235}, { 1,255}, { 5,253}, { 1,255}, {11,201}, 
    { 1,255}, { 4,247}, { 1,255}, { 5,254}, { 1,255}, { 5,223}, { 1,255}, {11, 22}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,219}, { 1,255}, { 5,251}, { 1,255}, {11,163}, 
    { 1,255}, { 4,247}, { 1,255}, { 8,159}, { 1,255}, { 5,239}, { 1,255}, {11, 87}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,245}, { 1,255}, { 5,253}, { 1,255}, {11,248}, 
  }
  #endif
};

static radaudio_huffman rada_nonzero_bitflags_huff4 =
{
  #ifdef HUFFMAN_ENCODE
  {
    { 983,11}, {2007,11}, {  55,11}, {1079,11}, { 567,11}, {1591,11}, { 311,11}, {1335,11}, 
    { 823,11}, {1847,11}, { 183,11}, {1207,11}, { 695,11}, {1719,11}, { 439,11}, {1463,11}, 
    { 951,11}, {1975,11}, { 119,11}, {1143,11}, { 631,11}, {1655,11}, { 375,11}, {1399,11}, 
    { 887,11}, {1911,11}, { 247,11}, {1271,11}, { 759,11}, {1783,11}, { 503,11}, { 251,10}, 
    {1527,11}, {1015,11}, {2039,11}, {  15,11}, {1039,11}, { 527,11}, {1551,11}, { 271,11}, 
    {1295,11}, { 783,11}, {1807,11}, { 143,11}, {1167,11}, { 655,11}, {1679,11}, { 763,10}, 
    { 399,11}, {1423,11}, { 911,11}, {1935,11}, {  79,11}, {1103,11}, { 591,11}, { 507,10}, 
    {1615,11}, { 335,11}, {1359,11}, {1019,10}, { 847,11}, {   7,10}, { 519,10}, {   3, 8}, 
    {1871,11}, { 207,11}, {1231,11}, { 719,11}, {1743,11}, { 463,11}, {1487,11}, { 975,11}, 
    {1999,11}, {  47,11}, {1071,11}, { 559,11}, {1583,11}, { 303,11}, {1327,11}, { 263,10}, 
    { 815,11}, {1839,11}, { 175,11}, {1199,11}, { 687,11}, {1711,11}, { 431,11}, { 775,10}, 
    {1455,11}, { 943,11}, {1967,11}, { 135,10}, { 111,11}, { 647,10}, { 391,10}, { 131, 8}, 
    {1135,11}, { 623,11}, {1647,11}, { 367,11}, {1391,11}, { 879,11}, {1903,11}, { 903,10}, 
    { 239,11}, {1263,11}, { 751,11}, {  71,10}, {1775,11}, { 583,10}, { 327,10}, {  67, 8}, 
    { 495,11}, {1519,11}, {1007,11}, { 839,10}, {2031,11}, { 199,10}, {  59, 9}, { 195, 8}, 
    {  31,11}, { 711,10}, { 455,10}, {  35, 8}, { 967,10}, { 163, 8}, {  99, 8}, {   1, 5}, 
    {1055,11}, { 543,11}, {1567,11}, { 287,11}, {1311,11}, { 799,11}, {1823,11}, { 159,11}, 
    {1183,11}, { 671,11}, {1695,11}, { 415,11}, {1439,11}, { 927,11}, {1951,11}, {  39,10}, 
    {  95,11}, {1119,11}, { 607,11}, {1631,11}, { 351,11}, {1375,11}, { 863,11}, { 551,10}, 
    { 295,10}, {1887,11}, { 223,11}, { 807,10}, {1247,11}, { 167,10}, { 679,10}, { 227, 8}, 
    { 735,11}, {1759,11}, { 479,11}, {1503,11}, { 991,11}, {2015,11}, {  63,11}, { 423,10}, 
    {1087,11}, { 575,11}, {1599,11}, { 935,10}, { 319,11}, { 103,10}, { 615,10}, {  19, 8}, 
    {1343,11}, { 831,11}, {1855,11}, { 359,10}, { 191,11}, { 871,10}, { 315, 9}, { 147, 8}, 
    {1215,11}, { 231,10}, { 743,10}, {  83, 8}, { 487,10}, { 211, 8}, {  51, 8}, {  17, 5}, 
    { 703,11}, {1727,11}, { 447,11}, {1471,11}, { 959,11}, {1983,11}, { 127,11}, { 999,10}, 
    {1151,11}, { 639,11}, {1663,11}, {  23,10}, { 383,11}, { 535,10}, { 187, 9}, { 179, 8}, 
    {1407,11}, { 895,11}, {1919,11}, { 279,10}, { 255,11}, { 791,10}, { 151,10}, { 115, 8}, 
    {1279,11}, { 663,10}, { 407,10}, { 243, 8}, { 919,10}, {  11, 8}, { 139, 8}, {   9, 5}, 
    { 767,11}, {1791,11}, { 511,11}, {  87,10}, {1535,11}, { 599,10}, { 343,10}, {  75, 8}, 
    {1023,11}, { 443, 9}, { 855,10}, { 203, 8}, { 215,10}, {  43, 8}, { 171, 8}, {  25, 5}, 
    {2047,11}, { 123, 9}, { 727,10}, { 107, 8}, { 471,10}, { 235, 8}, {  27, 8}, {   5, 5}, 
    { 379, 9}, { 155, 8}, {  91, 8}, {  21, 5}, { 219, 8}, {  13, 5}, {  29, 5}, {   0, 1}, 
  },
  #endif
  #ifdef HUFFMAN_DECODE
  {
    { 1,255}, { 5,127}, { 1,255}, { 8, 63}, { 1,255}, { 5,247}, { 1,255}, {10, 61}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,221}, { 1,255}, { 5,253}, { 1,255}, {11, 35}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,175}, { 1,255}, { 5,251}, { 1,255}, {10,203}, 
    { 1,255}, { 5,239}, { 1,255}, { 8,246}, { 1,255}, { 5,254}, { 1,255}, {11,120}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,123}, { 1,255}, { 5,247}, { 1,255}, {10,143}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,237}, { 1,255}, { 5,253}, { 1,255}, {11, 73}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,190}, { 1,255}, { 5,251}, { 1,255}, {11,  2}, 
    { 1,255}, { 5,239}, { 1,255}, { 9,118}, { 1,255}, { 5,254}, { 1,255}, {11,166}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,111}, { 1,255}, { 5,247}, { 1,255}, {10,107}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,231}, { 1,255}, { 5,253}, { 1,255}, {11, 52}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,187}, { 1,255}, { 5,251}, { 1,255}, {10,227}, 
    { 1,255}, { 5,239}, { 1,255}, { 8,250}, { 1,255}, { 5,254}, { 1,255}, {11,144}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,126}, { 1,255}, { 5,247}, { 1,255}, {10,173}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,243}, { 1,255}, { 5,253}, { 1,255}, {11, 92}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,215}, { 1,255}, { 5,251}, { 1,255}, {11, 18}, 
    { 1,255}, { 5,239}, { 1,255}, { 9,241}, { 1,255}, { 5,254}, { 1,255}, {11,198}, 
    { 1,255}, { 5,127}, { 1,255}, { 8, 95}, { 1,255}, { 5,247}, { 1,255}, {10, 91}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,222}, { 1,255}, { 5,253}, { 1,255}, {11, 43}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,183}, { 1,255}, { 5,251}, { 1,255}, {10,214}, 
    { 1,255}, { 5,239}, { 1,255}, { 8,249}, { 1,255}, { 5,254}, { 1,255}, {11,135}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,125}, { 1,255}, { 5,247}, { 1,255}, {10,157}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,238}, { 1,255}, { 5,253}, { 1,255}, {11, 82}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,207}, { 1,255}, { 5,251}, { 1,255}, {11, 10}, 
    { 1,255}, { 5,239}, { 1,255}, { 9,206}, { 1,255}, { 5,254}, { 1,255}, {11,180}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,119}, { 1,255}, { 5,247}, { 1,255}, {10,117}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,235}, { 1,255}, { 5,253}, { 1,255}, {11, 65}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,189}, { 1,255}, { 5,251}, { 1,255}, {10,236}, 
    { 1,255}, { 5,239}, { 1,255}, { 8,252}, { 1,255}, { 5,254}, { 1,255}, {11,154}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,159}, { 1,255}, { 5,247}, { 1,255}, {10,185}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,245}, { 1,255}, { 5,253}, { 1,255}, {11,104}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,219}, { 1,255}, { 5,251}, { 1,255}, {11, 26}, 
    { 1,255}, { 5,239}, { 1,255}, {10, 31}, { 1,255}, { 5,254}, { 1,255}, {11,212}, 
    { 1,255}, { 5,127}, { 1,255}, { 8, 63}, { 1,255}, { 5,247}, { 1,255}, {10, 79}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,221}, { 1,255}, { 5,253}, { 1,255}, {11, 39}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,175}, { 1,255}, { 5,251}, { 1,255}, {10,211}, 
    { 1,255}, { 5,239}, { 1,255}, { 8,246}, { 1,255}, { 5,254}, { 1,255}, {11,131}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,123}, { 1,255}, { 5,247}, { 1,255}, {10,152}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,237}, { 1,255}, { 5,253}, { 1,255}, {11, 77}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,190}, { 1,255}, { 5,251}, { 1,255}, {11,  6}, 
    { 1,255}, { 5,239}, { 1,255}, { 9,182}, { 1,255}, { 5,254}, { 1,255}, {11,172}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,111}, { 1,255}, { 5,247}, { 1,255}, {10,110}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,231}, { 1,255}, { 5,253}, { 1,255}, {11, 57}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,187}, { 1,255}, { 5,251}, { 1,255}, {10,230}, 
    { 1,255}, { 5,239}, { 1,255}, { 8,250}, { 1,255}, { 5,254}, { 1,255}, {11,148}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,126}, { 1,255}, { 5,247}, { 1,255}, {10,179}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,243}, { 1,255}, { 5,253}, { 1,255}, {11, 99}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,215}, { 1,255}, { 5,251}, { 1,255}, {11, 22}, 
    { 1,255}, { 5,239}, { 1,255}, { 9,248}, { 1,255}, { 5,254}, { 1,255}, {11,204}, 
    { 1,255}, { 5,127}, { 1,255}, { 8, 95}, { 1,255}, { 5,247}, { 1,255}, {10, 94}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,222}, { 1,255}, { 5,253}, { 1,255}, {11, 48}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,183}, { 1,255}, { 5,251}, { 1,255}, {10,218}, 
    { 1,255}, { 5,239}, { 1,255}, { 8,249}, { 1,255}, { 5,254}, { 1,255}, {11,139}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,125}, { 1,255}, { 5,247}, { 1,255}, {10,167}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,238}, { 1,255}, { 5,253}, { 1,255}, {11, 86}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,207}, { 1,255}, { 5,251}, { 1,255}, {11, 14}, 
    { 1,255}, { 5,239}, { 1,255}, { 9,233}, { 1,255}, { 5,254}, { 1,255}, {11,194}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,119}, { 1,255}, { 5,247}, { 1,255}, {10,122}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,235}, { 1,255}, { 5,253}, { 1,255}, {11, 69}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,189}, { 1,255}, { 5,251}, { 1,255}, {10,244}, 
    { 1,255}, { 5,239}, { 1,255}, { 8,252}, { 1,255}, { 5,254}, { 1,255}, {11,162}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,159}, { 1,255}, { 5,247}, { 1,255}, {10,188}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,245}, { 1,255}, { 5,253}, { 1,255}, {11,112}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,219}, { 1,255}, { 5,251}, { 1,255}, {11, 30}, 
    { 1,255}, { 5,239}, { 1,255}, {10, 55}, { 1,255}, { 5,254}, { 1,255}, {11,226}, 
    { 1,255}, { 5,127}, { 1,255}, { 8, 63}, { 1,255}, { 5,247}, { 1,255}, {10, 62}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,221}, { 1,255}, { 5,253}, { 1,255}, {11, 37}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,175}, { 1,255}, { 5,251}, { 1,255}, {10,205}, 
    { 1,255}, { 5,239}, { 1,255}, { 8,246}, { 1,255}, { 5,254}, { 1,255}, {11,129}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,123}, { 1,255}, { 5,247}, { 1,255}, {10,151}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,237}, { 1,255}, { 5,253}, { 1,255}, {11, 75}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,190}, { 1,255}, { 5,251}, { 1,255}, {11,  4}, 
    { 1,255}, { 5,239}, { 1,255}, { 9,118}, { 1,255}, { 5,254}, { 1,255}, {11,169}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,111}, { 1,255}, { 5,247}, { 1,255}, {10,109}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,231}, { 1,255}, { 5,253}, { 1,255}, {11, 54}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,187}, { 1,255}, { 5,251}, { 1,255}, {10,229}, 
    { 1,255}, { 5,239}, { 1,255}, { 8,250}, { 1,255}, { 5,254}, { 1,255}, {11,146}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,126}, { 1,255}, { 5,247}, { 1,255}, {10,174}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,243}, { 1,255}, { 5,253}, { 1,255}, {11, 97}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,215}, { 1,255}, { 5,251}, { 1,255}, {11, 20}, 
    { 1,255}, { 5,239}, { 1,255}, { 9,241}, { 1,255}, { 5,254}, { 1,255}, {11,201}, 
    { 1,255}, { 5,127}, { 1,255}, { 8, 95}, { 1,255}, { 5,247}, { 1,255}, {10, 93}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,222}, { 1,255}, { 5,253}, { 1,255}, {11, 45}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,183}, { 1,255}, { 5,251}, { 1,255}, {10,217}, 
    { 1,255}, { 5,239}, { 1,255}, { 8,249}, { 1,255}, { 5,254}, { 1,255}, {11,137}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,125}, { 1,255}, { 5,247}, { 1,255}, {10,158}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,238}, { 1,255}, { 5,253}, { 1,255}, {11, 84}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,207}, { 1,255}, { 5,251}, { 1,255}, {11, 12}, 
    { 1,255}, { 5,239}, { 1,255}, { 9,206}, { 1,255}, { 5,254}, { 1,255}, {11,192}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,119}, { 1,255}, { 5,247}, { 1,255}, {10,121}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,235}, { 1,255}, { 5,253}, { 1,255}, {11, 67}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,189}, { 1,255}, { 5,251}, { 1,255}, {10,242}, 
    { 1,255}, { 5,239}, { 1,255}, { 8,252}, { 1,255}, { 5,254}, { 1,255}, {11,160}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,159}, { 1,255}, { 5,247}, { 1,255}, {10,186}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,245}, { 1,255}, { 5,253}, { 1,255}, {11,106}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,219}, { 1,255}, { 5,251}, { 1,255}, {11, 28}, 
    { 1,255}, { 5,239}, { 1,255}, {10, 47}, { 1,255}, { 5,254}, { 1,255}, {11,224}, 
    { 1,255}, { 5,127}, { 1,255}, { 8, 63}, { 1,255}, { 5,247}, { 1,255}, {10, 87}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,221}, { 1,255}, { 5,253}, { 1,255}, {11, 41}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,175}, { 1,255}, { 5,251}, { 1,255}, {10,213}, 
    { 1,255}, { 5,239}, { 1,255}, { 8,246}, { 1,255}, { 5,254}, { 1,255}, {11,133}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,123}, { 1,255}, { 5,247}, { 1,255}, {10,155}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,237}, { 1,255}, { 5,253}, { 1,255}, {11, 80}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,190}, { 1,255}, { 5,251}, { 1,255}, {11,  8}, 
    { 1,255}, { 5,239}, { 1,255}, { 9,182}, { 1,255}, { 5,254}, { 1,255}, {11,177}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,111}, { 1,255}, { 5,247}, { 1,255}, {10,115}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,231}, { 1,255}, { 5,253}, { 1,255}, {11, 60}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,187}, { 1,255}, { 5,251}, { 1,255}, {10,234}, 
    { 1,255}, { 5,239}, { 1,255}, { 8,250}, { 1,255}, { 5,254}, { 1,255}, {11,150}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,126}, { 1,255}, { 5,247}, { 1,255}, {10,181}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,243}, { 1,255}, { 5,253}, { 1,255}, {11,101}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,215}, { 1,255}, { 5,251}, { 1,255}, {11, 24}, 
    { 1,255}, { 5,239}, { 1,255}, { 9,248}, { 1,255}, { 5,254}, { 1,255}, {11,209}, 
    { 1,255}, { 5,127}, { 1,255}, { 8, 95}, { 1,255}, { 5,247}, { 1,255}, {10,103}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,222}, { 1,255}, { 5,253}, { 1,255}, {11, 50}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,183}, { 1,255}, { 5,251}, { 1,255}, {10,220}, 
    { 1,255}, { 5,239}, { 1,255}, { 8,249}, { 1,255}, { 5,254}, { 1,255}, {11,141}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,125}, { 1,255}, { 5,247}, { 1,255}, {10,171}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,238}, { 1,255}, { 5,253}, { 1,255}, {11, 89}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,207}, { 1,255}, { 5,251}, { 1,255}, {11, 16}, 
    { 1,255}, { 5,239}, { 1,255}, { 9,233}, { 1,255}, { 5,254}, { 1,255}, {11,196}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,119}, { 1,255}, { 5,247}, { 1,255}, {10,124}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,235}, { 1,255}, { 5,253}, { 1,255}, {11, 71}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,189}, { 1,255}, { 5,251}, { 1,255}, {11,  0}, 
    { 1,255}, { 5,239}, { 1,255}, { 8,252}, { 1,255}, { 5,254}, { 1,255}, {11,164}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,159}, { 1,255}, { 5,247}, { 1,255}, {10,199}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,245}, { 1,255}, { 5,253}, { 1,255}, {11,114}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,219}, { 1,255}, { 5,251}, { 1,255}, {11, 33}, 
    { 1,255}, { 5,239}, { 1,255}, {10, 59}, { 1,255}, { 5,254}, { 1,255}, {11,232}, 
    { 1,255}, { 5,127}, { 1,255}, { 8, 63}, { 1,255}, { 5,247}, { 1,255}, {10, 61}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,221}, { 1,255}, { 5,253}, { 1,255}, {11, 36}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,175}, { 1,255}, { 5,251}, { 1,255}, {10,203}, 
    { 1,255}, { 5,239}, { 1,255}, { 8,246}, { 1,255}, { 5,254}, { 1,255}, {11,128}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,123}, { 1,255}, { 5,247}, { 1,255}, {10,143}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,237}, { 1,255}, { 5,253}, { 1,255}, {11, 74}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,190}, { 1,255}, { 5,251}, { 1,255}, {11,  3}, 
    { 1,255}, { 5,239}, { 1,255}, { 9,118}, { 1,255}, { 5,254}, { 1,255}, {11,168}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,111}, { 1,255}, { 5,247}, { 1,255}, {10,107}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,231}, { 1,255}, { 5,253}, { 1,255}, {11, 53}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,187}, { 1,255}, { 5,251}, { 1,255}, {10,227}, 
    { 1,255}, { 5,239}, { 1,255}, { 8,250}, { 1,255}, { 5,254}, { 1,255}, {11,145}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,126}, { 1,255}, { 5,247}, { 1,255}, {10,173}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,243}, { 1,255}, { 5,253}, { 1,255}, {11, 96}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,215}, { 1,255}, { 5,251}, { 1,255}, {11, 19}, 
    { 1,255}, { 5,239}, { 1,255}, { 9,241}, { 1,255}, { 5,254}, { 1,255}, {11,200}, 
    { 1,255}, { 5,127}, { 1,255}, { 8, 95}, { 1,255}, { 5,247}, { 1,255}, {10, 91}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,222}, { 1,255}, { 5,253}, { 1,255}, {11, 44}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,183}, { 1,255}, { 5,251}, { 1,255}, {10,214}, 
    { 1,255}, { 5,239}, { 1,255}, { 8,249}, { 1,255}, { 5,254}, { 1,255}, {11,136}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,125}, { 1,255}, { 5,247}, { 1,255}, {10,157}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,238}, { 1,255}, { 5,253}, { 1,255}, {11, 83}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,207}, { 1,255}, { 5,251}, { 1,255}, {11, 11}, 
    { 1,255}, { 5,239}, { 1,255}, { 9,206}, { 1,255}, { 5,254}, { 1,255}, {11,184}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,119}, { 1,255}, { 5,247}, { 1,255}, {10,117}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,235}, { 1,255}, { 5,253}, { 1,255}, {11, 66}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,189}, { 1,255}, { 5,251}, { 1,255}, {10,236}, 
    { 1,255}, { 5,239}, { 1,255}, { 8,252}, { 1,255}, { 5,254}, { 1,255}, {11,156}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,159}, { 1,255}, { 5,247}, { 1,255}, {10,185}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,245}, { 1,255}, { 5,253}, { 1,255}, {11,105}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,219}, { 1,255}, { 5,251}, { 1,255}, {11, 27}, 
    { 1,255}, { 5,239}, { 1,255}, {10, 31}, { 1,255}, { 5,254}, { 1,255}, {11,216}, 
    { 1,255}, { 5,127}, { 1,255}, { 8, 63}, { 1,255}, { 5,247}, { 1,255}, {10, 79}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,221}, { 1,255}, { 5,253}, { 1,255}, {11, 40}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,175}, { 1,255}, { 5,251}, { 1,255}, {10,211}, 
    { 1,255}, { 5,239}, { 1,255}, { 8,246}, { 1,255}, { 5,254}, { 1,255}, {11,132}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,123}, { 1,255}, { 5,247}, { 1,255}, {10,152}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,237}, { 1,255}, { 5,253}, { 1,255}, {11, 78}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,190}, { 1,255}, { 5,251}, { 1,255}, {11,  7}, 
    { 1,255}, { 5,239}, { 1,255}, { 9,182}, { 1,255}, { 5,254}, { 1,255}, {11,176}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,111}, { 1,255}, { 5,247}, { 1,255}, {10,110}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,231}, { 1,255}, { 5,253}, { 1,255}, {11, 58}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,187}, { 1,255}, { 5,251}, { 1,255}, {10,230}, 
    { 1,255}, { 5,239}, { 1,255}, { 8,250}, { 1,255}, { 5,254}, { 1,255}, {11,149}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,126}, { 1,255}, { 5,247}, { 1,255}, {10,179}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,243}, { 1,255}, { 5,253}, { 1,255}, {11,100}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,215}, { 1,255}, { 5,251}, { 1,255}, {11, 23}, 
    { 1,255}, { 5,239}, { 1,255}, { 9,248}, { 1,255}, { 5,254}, { 1,255}, {11,208}, 
    { 1,255}, { 5,127}, { 1,255}, { 8, 95}, { 1,255}, { 5,247}, { 1,255}, {10, 94}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,222}, { 1,255}, { 5,253}, { 1,255}, {11, 49}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,183}, { 1,255}, { 5,251}, { 1,255}, {10,218}, 
    { 1,255}, { 5,239}, { 1,255}, { 8,249}, { 1,255}, { 5,254}, { 1,255}, {11,140}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,125}, { 1,255}, { 5,247}, { 1,255}, {10,167}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,238}, { 1,255}, { 5,253}, { 1,255}, {11, 88}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,207}, { 1,255}, { 5,251}, { 1,255}, {11, 15}, 
    { 1,255}, { 5,239}, { 1,255}, { 9,233}, { 1,255}, { 5,254}, { 1,255}, {11,195}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,119}, { 1,255}, { 5,247}, { 1,255}, {10,122}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,235}, { 1,255}, { 5,253}, { 1,255}, {11, 70}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,189}, { 1,255}, { 5,251}, { 1,255}, {10,244}, 
    { 1,255}, { 5,239}, { 1,255}, { 8,252}, { 1,255}, { 5,254}, { 1,255}, {11,163}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,159}, { 1,255}, { 5,247}, { 1,255}, {10,188}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,245}, { 1,255}, { 5,253}, { 1,255}, {11,113}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,219}, { 1,255}, { 5,251}, { 1,255}, {11, 32}, 
    { 1,255}, { 5,239}, { 1,255}, {10, 55}, { 1,255}, { 5,254}, { 1,255}, {11,228}, 
    { 1,255}, { 5,127}, { 1,255}, { 8, 63}, { 1,255}, { 5,247}, { 1,255}, {10, 62}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,221}, { 1,255}, { 5,253}, { 1,255}, {11, 38}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,175}, { 1,255}, { 5,251}, { 1,255}, {10,205}, 
    { 1,255}, { 5,239}, { 1,255}, { 8,246}, { 1,255}, { 5,254}, { 1,255}, {11,130}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,123}, { 1,255}, { 5,247}, { 1,255}, {10,151}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,237}, { 1,255}, { 5,253}, { 1,255}, {11, 76}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,190}, { 1,255}, { 5,251}, { 1,255}, {11,  5}, 
    { 1,255}, { 5,239}, { 1,255}, { 9,118}, { 1,255}, { 5,254}, { 1,255}, {11,170}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,111}, { 1,255}, { 5,247}, { 1,255}, {10,109}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,231}, { 1,255}, { 5,253}, { 1,255}, {11, 56}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,187}, { 1,255}, { 5,251}, { 1,255}, {10,229}, 
    { 1,255}, { 5,239}, { 1,255}, { 8,250}, { 1,255}, { 5,254}, { 1,255}, {11,147}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,126}, { 1,255}, { 5,247}, { 1,255}, {10,174}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,243}, { 1,255}, { 5,253}, { 1,255}, {11, 98}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,215}, { 1,255}, { 5,251}, { 1,255}, {11, 21}, 
    { 1,255}, { 5,239}, { 1,255}, { 9,241}, { 1,255}, { 5,254}, { 1,255}, {11,202}, 
    { 1,255}, { 5,127}, { 1,255}, { 8, 95}, { 1,255}, { 5,247}, { 1,255}, {10, 93}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,222}, { 1,255}, { 5,253}, { 1,255}, {11, 46}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,183}, { 1,255}, { 5,251}, { 1,255}, {10,217}, 
    { 1,255}, { 5,239}, { 1,255}, { 8,249}, { 1,255}, { 5,254}, { 1,255}, {11,138}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,125}, { 1,255}, { 5,247}, { 1,255}, {10,158}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,238}, { 1,255}, { 5,253}, { 1,255}, {11, 85}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,207}, { 1,255}, { 5,251}, { 1,255}, {11, 13}, 
    { 1,255}, { 5,239}, { 1,255}, { 9,206}, { 1,255}, { 5,254}, { 1,255}, {11,193}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,119}, { 1,255}, { 5,247}, { 1,255}, {10,121}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,235}, { 1,255}, { 5,253}, { 1,255}, {11, 68}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,189}, { 1,255}, { 5,251}, { 1,255}, {10,242}, 
    { 1,255}, { 5,239}, { 1,255}, { 8,252}, { 1,255}, { 5,254}, { 1,255}, {11,161}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,159}, { 1,255}, { 5,247}, { 1,255}, {10,186}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,245}, { 1,255}, { 5,253}, { 1,255}, {11,108}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,219}, { 1,255}, { 5,251}, { 1,255}, {11, 29}, 
    { 1,255}, { 5,239}, { 1,255}, {10, 47}, { 1,255}, { 5,254}, { 1,255}, {11,225}, 
    { 1,255}, { 5,127}, { 1,255}, { 8, 63}, { 1,255}, { 5,247}, { 1,255}, {10, 87}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,221}, { 1,255}, { 5,253}, { 1,255}, {11, 42}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,175}, { 1,255}, { 5,251}, { 1,255}, {10,213}, 
    { 1,255}, { 5,239}, { 1,255}, { 8,246}, { 1,255}, { 5,254}, { 1,255}, {11,134}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,123}, { 1,255}, { 5,247}, { 1,255}, {10,155}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,237}, { 1,255}, { 5,253}, { 1,255}, {11, 81}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,190}, { 1,255}, { 5,251}, { 1,255}, {11,  9}, 
    { 1,255}, { 5,239}, { 1,255}, { 9,182}, { 1,255}, { 5,254}, { 1,255}, {11,178}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,111}, { 1,255}, { 5,247}, { 1,255}, {10,115}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,231}, { 1,255}, { 5,253}, { 1,255}, {11, 64}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,187}, { 1,255}, { 5,251}, { 1,255}, {10,234}, 
    { 1,255}, { 5,239}, { 1,255}, { 8,250}, { 1,255}, { 5,254}, { 1,255}, {11,153}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,126}, { 1,255}, { 5,247}, { 1,255}, {10,181}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,243}, { 1,255}, { 5,253}, { 1,255}, {11,102}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,215}, { 1,255}, { 5,251}, { 1,255}, {11, 25}, 
    { 1,255}, { 5,239}, { 1,255}, { 9,248}, { 1,255}, { 5,254}, { 1,255}, {11,210}, 
    { 1,255}, { 5,127}, { 1,255}, { 8, 95}, { 1,255}, { 5,247}, { 1,255}, {10,103}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,222}, { 1,255}, { 5,253}, { 1,255}, {11, 51}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,183}, { 1,255}, { 5,251}, { 1,255}, {10,220}, 
    { 1,255}, { 5,239}, { 1,255}, { 8,249}, { 1,255}, { 5,254}, { 1,255}, {11,142}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,125}, { 1,255}, { 5,247}, { 1,255}, {10,171}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,238}, { 1,255}, { 5,253}, { 1,255}, {11, 90}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,207}, { 1,255}, { 5,251}, { 1,255}, {11, 17}, 
    { 1,255}, { 5,239}, { 1,255}, { 9,233}, { 1,255}, { 5,254}, { 1,255}, {11,197}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,119}, { 1,255}, { 5,247}, { 1,255}, {10,124}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,235}, { 1,255}, { 5,253}, { 1,255}, {11, 72}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,189}, { 1,255}, { 5,251}, { 1,255}, {11,  1}, 
    { 1,255}, { 5,239}, { 1,255}, { 8,252}, { 1,255}, { 5,254}, { 1,255}, {11,165}, 
    { 1,255}, { 5,127}, { 1,255}, { 8,159}, { 1,255}, { 5,247}, { 1,255}, {10,199}, 
    { 1,255}, { 5,223}, { 1,255}, { 8,245}, { 1,255}, { 5,253}, { 1,255}, {11,116}, 
    { 1,255}, { 5,191}, { 1,255}, { 8,219}, { 1,255}, { 5,251}, { 1,255}, {11, 34}, 
    { 1,255}, { 5,239}, { 1,255}, {10, 59}, { 1,255}, { 5,254}, { 1,255}, {11,240}, 
  }
  #endif
};

static radaudio_huffman rada_nonzero_bitflags_huff5 =
{
  #ifdef HUFFMAN_ENCODE
  {
    {   0, 2}, {  27, 9}, { 283, 9}, {  59,10}, { 155, 9}, { 571,10}, { 503,11}, {1527,11}, 
    { 411, 9}, { 315,10}, {1015,11}, {2039,11}, { 827,10}, {  15,11}, {1039,11}, { 527,11}, 
    {  91, 9}, { 187,10}, { 699,10}, {1551,11}, { 271,11}, {1295,11}, { 783,11}, {1807,11}, 
    { 443,10}, { 143,11}, {1167,11}, { 655,11}, {1679,11}, { 399,11}, {1423,11}, { 955,10}, 
    { 347, 9}, { 123,10}, { 635,10}, { 911,11}, { 379,10}, {1935,11}, {  79,11}, {1103,11}, 
    { 891,10}, { 591,11}, {1615,11}, { 335,11}, {1359,11}, { 847,11}, {1871,11}, { 251,10}, 
    { 763,10}, { 207,11}, {1231,11}, { 719,11}, {1743,11}, { 463,11}, {1487,11}, { 507,10}, 
    { 975,11}, {1999,11}, {  47,11}, {1019,10}, {1071,11}, {   7,10}, { 519,10}, {  61, 8}, 
    { 219, 9}, { 263,10}, { 559,11}, {1583,11}, { 775,10}, { 303,11}, {1327,11}, { 815,11}, 
    {1839,11}, { 175,11}, {1199,11}, { 687,11}, {1711,11}, { 431,11}, {1455,11}, { 135,10}, 
    { 647,10}, { 943,11}, {1967,11}, { 111,11}, {1135,11}, { 623,11}, {1647,11}, { 391,10}, 
    { 367,11}, {1391,11}, { 879,11}, { 903,10}, {1903,11}, {  71,10}, { 583,10}, { 189, 8}, 
    { 327,10}, { 239,11}, {1263,11}, { 751,11}, {1775,11}, { 495,11}, {1519,11}, { 839,10}, 
    {1007,11}, {2031,11}, {  31,11}, { 199,10}, {1055,11}, { 711,10}, { 455,10}, { 125, 8}, 
    { 543,11}, {1567,11}, { 287,11}, { 967,10}, {1311,11}, {  39,10}, { 551,10}, { 253, 8}, 
    { 799,11}, { 295,10}, { 807,10}, {   3, 8}, { 167,10}, { 131, 8}, {  67, 8}, {   1, 5}, 
    { 475, 9}, { 679,10}, {1823,11}, { 159,11}, { 423,10}, {1183,11}, { 671,11}, {1695,11}, 
    { 935,10}, { 415,11}, {1439,11}, { 927,11}, {1951,11}, {  95,11}, {1119,11}, { 103,10}, 
    { 615,10}, { 607,11}, {1631,11}, { 351,11}, {1375,11}, { 863,11}, {1887,11}, { 359,10}, 
    { 223,11}, {1247,11}, { 735,11}, { 871,10}, {1759,11}, { 231,10}, { 743,10}, { 195, 8}, 
    { 487,10}, { 479,11}, {1503,11}, { 991,11}, {2015,11}, {  63,11}, {1087,11}, { 999,10}, 
    { 575,11}, {1599,11}, { 319,11}, {  23,10}, {1343,11}, { 535,10}, { 279,10}, {  35, 8}, 
    { 831,11}, {1855,11}, { 191,11}, { 791,10}, {1215,11}, { 151,10}, { 663,10}, { 163, 8}, 
    { 703,11}, { 407,10}, { 919,10}, {  99, 8}, {  87,10}, { 227, 8}, {  19, 8}, {  17, 5}, 
    { 599,10}, {1727,11}, { 447,11}, {1471,11}, { 959,11}, {1983,11}, { 127,11}, { 343,10}, 
    {1151,11}, { 639,11}, {1663,11}, { 855,10}, { 383,11}, { 215,10}, { 727,10}, { 147, 8}, 
    {1407,11}, { 895,11}, {1919,11}, { 471,10}, { 255,11}, { 983,10}, {  55,10}, {  83, 8}, 
    {1279,11}, { 567,10}, { 311,10}, { 211, 8}, { 823,10}, {  51, 8}, { 179, 8}, {   9, 5}, 
    { 767,11}, {1791,11}, { 511,11}, { 183,10}, {1535,11}, { 695,10}, { 439,10}, { 115, 8}, 
    {1023,11}, { 951,10}, { 119,10}, { 243, 8}, { 631,10}, {  11, 8}, { 139, 8}, {  25, 5}, 
    {2047,11}, { 375,10}, { 887,10}, {  75, 8}, { 247,10}, { 203, 8}, {  43, 8}, {   5, 5}, 
    { 759,10}, { 171, 8}, { 107, 8}, {  29, 6}, { 235, 8}, {  21, 5}, {  13, 5}, {   2, 2}, 
  },
  #endif
  #ifdef HUFFMAN_DECODE
  {
    { 2,  0}, { 5,127}, { 2,255}, { 8,123}, { 2,  0}, { 5,247}, { 2,255}, {10, 61}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,237}, { 2,  0}, { 5,254}, { 2,255}, {11, 13}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,190}, { 2,  0}, { 5,253}, { 2,255}, {10,171}, 
    { 2,  0}, { 5,239}, { 2,255}, { 9,  1}, { 2,  0}, { 6,251}, { 2,255}, {11,106}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,175}, { 2,  0}, { 5,247}, { 2,255}, {10,117}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,246}, { 2,  0}, { 5,254}, { 2,255}, {11, 58}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,221}, { 2,  0}, { 5,253}, { 2,255}, {10,214}, 
    { 2,  0}, { 5,239}, { 2,255}, {10,  3}, { 2,  0}, { 8, 63}, { 2,255}, {11,165}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,126}, { 2,  0}, { 5,247}, { 2,255}, {10, 93}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,243}, { 2,  0}, { 5,254}, { 2,255}, {11, 38}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,215}, { 2,  0}, { 5,253}, { 2,255}, {10,188}, 
    { 2,  0}, { 5,239}, { 2,255}, { 9, 16}, { 2,  0}, { 6,251}, { 2,255}, {11,141}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,187}, { 2,  0}, { 5,247}, { 2,255}, {10,143}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,250}, { 2,  0}, { 5,254}, { 2,255}, {11, 83}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,231}, { 2,  0}, { 5,253}, { 2,255}, {10,234}, 
    { 2,  0}, { 5,239}, { 2,255}, {10, 33}, { 2,  0}, { 8,111}, { 2,255}, {11,198}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,125}, { 2,  0}, { 5,247}, { 2,255}, {10, 79}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,238}, { 2,  0}, { 5,254}, { 2,255}, {11, 25}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,207}, { 2,  0}, { 5,253}, { 2,255}, {10,181}, 
    { 2,  0}, { 5,239}, { 2,255}, { 9,  4}, { 2,  0}, { 6,251}, { 2,255}, {11,131}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,183}, { 2,  0}, { 5,247}, { 2,255}, {10,124}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,249}, { 2,  0}, { 5,254}, { 2,255}, {11, 73}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,222}, { 2,  0}, { 5,253}, { 2,255}, {10,227}, 
    { 2,  0}, { 5,239}, { 2,255}, {10, 17}, { 2,  0}, { 8, 95}, { 2,255}, {11,178}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,159}, { 2,  0}, { 5,247}, { 2,255}, {10,107}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,245}, { 2,  0}, { 5,254}, { 2,255}, {11, 49}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,219}, { 2,  0}, { 5,253}, { 2,255}, {10,205}, 
    { 2,  0}, { 5,239}, { 2,255}, { 9, 64}, { 2,  0}, { 6,251}, { 2,255}, {11,152}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,189}, { 2,  0}, { 5,247}, { 2,255}, {10,157}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,252}, { 2,  0}, { 5,254}, { 2,255}, {11, 97}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,235}, { 2,  0}, { 5,253}, { 2,255}, {10,244}, 
    { 2,  0}, { 5,239}, { 2,255}, {10, 47}, { 2,  0}, { 8,119}, { 2,255}, {11,212}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,123}, { 2,  0}, { 5,247}, { 2,255}, {10, 65}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,237}, { 2,  0}, { 5,254}, { 2,255}, {11, 20}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,190}, { 2,  0}, { 5,253}, { 2,255}, {10,174}, 
    { 2,  0}, { 5,239}, { 2,255}, { 9,  2}, { 2,  0}, { 6,251}, { 2,255}, {11,114}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,175}, { 2,  0}, { 5,247}, { 2,255}, {10,121}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,246}, { 2,  0}, { 5,254}, { 2,255}, {11, 69}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,221}, { 2,  0}, { 5,253}, { 2,255}, {10,218}, 
    { 2,  0}, { 5,239}, { 2,255}, {10,  9}, { 2,  0}, { 8, 63}, { 2,255}, {11,170}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,126}, { 2,  0}, { 5,247}, { 2,255}, {10, 96}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,243}, { 2,  0}, { 5,254}, { 2,255}, {11, 43}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,215}, { 2,  0}, { 5,253}, { 2,255}, {10,199}, 
    { 2,  0}, { 5,239}, { 2,255}, { 9, 32}, { 2,  0}, { 6,251}, { 2,255}, {11,147}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,187}, { 2,  0}, { 5,247}, { 2,255}, {10,151}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,250}, { 2,  0}, { 5,254}, { 2,255}, {11, 88}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,231}, { 2,  0}, { 5,253}, { 2,255}, {10,241}, 
    { 2,  0}, { 5,239}, { 2,255}, {10, 36}, { 2,  0}, { 8,111}, { 2,255}, {11,204}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,125}, { 2,  0}, { 5,247}, { 2,255}, {10, 87}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,238}, { 2,  0}, { 5,254}, { 2,255}, {11, 29}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,207}, { 2,  0}, { 5,253}, { 2,255}, {10,185}, 
    { 2,  0}, { 5,239}, { 2,255}, { 9,  8}, { 2,  0}, { 6,251}, { 2,255}, {11,137}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,183}, { 2,  0}, { 5,247}, { 2,255}, {10,132}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,249}, { 2,  0}, { 5,254}, { 2,255}, {11, 77}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,222}, { 2,  0}, { 5,253}, { 2,255}, {10,230}, 
    { 2,  0}, { 5,239}, { 2,255}, {10, 24}, { 2,  0}, { 8, 95}, { 2,255}, {11,194}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,159}, { 2,  0}, { 5,247}, { 2,255}, {10,110}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,245}, { 2,  0}, { 5,254}, { 2,255}, {11, 53}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,219}, { 2,  0}, { 5,253}, { 2,255}, {10,211}, 
    { 2,  0}, { 5,239}, { 2,255}, { 9,128}, { 2,  0}, { 6,251}, { 2,255}, {11,161}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,189}, { 2,  0}, { 5,247}, { 2,255}, {10,160}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,252}, { 2,  0}, { 5,254}, { 2,255}, {11,101}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,235}, { 2,  0}, { 5,253}, { 2,255}, {11,  6}, 
    { 2,  0}, { 5,239}, { 2,255}, {10, 55}, { 2,  0}, { 8,119}, { 2,255}, {11,226}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,123}, { 2,  0}, { 5,247}, { 2,255}, {10, 62}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,237}, { 2,  0}, { 5,254}, { 2,255}, {11, 15}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,190}, { 2,  0}, { 5,253}, { 2,255}, {10,173}, 
    { 2,  0}, { 5,239}, { 2,255}, { 9,  1}, { 2,  0}, { 6,251}, { 2,255}, {11,112}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,175}, { 2,  0}, { 5,247}, { 2,255}, {10,118}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,246}, { 2,  0}, { 5,254}, { 2,255}, {11, 66}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,221}, { 2,  0}, { 5,253}, { 2,255}, {10,217}, 
    { 2,  0}, { 5,239}, { 2,255}, {10,  5}, { 2,  0}, { 8, 63}, { 2,255}, {11,168}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,126}, { 2,  0}, { 5,247}, { 2,255}, {10, 94}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,243}, { 2,  0}, { 5,254}, { 2,255}, {11, 41}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,215}, { 2,  0}, { 5,253}, { 2,255}, {10,192}, 
    { 2,  0}, { 5,239}, { 2,255}, { 9, 16}, { 2,  0}, { 6,251}, { 2,255}, {11,145}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,187}, { 2,  0}, { 5,247}, { 2,255}, {10,144}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,250}, { 2,  0}, { 5,254}, { 2,255}, {11, 85}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,231}, { 2,  0}, { 5,253}, { 2,255}, {10,236}, 
    { 2,  0}, { 5,239}, { 2,255}, {10, 34}, { 2,  0}, { 8,111}, { 2,255}, {11,201}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,125}, { 2,  0}, { 5,247}, { 2,255}, {10, 80}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,238}, { 2,  0}, { 5,254}, { 2,255}, {11, 27}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,207}, { 2,  0}, { 5,253}, { 2,255}, {10,182}, 
    { 2,  0}, { 5,239}, { 2,255}, { 9,  4}, { 2,  0}, { 6,251}, { 2,255}, {11,134}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,183}, { 2,  0}, { 5,247}, { 2,255}, {10,129}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,249}, { 2,  0}, { 5,254}, { 2,255}, {11, 75}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,222}, { 2,  0}, { 5,253}, { 2,255}, {10,229}, 
    { 2,  0}, { 5,239}, { 2,255}, {10, 18}, { 2,  0}, { 8, 95}, { 2,255}, {11,184}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,159}, { 2,  0}, { 5,247}, { 2,255}, {10,109}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,245}, { 2,  0}, { 5,254}, { 2,255}, {11, 51}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,219}, { 2,  0}, { 5,253}, { 2,255}, {10,206}, 
    { 2,  0}, { 5,239}, { 2,255}, { 9, 64}, { 2,  0}, { 6,251}, { 2,255}, {11,154}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,189}, { 2,  0}, { 5,247}, { 2,255}, {10,158}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,252}, { 2,  0}, { 5,254}, { 2,255}, {11, 99}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,235}, { 2,  0}, { 5,253}, { 2,255}, {10,248}, 
    { 2,  0}, { 5,239}, { 2,255}, {10, 48}, { 2,  0}, { 8,119}, { 2,255}, {11,224}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,123}, { 2,  0}, { 5,247}, { 2,255}, {10, 68}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,237}, { 2,  0}, { 5,254}, { 2,255}, {11, 22}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,190}, { 2,  0}, { 5,253}, { 2,255}, {10,179}, 
    { 2,  0}, { 5,239}, { 2,255}, { 9,  2}, { 2,  0}, { 6,251}, { 2,255}, {11,120}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,175}, { 2,  0}, { 5,247}, { 2,255}, {10,122}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,246}, { 2,  0}, { 5,254}, { 2,255}, {11, 71}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,221}, { 2,  0}, { 5,253}, { 2,255}, {10,220}, 
    { 2,  0}, { 5,239}, { 2,255}, {10, 12}, { 2,  0}, { 8, 63}, { 2,255}, {11,176}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,126}, { 2,  0}, { 5,247}, { 2,255}, {10,103}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,243}, { 2,  0}, { 5,254}, { 2,255}, {11, 45}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,215}, { 2,  0}, { 5,253}, { 2,255}, {10,203}, 
    { 2,  0}, { 5,239}, { 2,255}, { 9, 32}, { 2,  0}, { 6,251}, { 2,255}, {11,149}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,187}, { 2,  0}, { 5,247}, { 2,255}, {10,155}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,250}, { 2,  0}, { 5,254}, { 2,255}, {11, 90}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,231}, { 2,  0}, { 5,253}, { 2,255}, {10,242}, 
    { 2,  0}, { 5,239}, { 2,255}, {10, 40}, { 2,  0}, { 8,111}, { 2,255}, {11,209}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,125}, { 2,  0}, { 5,247}, { 2,255}, {10, 91}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,238}, { 2,  0}, { 5,254}, { 2,255}, {11, 35}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,207}, { 2,  0}, { 5,253}, { 2,255}, {10,186}, 
    { 2,  0}, { 5,239}, { 2,255}, { 9,  8}, { 2,  0}, { 6,251}, { 2,255}, {11,139}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,183}, { 2,  0}, { 5,247}, { 2,255}, {10,136}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,249}, { 2,  0}, { 5,254}, { 2,255}, {11, 81}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,222}, { 2,  0}, { 5,253}, { 2,255}, {10,233}, 
    { 2,  0}, { 5,239}, { 2,255}, {10, 31}, { 2,  0}, { 8, 95}, { 2,255}, {11,196}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,159}, { 2,  0}, { 5,247}, { 2,255}, {10,115}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,245}, { 2,  0}, { 5,254}, { 2,255}, {11, 56}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,219}, { 2,  0}, { 5,253}, { 2,255}, {10,213}, 
    { 2,  0}, { 5,239}, { 2,255}, { 9,128}, { 2,  0}, { 6,251}, { 2,255}, {11,163}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,189}, { 2,  0}, { 5,247}, { 2,255}, {10,167}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,252}, { 2,  0}, { 5,254}, { 2,255}, {11,104}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,235}, { 2,  0}, { 5,253}, { 2,255}, {11, 10}, 
    { 2,  0}, { 5,239}, { 2,255}, {10, 59}, { 2,  0}, { 8,119}, { 2,255}, {11,232}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,123}, { 2,  0}, { 5,247}, { 2,255}, {10, 61}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,237}, { 2,  0}, { 5,254}, { 2,255}, {11, 14}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,190}, { 2,  0}, { 5,253}, { 2,255}, {10,171}, 
    { 2,  0}, { 5,239}, { 2,255}, { 9,  1}, { 2,  0}, { 6,251}, { 2,255}, {11,108}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,175}, { 2,  0}, { 5,247}, { 2,255}, {10,117}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,246}, { 2,  0}, { 5,254}, { 2,255}, {11, 60}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,221}, { 2,  0}, { 5,253}, { 2,255}, {10,214}, 
    { 2,  0}, { 5,239}, { 2,255}, {10,  3}, { 2,  0}, { 8, 63}, { 2,255}, {11,166}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,126}, { 2,  0}, { 5,247}, { 2,255}, {10, 93}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,243}, { 2,  0}, { 5,254}, { 2,255}, {11, 39}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,215}, { 2,  0}, { 5,253}, { 2,255}, {10,188}, 
    { 2,  0}, { 5,239}, { 2,255}, { 9, 16}, { 2,  0}, { 6,251}, { 2,255}, {11,142}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,187}, { 2,  0}, { 5,247}, { 2,255}, {10,143}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,250}, { 2,  0}, { 5,254}, { 2,255}, {11, 84}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,231}, { 2,  0}, { 5,253}, { 2,255}, {10,234}, 
    { 2,  0}, { 5,239}, { 2,255}, {10, 33}, { 2,  0}, { 8,111}, { 2,255}, {11,200}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,125}, { 2,  0}, { 5,247}, { 2,255}, {10, 79}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,238}, { 2,  0}, { 5,254}, { 2,255}, {11, 26}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,207}, { 2,  0}, { 5,253}, { 2,255}, {10,181}, 
    { 2,  0}, { 5,239}, { 2,255}, { 9,  4}, { 2,  0}, { 6,251}, { 2,255}, {11,133}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,183}, { 2,  0}, { 5,247}, { 2,255}, {10,124}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,249}, { 2,  0}, { 5,254}, { 2,255}, {11, 74}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,222}, { 2,  0}, { 5,253}, { 2,255}, {10,227}, 
    { 2,  0}, { 5,239}, { 2,255}, {10, 17}, { 2,  0}, { 8, 95}, { 2,255}, {11,180}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,159}, { 2,  0}, { 5,247}, { 2,255}, {10,107}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,245}, { 2,  0}, { 5,254}, { 2,255}, {11, 50}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,219}, { 2,  0}, { 5,253}, { 2,255}, {10,205}, 
    { 2,  0}, { 5,239}, { 2,255}, { 9, 64}, { 2,  0}, { 6,251}, { 2,255}, {11,153}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,189}, { 2,  0}, { 5,247}, { 2,255}, {10,157}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,252}, { 2,  0}, { 5,254}, { 2,255}, {11, 98}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,235}, { 2,  0}, { 5,253}, { 2,255}, {10,244}, 
    { 2,  0}, { 5,239}, { 2,255}, {10, 47}, { 2,  0}, { 8,119}, { 2,255}, {11,216}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,123}, { 2,  0}, { 5,247}, { 2,255}, {10, 65}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,237}, { 2,  0}, { 5,254}, { 2,255}, {11, 21}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,190}, { 2,  0}, { 5,253}, { 2,255}, {10,174}, 
    { 2,  0}, { 5,239}, { 2,255}, { 9,  2}, { 2,  0}, { 6,251}, { 2,255}, {11,116}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,175}, { 2,  0}, { 5,247}, { 2,255}, {10,121}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,246}, { 2,  0}, { 5,254}, { 2,255}, {11, 70}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,221}, { 2,  0}, { 5,253}, { 2,255}, {10,218}, 
    { 2,  0}, { 5,239}, { 2,255}, {10,  9}, { 2,  0}, { 8, 63}, { 2,255}, {11,172}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,126}, { 2,  0}, { 5,247}, { 2,255}, {10, 96}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,243}, { 2,  0}, { 5,254}, { 2,255}, {11, 44}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,215}, { 2,  0}, { 5,253}, { 2,255}, {10,199}, 
    { 2,  0}, { 5,239}, { 2,255}, { 9, 32}, { 2,  0}, { 6,251}, { 2,255}, {11,148}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,187}, { 2,  0}, { 5,247}, { 2,255}, {10,151}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,250}, { 2,  0}, { 5,254}, { 2,255}, {11, 89}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,231}, { 2,  0}, { 5,253}, { 2,255}, {10,241}, 
    { 2,  0}, { 5,239}, { 2,255}, {10, 36}, { 2,  0}, { 8,111}, { 2,255}, {11,208}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,125}, { 2,  0}, { 5,247}, { 2,255}, {10, 87}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,238}, { 2,  0}, { 5,254}, { 2,255}, {11, 30}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,207}, { 2,  0}, { 5,253}, { 2,255}, {10,185}, 
    { 2,  0}, { 5,239}, { 2,255}, { 9,  8}, { 2,  0}, { 6,251}, { 2,255}, {11,138}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,183}, { 2,  0}, { 5,247}, { 2,255}, {10,132}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,249}, { 2,  0}, { 5,254}, { 2,255}, {11, 78}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,222}, { 2,  0}, { 5,253}, { 2,255}, {10,230}, 
    { 2,  0}, { 5,239}, { 2,255}, {10, 24}, { 2,  0}, { 8, 95}, { 2,255}, {11,195}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,159}, { 2,  0}, { 5,247}, { 2,255}, {10,110}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,245}, { 2,  0}, { 5,254}, { 2,255}, {11, 54}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,219}, { 2,  0}, { 5,253}, { 2,255}, {10,211}, 
    { 2,  0}, { 5,239}, { 2,255}, { 9,128}, { 2,  0}, { 6,251}, { 2,255}, {11,162}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,189}, { 2,  0}, { 5,247}, { 2,255}, {10,160}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,252}, { 2,  0}, { 5,254}, { 2,255}, {11,102}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,235}, { 2,  0}, { 5,253}, { 2,255}, {11,  7}, 
    { 2,  0}, { 5,239}, { 2,255}, {10, 55}, { 2,  0}, { 8,119}, { 2,255}, {11,228}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,123}, { 2,  0}, { 5,247}, { 2,255}, {10, 62}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,237}, { 2,  0}, { 5,254}, { 2,255}, {11, 19}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,190}, { 2,  0}, { 5,253}, { 2,255}, {10,173}, 
    { 2,  0}, { 5,239}, { 2,255}, { 9,  1}, { 2,  0}, { 6,251}, { 2,255}, {11,113}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,175}, { 2,  0}, { 5,247}, { 2,255}, {10,118}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,246}, { 2,  0}, { 5,254}, { 2,255}, {11, 67}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,221}, { 2,  0}, { 5,253}, { 2,255}, {10,217}, 
    { 2,  0}, { 5,239}, { 2,255}, {10,  5}, { 2,  0}, { 8, 63}, { 2,255}, {11,169}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,126}, { 2,  0}, { 5,247}, { 2,255}, {10, 94}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,243}, { 2,  0}, { 5,254}, { 2,255}, {11, 42}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,215}, { 2,  0}, { 5,253}, { 2,255}, {10,192}, 
    { 2,  0}, { 5,239}, { 2,255}, { 9, 16}, { 2,  0}, { 6,251}, { 2,255}, {11,146}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,187}, { 2,  0}, { 5,247}, { 2,255}, {10,144}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,250}, { 2,  0}, { 5,254}, { 2,255}, {11, 86}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,231}, { 2,  0}, { 5,253}, { 2,255}, {10,236}, 
    { 2,  0}, { 5,239}, { 2,255}, {10, 34}, { 2,  0}, { 8,111}, { 2,255}, {11,202}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,125}, { 2,  0}, { 5,247}, { 2,255}, {10, 80}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,238}, { 2,  0}, { 5,254}, { 2,255}, {11, 28}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,207}, { 2,  0}, { 5,253}, { 2,255}, {10,182}, 
    { 2,  0}, { 5,239}, { 2,255}, { 9,  4}, { 2,  0}, { 6,251}, { 2,255}, {11,135}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,183}, { 2,  0}, { 5,247}, { 2,255}, {10,129}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,249}, { 2,  0}, { 5,254}, { 2,255}, {11, 76}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,222}, { 2,  0}, { 5,253}, { 2,255}, {10,229}, 
    { 2,  0}, { 5,239}, { 2,255}, {10, 18}, { 2,  0}, { 8, 95}, { 2,255}, {11,193}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,159}, { 2,  0}, { 5,247}, { 2,255}, {10,109}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,245}, { 2,  0}, { 5,254}, { 2,255}, {11, 52}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,219}, { 2,  0}, { 5,253}, { 2,255}, {10,206}, 
    { 2,  0}, { 5,239}, { 2,255}, { 9, 64}, { 2,  0}, { 6,251}, { 2,255}, {11,156}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,189}, { 2,  0}, { 5,247}, { 2,255}, {10,158}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,252}, { 2,  0}, { 5,254}, { 2,255}, {11,100}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,235}, { 2,  0}, { 5,253}, { 2,255}, {10,248}, 
    { 2,  0}, { 5,239}, { 2,255}, {10, 48}, { 2,  0}, { 8,119}, { 2,255}, {11,225}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,123}, { 2,  0}, { 5,247}, { 2,255}, {10, 68}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,237}, { 2,  0}, { 5,254}, { 2,255}, {11, 23}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,190}, { 2,  0}, { 5,253}, { 2,255}, {10,179}, 
    { 2,  0}, { 5,239}, { 2,255}, { 9,  2}, { 2,  0}, { 6,251}, { 2,255}, {11,130}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,175}, { 2,  0}, { 5,247}, { 2,255}, {10,122}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,246}, { 2,  0}, { 5,254}, { 2,255}, {11, 72}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,221}, { 2,  0}, { 5,253}, { 2,255}, {10,220}, 
    { 2,  0}, { 5,239}, { 2,255}, {10, 12}, { 2,  0}, { 8, 63}, { 2,255}, {11,177}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,126}, { 2,  0}, { 5,247}, { 2,255}, {10,103}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,243}, { 2,  0}, { 5,254}, { 2,255}, {11, 46}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,215}, { 2,  0}, { 5,253}, { 2,255}, {10,203}, 
    { 2,  0}, { 5,239}, { 2,255}, { 9, 32}, { 2,  0}, { 6,251}, { 2,255}, {11,150}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,187}, { 2,  0}, { 5,247}, { 2,255}, {10,155}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,250}, { 2,  0}, { 5,254}, { 2,255}, {11, 92}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,231}, { 2,  0}, { 5,253}, { 2,255}, {10,242}, 
    { 2,  0}, { 5,239}, { 2,255}, {10, 40}, { 2,  0}, { 8,111}, { 2,255}, {11,210}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,125}, { 2,  0}, { 5,247}, { 2,255}, {10, 91}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,238}, { 2,  0}, { 5,254}, { 2,255}, {11, 37}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,207}, { 2,  0}, { 5,253}, { 2,255}, {10,186}, 
    { 2,  0}, { 5,239}, { 2,255}, { 9,  8}, { 2,  0}, { 6,251}, { 2,255}, {11,140}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,183}, { 2,  0}, { 5,247}, { 2,255}, {10,136}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,249}, { 2,  0}, { 5,254}, { 2,255}, {11, 82}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,222}, { 2,  0}, { 5,253}, { 2,255}, {10,233}, 
    { 2,  0}, { 5,239}, { 2,255}, {10, 31}, { 2,  0}, { 8, 95}, { 2,255}, {11,197}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,159}, { 2,  0}, { 5,247}, { 2,255}, {10,115}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,245}, { 2,  0}, { 5,254}, { 2,255}, {11, 57}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,219}, { 2,  0}, { 5,253}, { 2,255}, {10,213}, 
    { 2,  0}, { 5,239}, { 2,255}, { 9,128}, { 2,  0}, { 6,251}, { 2,255}, {11,164}, 
    { 2,  0}, { 5,127}, { 2,255}, { 8,189}, { 2,  0}, { 5,247}, { 2,255}, {10,167}, 
    { 2,  0}, { 5,223}, { 2,255}, { 8,252}, { 2,  0}, { 5,254}, { 2,255}, {11,105}, 
    { 2,  0}, { 5,191}, { 2,255}, { 8,235}, { 2,  0}, { 5,253}, { 2,255}, {11, 11}, 
    { 2,  0}, { 5,239}, { 2,255}, {10, 59}, { 2,  0}, { 8,119}, { 2,255}, {11,240}, 
  }
  #endif
};

static radaudio_huffman rada_nonzero_coefficient_big_huff0 =
{
  #ifdef HUFFMAN_ENCODE
  {
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 3}, {   4, 3}, {   1, 4}, {   5, 5}, {  21, 5}, {   3, 6}, {  35, 6}, {  19, 6}, 
    {  27, 7}, {  91, 7}, { 123, 8}, { 251, 8}, { 199, 9}, { 455, 9}, { 167,10}, { 935,11}, 
    {1959,11}, { 103,11}, {1127,11}, { 615,11}, {1639,11}, { 359,11}, {1383,11}, { 871,11}, 
    {1895,11}, { 231,11}, {1255,11}, { 743,11}, {1767,11}, { 487,11}, {1511,11}, { 999,11}, 
    {2023,11}, {  23,11}, {1047,11}, { 535,11}, {1559,11}, { 279,11}, {1303,11}, { 791,11}, 
    {1815,11}, { 151,11}, {1175,11}, { 663,11}, {1687,11}, { 407,11}, {1431,11}, { 919,11}, 
    {1943,11}, {  87,11}, {1111,11}, { 599,11}, {1623,11}, { 343,11}, {1367,11}, { 855,11}, 
    {1879,11}, { 215,11}, {1239,11}, { 727,11}, {1751,11}, { 471,11}, {1495,11}, { 983,11}, 
    {2007,11}, {  55,11}, {1079,11}, { 567,11}, {1591,11}, { 311,11}, {1335,11}, { 823,11}, 
    {1847,11}, { 183,11}, {1207,11}, { 695,11}, {1719,11}, { 439,11}, {1463,11}, { 951,11}, 
    {1975,11}, { 119,11}, {1143,11}, { 631,11}, {1655,11}, { 375,11}, {1399,11}, { 887,11}, 
    {1911,11}, { 247,11}, {1271,11}, { 759,11}, {1783,11}, { 503,11}, {1527,11}, {1015,11}, 
    {2039,11}, {  15,11}, {1039,11}, { 527,11}, {1551,11}, { 271,11}, {1295,11}, { 783,11}, 
    {1807,11}, { 143,11}, {1167,11}, { 655,11}, {1679,11}, { 399,11}, {1423,11}, { 911,11}, 
    {1935,11}, {  79,11}, {1103,11}, { 591,11}, {1615,11}, { 335,11}, {1359,11}, { 847,11}, 
    {1871,11}, { 207,11}, {1231,11}, { 719,11}, {1743,11}, { 463,11}, {1487,11}, { 975,11}, 
    {1999,11}, {  47,11}, {1071,11}, { 559,11}, {1583,11}, { 303,11}, {1327,11}, { 815,11}, 
    {1839,11}, { 175,11}, {1199,11}, { 687,11}, {1711,11}, { 431,11}, {1455,11}, { 943,11}, 
    {1967,11}, { 111,11}, {1135,11}, { 623,11}, {1647,11}, { 367,11}, {1391,11}, { 879,11}, 
    {1903,11}, { 239,11}, {1263,11}, { 751,11}, {1775,11}, { 495,11}, {1519,11}, {1007,11}, 
    {2031,11}, {  31,11}, {1055,11}, { 543,11}, {1567,11}, { 287,11}, {1311,11}, { 799,11}, 
    {1823,11}, { 159,11}, {1183,11}, { 671,11}, {1695,11}, { 415,11}, {1439,11}, { 927,11}, 
    {1951,11}, {  95,11}, {1119,11}, { 607,11}, {1631,11}, { 351,11}, {1375,11}, { 863,11}, 
    {1887,11}, { 223,11}, {1247,11}, { 735,11}, {1759,11}, { 479,11}, {1503,11}, { 991,11}, 
    {2015,11}, {  63,11}, {1087,11}, { 575,11}, {1599,11}, { 319,11}, {1343,11}, { 831,11}, 
    {1855,11}, { 191,11}, {1215,11}, { 703,11}, {1727,11}, { 447,11}, {1471,11}, { 959,11}, 
    {1983,11}, { 127,11}, {1151,11}, { 639,11}, {1663,11}, { 383,11}, {1407,11}, { 895,11}, 
    {1919,11}, { 255,11}, {1279,11}, { 767,11}, {1791,11}, { 511,11}, {1535,11}, {1023,11}, 
    {2047,11}, { 679,10}, { 423,10}, {  39, 9}, { 295, 9}, {   7, 8}, { 135, 8}, {  71, 8}, 
    {  59, 7}, {  51, 6}, {  11, 6}, {  43, 6}, {  13, 5}, {  29, 5}, {   9, 4}, {   2, 3}, 
    {   6, 3}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
  },
  #endif
  #ifdef HUFFMAN_DECODE
  {
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 13}, { 3,  9}, { 5, 11}, { 3,248}, { 8,237}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,242}, { 3,  9}, { 5,244}, { 3,248}, {11,105}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 15}, { 3,  9}, { 5, 12}, { 3,248}, {11, 41}, 
    { 3,  8}, { 4,246}, { 3,247}, { 7, 16}, { 3,  9}, { 5,245}, { 3,248}, {11,169}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 14}, { 3,  9}, { 5, 11}, { 3,248}, { 9,235}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,243}, { 3,  9}, { 5,244}, { 3,248}, {11,137}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6,241}, { 3,  9}, { 5, 12}, { 3,248}, {11, 73}, 
    { 3,  8}, { 4,246}, { 3,247}, { 7,240}, { 3,  9}, { 5,245}, { 3,248}, {11,201}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 13}, { 3,  9}, { 5, 11}, { 3,248}, { 8,239}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,242}, { 3,  9}, { 5,244}, { 3,248}, {11,121}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 15}, { 3,  9}, { 5, 12}, { 3,248}, {11, 57}, 
    { 3,  8}, { 4,246}, { 3,247}, { 7, 17}, { 3,  9}, { 5,245}, { 3,248}, {11,185}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 14}, { 3,  9}, { 5, 11}, { 3,248}, {11, 25}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,243}, { 3,  9}, { 5,244}, { 3,248}, {11,153}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6,241}, { 3,  9}, { 5, 12}, { 3,248}, {11, 89}, 
    { 3,  8}, { 4,246}, { 3,247}, { 8, 18}, { 3,  9}, { 5,245}, { 3,248}, {11,217}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 13}, { 3,  9}, { 5, 11}, { 3,248}, { 8,238}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,242}, { 3,  9}, { 5,244}, { 3,248}, {11,113}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 15}, { 3,  9}, { 5, 12}, { 3,248}, {11, 49}, 
    { 3,  8}, { 4,246}, { 3,247}, { 7, 16}, { 3,  9}, { 5,245}, { 3,248}, {11,177}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 14}, { 3,  9}, { 5, 11}, { 3,248}, {10, 22}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,243}, { 3,  9}, { 5,244}, { 3,248}, {11,145}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6,241}, { 3,  9}, { 5, 12}, { 3,248}, {11, 81}, 
    { 3,  8}, { 4,246}, { 3,247}, { 7,240}, { 3,  9}, { 5,245}, { 3,248}, {11,209}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 13}, { 3,  9}, { 5, 11}, { 3,248}, { 9, 20}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,242}, { 3,  9}, { 5,244}, { 3,248}, {11,129}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 15}, { 3,  9}, { 5, 12}, { 3,248}, {11, 65}, 
    { 3,  8}, { 4,246}, { 3,247}, { 7, 17}, { 3,  9}, { 5,245}, { 3,248}, {11,193}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 14}, { 3,  9}, { 5, 11}, { 3,248}, {11, 33}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,243}, { 3,  9}, { 5,244}, { 3,248}, {11,161}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6,241}, { 3,  9}, { 5, 12}, { 3,248}, {11, 97}, 
    { 3,  8}, { 4,246}, { 3,247}, { 8, 19}, { 3,  9}, { 5,245}, { 3,248}, {11,225}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 13}, { 3,  9}, { 5, 11}, { 3,248}, { 8,237}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,242}, { 3,  9}, { 5,244}, { 3,248}, {11,109}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 15}, { 3,  9}, { 5, 12}, { 3,248}, {11, 45}, 
    { 3,  8}, { 4,246}, { 3,247}, { 7, 16}, { 3,  9}, { 5,245}, { 3,248}, {11,173}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 14}, { 3,  9}, { 5, 11}, { 3,248}, { 9,236}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,243}, { 3,  9}, { 5,244}, { 3,248}, {11,141}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6,241}, { 3,  9}, { 5, 12}, { 3,248}, {11, 77}, 
    { 3,  8}, { 4,246}, { 3,247}, { 7,240}, { 3,  9}, { 5,245}, { 3,248}, {11,205}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 13}, { 3,  9}, { 5, 11}, { 3,248}, { 8,239}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,242}, { 3,  9}, { 5,244}, { 3,248}, {11,125}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 15}, { 3,  9}, { 5, 12}, { 3,248}, {11, 61}, 
    { 3,  8}, { 4,246}, { 3,247}, { 7, 17}, { 3,  9}, { 5,245}, { 3,248}, {11,189}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 14}, { 3,  9}, { 5, 11}, { 3,248}, {11, 29}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,243}, { 3,  9}, { 5,244}, { 3,248}, {11,157}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6,241}, { 3,  9}, { 5, 12}, { 3,248}, {11, 93}, 
    { 3,  8}, { 4,246}, { 3,247}, { 8, 18}, { 3,  9}, { 5,245}, { 3,248}, {11,221}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 13}, { 3,  9}, { 5, 11}, { 3,248}, { 8,238}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,242}, { 3,  9}, { 5,244}, { 3,248}, {11,117}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 15}, { 3,  9}, { 5, 12}, { 3,248}, {11, 53}, 
    { 3,  8}, { 4,246}, { 3,247}, { 7, 16}, { 3,  9}, { 5,245}, { 3,248}, {11,181}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 14}, { 3,  9}, { 5, 11}, { 3,248}, {10,234}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,243}, { 3,  9}, { 5,244}, { 3,248}, {11,149}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6,241}, { 3,  9}, { 5, 12}, { 3,248}, {11, 85}, 
    { 3,  8}, { 4,246}, { 3,247}, { 7,240}, { 3,  9}, { 5,245}, { 3,248}, {11,213}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 13}, { 3,  9}, { 5, 11}, { 3,248}, { 9, 21}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,242}, { 3,  9}, { 5,244}, { 3,248}, {11,133}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 15}, { 3,  9}, { 5, 12}, { 3,248}, {11, 69}, 
    { 3,  8}, { 4,246}, { 3,247}, { 7, 17}, { 3,  9}, { 5,245}, { 3,248}, {11,197}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 14}, { 3,  9}, { 5, 11}, { 3,248}, {11, 37}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,243}, { 3,  9}, { 5,244}, { 3,248}, {11,165}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6,241}, { 3,  9}, { 5, 12}, { 3,248}, {11,101}, 
    { 3,  8}, { 4,246}, { 3,247}, { 8, 19}, { 3,  9}, { 5,245}, { 3,248}, {11,229}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 13}, { 3,  9}, { 5, 11}, { 3,248}, { 8,237}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,242}, { 3,  9}, { 5,244}, { 3,248}, {11,107}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 15}, { 3,  9}, { 5, 12}, { 3,248}, {11, 43}, 
    { 3,  8}, { 4,246}, { 3,247}, { 7, 16}, { 3,  9}, { 5,245}, { 3,248}, {11,171}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 14}, { 3,  9}, { 5, 11}, { 3,248}, { 9,235}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,243}, { 3,  9}, { 5,244}, { 3,248}, {11,139}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6,241}, { 3,  9}, { 5, 12}, { 3,248}, {11, 75}, 
    { 3,  8}, { 4,246}, { 3,247}, { 7,240}, { 3,  9}, { 5,245}, { 3,248}, {11,203}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 13}, { 3,  9}, { 5, 11}, { 3,248}, { 8,239}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,242}, { 3,  9}, { 5,244}, { 3,248}, {11,123}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 15}, { 3,  9}, { 5, 12}, { 3,248}, {11, 59}, 
    { 3,  8}, { 4,246}, { 3,247}, { 7, 17}, { 3,  9}, { 5,245}, { 3,248}, {11,187}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 14}, { 3,  9}, { 5, 11}, { 3,248}, {11, 27}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,243}, { 3,  9}, { 5,244}, { 3,248}, {11,155}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6,241}, { 3,  9}, { 5, 12}, { 3,248}, {11, 91}, 
    { 3,  8}, { 4,246}, { 3,247}, { 8, 18}, { 3,  9}, { 5,245}, { 3,248}, {11,219}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 13}, { 3,  9}, { 5, 11}, { 3,248}, { 8,238}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,242}, { 3,  9}, { 5,244}, { 3,248}, {11,115}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 15}, { 3,  9}, { 5, 12}, { 3,248}, {11, 51}, 
    { 3,  8}, { 4,246}, { 3,247}, { 7, 16}, { 3,  9}, { 5,245}, { 3,248}, {11,179}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 14}, { 3,  9}, { 5, 11}, { 3,248}, {10,233}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,243}, { 3,  9}, { 5,244}, { 3,248}, {11,147}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6,241}, { 3,  9}, { 5, 12}, { 3,248}, {11, 83}, 
    { 3,  8}, { 4,246}, { 3,247}, { 7,240}, { 3,  9}, { 5,245}, { 3,248}, {11,211}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 13}, { 3,  9}, { 5, 11}, { 3,248}, { 9, 20}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,242}, { 3,  9}, { 5,244}, { 3,248}, {11,131}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 15}, { 3,  9}, { 5, 12}, { 3,248}, {11, 67}, 
    { 3,  8}, { 4,246}, { 3,247}, { 7, 17}, { 3,  9}, { 5,245}, { 3,248}, {11,195}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 14}, { 3,  9}, { 5, 11}, { 3,248}, {11, 35}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,243}, { 3,  9}, { 5,244}, { 3,248}, {11,163}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6,241}, { 3,  9}, { 5, 12}, { 3,248}, {11, 99}, 
    { 3,  8}, { 4,246}, { 3,247}, { 8, 19}, { 3,  9}, { 5,245}, { 3,248}, {11,227}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 13}, { 3,  9}, { 5, 11}, { 3,248}, { 8,237}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,242}, { 3,  9}, { 5,244}, { 3,248}, {11,111}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 15}, { 3,  9}, { 5, 12}, { 3,248}, {11, 47}, 
    { 3,  8}, { 4,246}, { 3,247}, { 7, 16}, { 3,  9}, { 5,245}, { 3,248}, {11,175}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 14}, { 3,  9}, { 5, 11}, { 3,248}, { 9,236}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,243}, { 3,  9}, { 5,244}, { 3,248}, {11,143}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6,241}, { 3,  9}, { 5, 12}, { 3,248}, {11, 79}, 
    { 3,  8}, { 4,246}, { 3,247}, { 7,240}, { 3,  9}, { 5,245}, { 3,248}, {11,207}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 13}, { 3,  9}, { 5, 11}, { 3,248}, { 8,239}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,242}, { 3,  9}, { 5,244}, { 3,248}, {11,127}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 15}, { 3,  9}, { 5, 12}, { 3,248}, {11, 63}, 
    { 3,  8}, { 4,246}, { 3,247}, { 7, 17}, { 3,  9}, { 5,245}, { 3,248}, {11,191}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 14}, { 3,  9}, { 5, 11}, { 3,248}, {11, 31}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,243}, { 3,  9}, { 5,244}, { 3,248}, {11,159}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6,241}, { 3,  9}, { 5, 12}, { 3,248}, {11, 95}, 
    { 3,  8}, { 4,246}, { 3,247}, { 8, 18}, { 3,  9}, { 5,245}, { 3,248}, {11,223}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 13}, { 3,  9}, { 5, 11}, { 3,248}, { 8,238}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,242}, { 3,  9}, { 5,244}, { 3,248}, {11,119}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 15}, { 3,  9}, { 5, 12}, { 3,248}, {11, 55}, 
    { 3,  8}, { 4,246}, { 3,247}, { 7, 16}, { 3,  9}, { 5,245}, { 3,248}, {11,183}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 14}, { 3,  9}, { 5, 11}, { 3,248}, {11, 23}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,243}, { 3,  9}, { 5,244}, { 3,248}, {11,151}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6,241}, { 3,  9}, { 5, 12}, { 3,248}, {11, 87}, 
    { 3,  8}, { 4,246}, { 3,247}, { 7,240}, { 3,  9}, { 5,245}, { 3,248}, {11,215}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 13}, { 3,  9}, { 5, 11}, { 3,248}, { 9, 21}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,242}, { 3,  9}, { 5,244}, { 3,248}, {11,135}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 15}, { 3,  9}, { 5, 12}, { 3,248}, {11, 71}, 
    { 3,  8}, { 4,246}, { 3,247}, { 7, 17}, { 3,  9}, { 5,245}, { 3,248}, {11,199}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 14}, { 3,  9}, { 5, 11}, { 3,248}, {11, 39}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,243}, { 3,  9}, { 5,244}, { 3,248}, {11,167}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6,241}, { 3,  9}, { 5, 12}, { 3,248}, {11,103}, 
    { 3,  8}, { 4,246}, { 3,247}, { 8, 19}, { 3,  9}, { 5,245}, { 3,248}, {11,231}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 13}, { 3,  9}, { 5, 11}, { 3,248}, { 8,237}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,242}, { 3,  9}, { 5,244}, { 3,248}, {11,106}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 15}, { 3,  9}, { 5, 12}, { 3,248}, {11, 42}, 
    { 3,  8}, { 4,246}, { 3,247}, { 7, 16}, { 3,  9}, { 5,245}, { 3,248}, {11,170}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 14}, { 3,  9}, { 5, 11}, { 3,248}, { 9,235}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,243}, { 3,  9}, { 5,244}, { 3,248}, {11,138}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6,241}, { 3,  9}, { 5, 12}, { 3,248}, {11, 74}, 
    { 3,  8}, { 4,246}, { 3,247}, { 7,240}, { 3,  9}, { 5,245}, { 3,248}, {11,202}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 13}, { 3,  9}, { 5, 11}, { 3,248}, { 8,239}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,242}, { 3,  9}, { 5,244}, { 3,248}, {11,122}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 15}, { 3,  9}, { 5, 12}, { 3,248}, {11, 58}, 
    { 3,  8}, { 4,246}, { 3,247}, { 7, 17}, { 3,  9}, { 5,245}, { 3,248}, {11,186}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 14}, { 3,  9}, { 5, 11}, { 3,248}, {11, 26}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,243}, { 3,  9}, { 5,244}, { 3,248}, {11,154}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6,241}, { 3,  9}, { 5, 12}, { 3,248}, {11, 90}, 
    { 3,  8}, { 4,246}, { 3,247}, { 8, 18}, { 3,  9}, { 5,245}, { 3,248}, {11,218}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 13}, { 3,  9}, { 5, 11}, { 3,248}, { 8,238}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,242}, { 3,  9}, { 5,244}, { 3,248}, {11,114}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 15}, { 3,  9}, { 5, 12}, { 3,248}, {11, 50}, 
    { 3,  8}, { 4,246}, { 3,247}, { 7, 16}, { 3,  9}, { 5,245}, { 3,248}, {11,178}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 14}, { 3,  9}, { 5, 11}, { 3,248}, {10, 22}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,243}, { 3,  9}, { 5,244}, { 3,248}, {11,146}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6,241}, { 3,  9}, { 5, 12}, { 3,248}, {11, 82}, 
    { 3,  8}, { 4,246}, { 3,247}, { 7,240}, { 3,  9}, { 5,245}, { 3,248}, {11,210}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 13}, { 3,  9}, { 5, 11}, { 3,248}, { 9, 20}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,242}, { 3,  9}, { 5,244}, { 3,248}, {11,130}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 15}, { 3,  9}, { 5, 12}, { 3,248}, {11, 66}, 
    { 3,  8}, { 4,246}, { 3,247}, { 7, 17}, { 3,  9}, { 5,245}, { 3,248}, {11,194}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 14}, { 3,  9}, { 5, 11}, { 3,248}, {11, 34}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,243}, { 3,  9}, { 5,244}, { 3,248}, {11,162}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6,241}, { 3,  9}, { 5, 12}, { 3,248}, {11, 98}, 
    { 3,  8}, { 4,246}, { 3,247}, { 8, 19}, { 3,  9}, { 5,245}, { 3,248}, {11,226}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 13}, { 3,  9}, { 5, 11}, { 3,248}, { 8,237}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,242}, { 3,  9}, { 5,244}, { 3,248}, {11,110}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 15}, { 3,  9}, { 5, 12}, { 3,248}, {11, 46}, 
    { 3,  8}, { 4,246}, { 3,247}, { 7, 16}, { 3,  9}, { 5,245}, { 3,248}, {11,174}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 14}, { 3,  9}, { 5, 11}, { 3,248}, { 9,236}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,243}, { 3,  9}, { 5,244}, { 3,248}, {11,142}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6,241}, { 3,  9}, { 5, 12}, { 3,248}, {11, 78}, 
    { 3,  8}, { 4,246}, { 3,247}, { 7,240}, { 3,  9}, { 5,245}, { 3,248}, {11,206}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 13}, { 3,  9}, { 5, 11}, { 3,248}, { 8,239}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,242}, { 3,  9}, { 5,244}, { 3,248}, {11,126}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 15}, { 3,  9}, { 5, 12}, { 3,248}, {11, 62}, 
    { 3,  8}, { 4,246}, { 3,247}, { 7, 17}, { 3,  9}, { 5,245}, { 3,248}, {11,190}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 14}, { 3,  9}, { 5, 11}, { 3,248}, {11, 30}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,243}, { 3,  9}, { 5,244}, { 3,248}, {11,158}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6,241}, { 3,  9}, { 5, 12}, { 3,248}, {11, 94}, 
    { 3,  8}, { 4,246}, { 3,247}, { 8, 18}, { 3,  9}, { 5,245}, { 3,248}, {11,222}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 13}, { 3,  9}, { 5, 11}, { 3,248}, { 8,238}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,242}, { 3,  9}, { 5,244}, { 3,248}, {11,118}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 15}, { 3,  9}, { 5, 12}, { 3,248}, {11, 54}, 
    { 3,  8}, { 4,246}, { 3,247}, { 7, 16}, { 3,  9}, { 5,245}, { 3,248}, {11,182}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 14}, { 3,  9}, { 5, 11}, { 3,248}, {10,234}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,243}, { 3,  9}, { 5,244}, { 3,248}, {11,150}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6,241}, { 3,  9}, { 5, 12}, { 3,248}, {11, 86}, 
    { 3,  8}, { 4,246}, { 3,247}, { 7,240}, { 3,  9}, { 5,245}, { 3,248}, {11,214}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 13}, { 3,  9}, { 5, 11}, { 3,248}, { 9, 21}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,242}, { 3,  9}, { 5,244}, { 3,248}, {11,134}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 15}, { 3,  9}, { 5, 12}, { 3,248}, {11, 70}, 
    { 3,  8}, { 4,246}, { 3,247}, { 7, 17}, { 3,  9}, { 5,245}, { 3,248}, {11,198}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 14}, { 3,  9}, { 5, 11}, { 3,248}, {11, 38}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,243}, { 3,  9}, { 5,244}, { 3,248}, {11,166}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6,241}, { 3,  9}, { 5, 12}, { 3,248}, {11,102}, 
    { 3,  8}, { 4,246}, { 3,247}, { 8, 19}, { 3,  9}, { 5,245}, { 3,248}, {11,230}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 13}, { 3,  9}, { 5, 11}, { 3,248}, { 8,237}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,242}, { 3,  9}, { 5,244}, { 3,248}, {11,108}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 15}, { 3,  9}, { 5, 12}, { 3,248}, {11, 44}, 
    { 3,  8}, { 4,246}, { 3,247}, { 7, 16}, { 3,  9}, { 5,245}, { 3,248}, {11,172}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 14}, { 3,  9}, { 5, 11}, { 3,248}, { 9,235}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,243}, { 3,  9}, { 5,244}, { 3,248}, {11,140}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6,241}, { 3,  9}, { 5, 12}, { 3,248}, {11, 76}, 
    { 3,  8}, { 4,246}, { 3,247}, { 7,240}, { 3,  9}, { 5,245}, { 3,248}, {11,204}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 13}, { 3,  9}, { 5, 11}, { 3,248}, { 8,239}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,242}, { 3,  9}, { 5,244}, { 3,248}, {11,124}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 15}, { 3,  9}, { 5, 12}, { 3,248}, {11, 60}, 
    { 3,  8}, { 4,246}, { 3,247}, { 7, 17}, { 3,  9}, { 5,245}, { 3,248}, {11,188}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 14}, { 3,  9}, { 5, 11}, { 3,248}, {11, 28}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,243}, { 3,  9}, { 5,244}, { 3,248}, {11,156}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6,241}, { 3,  9}, { 5, 12}, { 3,248}, {11, 92}, 
    { 3,  8}, { 4,246}, { 3,247}, { 8, 18}, { 3,  9}, { 5,245}, { 3,248}, {11,220}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 13}, { 3,  9}, { 5, 11}, { 3,248}, { 8,238}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,242}, { 3,  9}, { 5,244}, { 3,248}, {11,116}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 15}, { 3,  9}, { 5, 12}, { 3,248}, {11, 52}, 
    { 3,  8}, { 4,246}, { 3,247}, { 7, 16}, { 3,  9}, { 5,245}, { 3,248}, {11,180}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 14}, { 3,  9}, { 5, 11}, { 3,248}, {10,233}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,243}, { 3,  9}, { 5,244}, { 3,248}, {11,148}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6,241}, { 3,  9}, { 5, 12}, { 3,248}, {11, 84}, 
    { 3,  8}, { 4,246}, { 3,247}, { 7,240}, { 3,  9}, { 5,245}, { 3,248}, {11,212}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 13}, { 3,  9}, { 5, 11}, { 3,248}, { 9, 20}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,242}, { 3,  9}, { 5,244}, { 3,248}, {11,132}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 15}, { 3,  9}, { 5, 12}, { 3,248}, {11, 68}, 
    { 3,  8}, { 4,246}, { 3,247}, { 7, 17}, { 3,  9}, { 5,245}, { 3,248}, {11,196}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 14}, { 3,  9}, { 5, 11}, { 3,248}, {11, 36}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,243}, { 3,  9}, { 5,244}, { 3,248}, {11,164}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6,241}, { 3,  9}, { 5, 12}, { 3,248}, {11,100}, 
    { 3,  8}, { 4,246}, { 3,247}, { 8, 19}, { 3,  9}, { 5,245}, { 3,248}, {11,228}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 13}, { 3,  9}, { 5, 11}, { 3,248}, { 8,237}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,242}, { 3,  9}, { 5,244}, { 3,248}, {11,112}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 15}, { 3,  9}, { 5, 12}, { 3,248}, {11, 48}, 
    { 3,  8}, { 4,246}, { 3,247}, { 7, 16}, { 3,  9}, { 5,245}, { 3,248}, {11,176}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 14}, { 3,  9}, { 5, 11}, { 3,248}, { 9,236}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,243}, { 3,  9}, { 5,244}, { 3,248}, {11,144}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6,241}, { 3,  9}, { 5, 12}, { 3,248}, {11, 80}, 
    { 3,  8}, { 4,246}, { 3,247}, { 7,240}, { 3,  9}, { 5,245}, { 3,248}, {11,208}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 13}, { 3,  9}, { 5, 11}, { 3,248}, { 8,239}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,242}, { 3,  9}, { 5,244}, { 3,248}, {11,128}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 15}, { 3,  9}, { 5, 12}, { 3,248}, {11, 64}, 
    { 3,  8}, { 4,246}, { 3,247}, { 7, 17}, { 3,  9}, { 5,245}, { 3,248}, {11,192}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 14}, { 3,  9}, { 5, 11}, { 3,248}, {11, 32}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,243}, { 3,  9}, { 5,244}, { 3,248}, {11,160}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6,241}, { 3,  9}, { 5, 12}, { 3,248}, {11, 96}, 
    { 3,  8}, { 4,246}, { 3,247}, { 8, 18}, { 3,  9}, { 5,245}, { 3,248}, {11,224}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 13}, { 3,  9}, { 5, 11}, { 3,248}, { 8,238}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,242}, { 3,  9}, { 5,244}, { 3,248}, {11,120}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 15}, { 3,  9}, { 5, 12}, { 3,248}, {11, 56}, 
    { 3,  8}, { 4,246}, { 3,247}, { 7, 16}, { 3,  9}, { 5,245}, { 3,248}, {11,184}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 14}, { 3,  9}, { 5, 11}, { 3,248}, {11, 24}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,243}, { 3,  9}, { 5,244}, { 3,248}, {11,152}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6,241}, { 3,  9}, { 5, 12}, { 3,248}, {11, 88}, 
    { 3,  8}, { 4,246}, { 3,247}, { 7,240}, { 3,  9}, { 5,245}, { 3,248}, {11,216}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 13}, { 3,  9}, { 5, 11}, { 3,248}, { 9, 21}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,242}, { 3,  9}, { 5,244}, { 3,248}, {11,136}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 15}, { 3,  9}, { 5, 12}, { 3,248}, {11, 72}, 
    { 3,  8}, { 4,246}, { 3,247}, { 7, 17}, { 3,  9}, { 5,245}, { 3,248}, {11,200}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6, 14}, { 3,  9}, { 5, 11}, { 3,248}, {11, 40}, 
    { 3,  8}, { 4,246}, { 3,247}, { 6,243}, { 3,  9}, { 5,244}, { 3,248}, {11,168}, 
    { 3,  8}, { 4, 10}, { 3,247}, { 6,241}, { 3,  9}, { 5, 12}, { 3,248}, {11,104}, 
    { 3,  8}, { 4,246}, { 3,247}, { 8, 19}, { 3,  9}, { 5,245}, { 3,248}, {11,232}, 
  }
  #endif
};

static radaudio_huffman rada_nonzero_coefficient_big_huff1 =
{
  #ifdef HUFFMAN_ENCODE
  {
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 3}, {   2, 4}, {  10, 4}, {   6, 4}, {   9, 5}, {  25, 5}, {  29, 6}, {  61, 6}, 
    {   3, 6}, {  11, 7}, {  75, 7}, {  43, 7}, {  59, 8}, { 187, 8}, { 123, 8}, {  71, 9}, 
    { 327, 9}, { 199, 9}, { 455, 9}, { 423,10}, { 935,10}, { 103,10}, { 487,11}, {1511,11}, 
    { 999,11}, {2023,11}, {  23,11}, {1047,11}, { 535,11}, {1559,11}, { 279,11}, {1303,11}, 
    { 791,11}, {1815,11}, { 151,11}, {1175,11}, { 663,11}, {1687,11}, { 407,11}, {1431,11}, 
    { 919,11}, {1943,11}, {  87,11}, {1111,11}, { 599,11}, {1623,11}, { 343,11}, {1367,11}, 
    { 855,11}, {1879,11}, { 215,11}, {1239,11}, { 727,11}, {1751,11}, { 471,11}, {1495,11}, 
    { 983,11}, {2007,11}, {  55,11}, {1079,11}, { 567,11}, {1591,11}, { 311,11}, {1335,11}, 
    { 823,11}, {1847,11}, { 183,11}, {1207,11}, { 695,11}, {1719,11}, { 439,11}, {1463,11}, 
    { 951,11}, {1975,11}, { 119,11}, {1143,11}, { 631,11}, {1655,11}, { 375,11}, {1399,11}, 
    { 887,11}, {1911,11}, { 247,11}, {1271,11}, { 759,11}, {1783,11}, { 503,11}, {1527,11}, 
    {1015,11}, {2039,11}, {  15,11}, {1039,11}, { 527,11}, {1551,11}, { 271,11}, {1295,11}, 
    { 783,11}, {1807,11}, { 143,11}, {1167,11}, { 655,11}, {1679,11}, { 399,11}, {1423,11}, 
    { 911,11}, {1935,11}, {  79,11}, {1103,11}, { 591,11}, {1615,11}, { 335,11}, {1359,11}, 
    { 847,11}, {1871,11}, { 207,11}, {1231,11}, { 719,11}, {1743,11}, { 463,11}, {1487,11}, 
    { 975,11}, {1999,11}, {  47,11}, {1071,11}, { 559,11}, {1583,11}, { 303,11}, {1327,11}, 
    { 815,11}, {1839,11}, { 175,11}, {1199,11}, { 687,11}, {1711,11}, { 431,11}, {1455,11}, 
    { 943,11}, {1967,11}, { 111,11}, {1135,11}, { 623,11}, {1647,11}, { 367,11}, {1391,11}, 
    { 879,11}, {1903,11}, { 239,11}, {1263,11}, { 751,11}, {1775,11}, { 495,11}, {1519,11}, 
    {1007,11}, {2031,11}, {  31,11}, {1055,11}, { 543,11}, {1567,11}, { 287,11}, {1311,11}, 
    { 799,11}, {1823,11}, { 159,11}, {1183,11}, { 671,11}, {1695,11}, { 415,11}, {1439,11}, 
    { 927,11}, {1951,11}, {  95,11}, {1119,11}, { 607,11}, {1631,11}, { 351,11}, {1375,11}, 
    { 863,11}, {1887,11}, { 223,11}, {1247,11}, { 735,11}, {1759,11}, { 479,11}, {1503,11}, 
    { 991,11}, {2015,11}, {  63,11}, {1087,11}, { 575,11}, {1599,11}, { 319,11}, {1343,11}, 
    { 831,11}, {1855,11}, { 191,11}, {1215,11}, { 703,11}, {1727,11}, { 447,11}, {1471,11}, 
    { 959,11}, {1983,11}, { 127,11}, {1151,11}, { 639,11}, {1663,11}, { 383,11}, {1407,11}, 
    { 895,11}, {1919,11}, { 255,11}, {1279,11}, { 767,11}, {1791,11}, { 511,11}, {1535,11}, 
    {1023,11}, {2047,11}, { 615,10}, { 359,10}, { 871,10}, { 231,10}, { 743,10}, {  39, 9}, 
    { 295, 9}, { 167, 9}, { 251, 8}, {   7, 8}, { 135, 8}, { 107, 7}, {  27, 7}, {  91, 7}, 
    {  35, 6}, {  19, 6}, {  51, 6}, {   5, 5}, {  21, 5}, {  13, 5}, {  14, 4}, {   1, 4}, 
    {   4, 3}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
  },
  #endif
  #ifdef HUFFMAN_DECODE
  {
    { 3,  8}, { 4,247}, { 4,  9}, { 6, 16}, { 3,248}, { 5,243}, { 4, 11}, { 8,235}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7, 17}, { 3,248}, { 5,245}, { 4,246}, {11, 98}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,241}, { 3,248}, { 5,244}, { 4, 11}, {11, 34}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 7,238}, { 3,248}, { 6, 14}, { 4,246}, {11,162}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,240}, { 3,248}, { 5,243}, { 4, 11}, { 9,231}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7, 19}, { 3,248}, { 5,245}, { 4,246}, {11,130}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,242}, { 3,248}, { 5,244}, { 4, 11}, {11, 66}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 8, 20}, { 3,248}, { 6, 15}, { 4,246}, {11,194}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6, 16}, { 3,248}, { 5,243}, { 4, 11}, { 9, 23}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7, 18}, { 3,248}, { 5,245}, { 4,246}, {11,114}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,241}, { 3,248}, { 5,244}, { 4, 11}, {11, 50}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 7,239}, { 3,248}, { 6, 14}, { 4,246}, {11,178}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,240}, { 3,248}, { 5,243}, { 4, 11}, {10, 29}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7,237}, { 3,248}, { 5,245}, { 4,246}, {11,146}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,242}, { 3,248}, { 5,244}, { 4, 11}, {11, 82}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 8, 22}, { 3,248}, { 6, 15}, { 4,246}, {11,210}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6, 16}, { 3,248}, { 5,243}, { 4, 11}, { 8,236}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7, 17}, { 3,248}, { 5,245}, { 4,246}, {11,106}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,241}, { 3,248}, { 5,244}, { 4, 11}, {11, 42}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 7,238}, { 3,248}, { 6, 14}, { 4,246}, {11,170}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,240}, { 3,248}, { 5,243}, { 4, 11}, { 9,233}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7, 19}, { 3,248}, { 5,245}, { 4,246}, {11,138}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,242}, { 3,248}, { 5,244}, { 4, 11}, {11, 74}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 8, 21}, { 3,248}, { 6, 15}, { 4,246}, {11,202}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6, 16}, { 3,248}, { 5,243}, { 4, 11}, { 9, 25}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7, 18}, { 3,248}, { 5,245}, { 4,246}, {11,122}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,241}, { 3,248}, { 5,244}, { 4, 11}, {11, 58}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 7,239}, { 3,248}, { 6, 14}, { 4,246}, {11,186}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,240}, { 3,248}, { 5,243}, { 4, 11}, {10,229}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7,237}, { 3,248}, { 5,245}, { 4,246}, {11,154}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,242}, { 3,248}, { 5,244}, { 4, 11}, {11, 90}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 8,234}, { 3,248}, { 6, 15}, { 4,246}, {11,218}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6, 16}, { 3,248}, { 5,243}, { 4, 11}, { 8,235}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7, 17}, { 3,248}, { 5,245}, { 4,246}, {11,102}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,241}, { 3,248}, { 5,244}, { 4, 11}, {11, 38}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 7,238}, { 3,248}, { 6, 14}, { 4,246}, {11,166}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,240}, { 3,248}, { 5,243}, { 4, 11}, { 9,232}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7, 19}, { 3,248}, { 5,245}, { 4,246}, {11,134}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,242}, { 3,248}, { 5,244}, { 4, 11}, {11, 70}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 8, 20}, { 3,248}, { 6, 15}, { 4,246}, {11,198}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6, 16}, { 3,248}, { 5,243}, { 4, 11}, { 9, 24}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7, 18}, { 3,248}, { 5,245}, { 4,246}, {11,118}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,241}, { 3,248}, { 5,244}, { 4, 11}, {11, 54}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 7,239}, { 3,248}, { 6, 14}, { 4,246}, {11,182}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,240}, { 3,248}, { 5,243}, { 4, 11}, {10,227}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7,237}, { 3,248}, { 5,245}, { 4,246}, {11,150}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,242}, { 3,248}, { 5,244}, { 4, 11}, {11, 86}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 8, 22}, { 3,248}, { 6, 15}, { 4,246}, {11,214}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6, 16}, { 3,248}, { 5,243}, { 4, 11}, { 8,236}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7, 17}, { 3,248}, { 5,245}, { 4,246}, {11,110}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,241}, { 3,248}, { 5,244}, { 4, 11}, {11, 46}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 7,238}, { 3,248}, { 6, 14}, { 4,246}, {11,174}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,240}, { 3,248}, { 5,243}, { 4, 11}, {10, 27}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7, 19}, { 3,248}, { 5,245}, { 4,246}, {11,142}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,242}, { 3,248}, { 5,244}, { 4, 11}, {11, 78}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 8, 21}, { 3,248}, { 6, 15}, { 4,246}, {11,206}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6, 16}, { 3,248}, { 5,243}, { 4, 11}, { 9, 26}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7, 18}, { 3,248}, { 5,245}, { 4,246}, {11,126}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,241}, { 3,248}, { 5,244}, { 4, 11}, {11, 62}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 7,239}, { 3,248}, { 6, 14}, { 4,246}, {11,190}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,240}, { 3,248}, { 5,243}, { 4, 11}, {11, 30}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7,237}, { 3,248}, { 5,245}, { 4,246}, {11,158}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,242}, { 3,248}, { 5,244}, { 4, 11}, {11, 94}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 8,234}, { 3,248}, { 6, 15}, { 4,246}, {11,222}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6, 16}, { 3,248}, { 5,243}, { 4, 11}, { 8,235}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7, 17}, { 3,248}, { 5,245}, { 4,246}, {11,100}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,241}, { 3,248}, { 5,244}, { 4, 11}, {11, 36}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 7,238}, { 3,248}, { 6, 14}, { 4,246}, {11,164}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,240}, { 3,248}, { 5,243}, { 4, 11}, { 9,231}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7, 19}, { 3,248}, { 5,245}, { 4,246}, {11,132}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,242}, { 3,248}, { 5,244}, { 4, 11}, {11, 68}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 8, 20}, { 3,248}, { 6, 15}, { 4,246}, {11,196}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6, 16}, { 3,248}, { 5,243}, { 4, 11}, { 9, 23}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7, 18}, { 3,248}, { 5,245}, { 4,246}, {11,116}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,241}, { 3,248}, { 5,244}, { 4, 11}, {11, 52}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 7,239}, { 3,248}, { 6, 14}, { 4,246}, {11,180}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,240}, { 3,248}, { 5,243}, { 4, 11}, {10,226}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7,237}, { 3,248}, { 5,245}, { 4,246}, {11,148}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,242}, { 3,248}, { 5,244}, { 4, 11}, {11, 84}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 8, 22}, { 3,248}, { 6, 15}, { 4,246}, {11,212}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6, 16}, { 3,248}, { 5,243}, { 4, 11}, { 8,236}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7, 17}, { 3,248}, { 5,245}, { 4,246}, {11,108}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,241}, { 3,248}, { 5,244}, { 4, 11}, {11, 44}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 7,238}, { 3,248}, { 6, 14}, { 4,246}, {11,172}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,240}, { 3,248}, { 5,243}, { 4, 11}, { 9,233}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7, 19}, { 3,248}, { 5,245}, { 4,246}, {11,140}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,242}, { 3,248}, { 5,244}, { 4, 11}, {11, 76}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 8, 21}, { 3,248}, { 6, 15}, { 4,246}, {11,204}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6, 16}, { 3,248}, { 5,243}, { 4, 11}, { 9, 25}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7, 18}, { 3,248}, { 5,245}, { 4,246}, {11,124}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,241}, { 3,248}, { 5,244}, { 4, 11}, {11, 60}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 7,239}, { 3,248}, { 6, 14}, { 4,246}, {11,188}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,240}, { 3,248}, { 5,243}, { 4, 11}, {10,230}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7,237}, { 3,248}, { 5,245}, { 4,246}, {11,156}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,242}, { 3,248}, { 5,244}, { 4, 11}, {11, 92}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 8,234}, { 3,248}, { 6, 15}, { 4,246}, {11,220}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6, 16}, { 3,248}, { 5,243}, { 4, 11}, { 8,235}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7, 17}, { 3,248}, { 5,245}, { 4,246}, {11,104}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,241}, { 3,248}, { 5,244}, { 4, 11}, {11, 40}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 7,238}, { 3,248}, { 6, 14}, { 4,246}, {11,168}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,240}, { 3,248}, { 5,243}, { 4, 11}, { 9,232}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7, 19}, { 3,248}, { 5,245}, { 4,246}, {11,136}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,242}, { 3,248}, { 5,244}, { 4, 11}, {11, 72}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 8, 20}, { 3,248}, { 6, 15}, { 4,246}, {11,200}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6, 16}, { 3,248}, { 5,243}, { 4, 11}, { 9, 24}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7, 18}, { 3,248}, { 5,245}, { 4,246}, {11,120}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,241}, { 3,248}, { 5,244}, { 4, 11}, {11, 56}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 7,239}, { 3,248}, { 6, 14}, { 4,246}, {11,184}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,240}, { 3,248}, { 5,243}, { 4, 11}, {10,228}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7,237}, { 3,248}, { 5,245}, { 4,246}, {11,152}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,242}, { 3,248}, { 5,244}, { 4, 11}, {11, 88}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 8, 22}, { 3,248}, { 6, 15}, { 4,246}, {11,216}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6, 16}, { 3,248}, { 5,243}, { 4, 11}, { 8,236}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7, 17}, { 3,248}, { 5,245}, { 4,246}, {11,112}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,241}, { 3,248}, { 5,244}, { 4, 11}, {11, 48}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 7,238}, { 3,248}, { 6, 14}, { 4,246}, {11,176}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,240}, { 3,248}, { 5,243}, { 4, 11}, {10, 28}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7, 19}, { 3,248}, { 5,245}, { 4,246}, {11,144}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,242}, { 3,248}, { 5,244}, { 4, 11}, {11, 80}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 8, 21}, { 3,248}, { 6, 15}, { 4,246}, {11,208}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6, 16}, { 3,248}, { 5,243}, { 4, 11}, { 9, 26}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7, 18}, { 3,248}, { 5,245}, { 4,246}, {11,128}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,241}, { 3,248}, { 5,244}, { 4, 11}, {11, 64}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 7,239}, { 3,248}, { 6, 14}, { 4,246}, {11,192}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,240}, { 3,248}, { 5,243}, { 4, 11}, {11, 32}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7,237}, { 3,248}, { 5,245}, { 4,246}, {11,160}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,242}, { 3,248}, { 5,244}, { 4, 11}, {11, 96}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 8,234}, { 3,248}, { 6, 15}, { 4,246}, {11,224}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6, 16}, { 3,248}, { 5,243}, { 4, 11}, { 8,235}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7, 17}, { 3,248}, { 5,245}, { 4,246}, {11, 99}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,241}, { 3,248}, { 5,244}, { 4, 11}, {11, 35}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 7,238}, { 3,248}, { 6, 14}, { 4,246}, {11,163}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,240}, { 3,248}, { 5,243}, { 4, 11}, { 9,231}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7, 19}, { 3,248}, { 5,245}, { 4,246}, {11,131}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,242}, { 3,248}, { 5,244}, { 4, 11}, {11, 67}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 8, 20}, { 3,248}, { 6, 15}, { 4,246}, {11,195}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6, 16}, { 3,248}, { 5,243}, { 4, 11}, { 9, 23}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7, 18}, { 3,248}, { 5,245}, { 4,246}, {11,115}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,241}, { 3,248}, { 5,244}, { 4, 11}, {11, 51}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 7,239}, { 3,248}, { 6, 14}, { 4,246}, {11,179}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,240}, { 3,248}, { 5,243}, { 4, 11}, {10, 29}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7,237}, { 3,248}, { 5,245}, { 4,246}, {11,147}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,242}, { 3,248}, { 5,244}, { 4, 11}, {11, 83}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 8, 22}, { 3,248}, { 6, 15}, { 4,246}, {11,211}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6, 16}, { 3,248}, { 5,243}, { 4, 11}, { 8,236}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7, 17}, { 3,248}, { 5,245}, { 4,246}, {11,107}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,241}, { 3,248}, { 5,244}, { 4, 11}, {11, 43}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 7,238}, { 3,248}, { 6, 14}, { 4,246}, {11,171}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,240}, { 3,248}, { 5,243}, { 4, 11}, { 9,233}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7, 19}, { 3,248}, { 5,245}, { 4,246}, {11,139}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,242}, { 3,248}, { 5,244}, { 4, 11}, {11, 75}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 8, 21}, { 3,248}, { 6, 15}, { 4,246}, {11,203}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6, 16}, { 3,248}, { 5,243}, { 4, 11}, { 9, 25}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7, 18}, { 3,248}, { 5,245}, { 4,246}, {11,123}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,241}, { 3,248}, { 5,244}, { 4, 11}, {11, 59}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 7,239}, { 3,248}, { 6, 14}, { 4,246}, {11,187}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,240}, { 3,248}, { 5,243}, { 4, 11}, {10,229}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7,237}, { 3,248}, { 5,245}, { 4,246}, {11,155}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,242}, { 3,248}, { 5,244}, { 4, 11}, {11, 91}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 8,234}, { 3,248}, { 6, 15}, { 4,246}, {11,219}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6, 16}, { 3,248}, { 5,243}, { 4, 11}, { 8,235}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7, 17}, { 3,248}, { 5,245}, { 4,246}, {11,103}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,241}, { 3,248}, { 5,244}, { 4, 11}, {11, 39}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 7,238}, { 3,248}, { 6, 14}, { 4,246}, {11,167}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,240}, { 3,248}, { 5,243}, { 4, 11}, { 9,232}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7, 19}, { 3,248}, { 5,245}, { 4,246}, {11,135}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,242}, { 3,248}, { 5,244}, { 4, 11}, {11, 71}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 8, 20}, { 3,248}, { 6, 15}, { 4,246}, {11,199}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6, 16}, { 3,248}, { 5,243}, { 4, 11}, { 9, 24}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7, 18}, { 3,248}, { 5,245}, { 4,246}, {11,119}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,241}, { 3,248}, { 5,244}, { 4, 11}, {11, 55}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 7,239}, { 3,248}, { 6, 14}, { 4,246}, {11,183}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,240}, { 3,248}, { 5,243}, { 4, 11}, {10,227}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7,237}, { 3,248}, { 5,245}, { 4,246}, {11,151}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,242}, { 3,248}, { 5,244}, { 4, 11}, {11, 87}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 8, 22}, { 3,248}, { 6, 15}, { 4,246}, {11,215}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6, 16}, { 3,248}, { 5,243}, { 4, 11}, { 8,236}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7, 17}, { 3,248}, { 5,245}, { 4,246}, {11,111}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,241}, { 3,248}, { 5,244}, { 4, 11}, {11, 47}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 7,238}, { 3,248}, { 6, 14}, { 4,246}, {11,175}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,240}, { 3,248}, { 5,243}, { 4, 11}, {10, 27}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7, 19}, { 3,248}, { 5,245}, { 4,246}, {11,143}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,242}, { 3,248}, { 5,244}, { 4, 11}, {11, 79}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 8, 21}, { 3,248}, { 6, 15}, { 4,246}, {11,207}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6, 16}, { 3,248}, { 5,243}, { 4, 11}, { 9, 26}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7, 18}, { 3,248}, { 5,245}, { 4,246}, {11,127}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,241}, { 3,248}, { 5,244}, { 4, 11}, {11, 63}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 7,239}, { 3,248}, { 6, 14}, { 4,246}, {11,191}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,240}, { 3,248}, { 5,243}, { 4, 11}, {11, 31}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7,237}, { 3,248}, { 5,245}, { 4,246}, {11,159}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,242}, { 3,248}, { 5,244}, { 4, 11}, {11, 95}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 8,234}, { 3,248}, { 6, 15}, { 4,246}, {11,223}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6, 16}, { 3,248}, { 5,243}, { 4, 11}, { 8,235}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7, 17}, { 3,248}, { 5,245}, { 4,246}, {11,101}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,241}, { 3,248}, { 5,244}, { 4, 11}, {11, 37}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 7,238}, { 3,248}, { 6, 14}, { 4,246}, {11,165}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,240}, { 3,248}, { 5,243}, { 4, 11}, { 9,231}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7, 19}, { 3,248}, { 5,245}, { 4,246}, {11,133}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,242}, { 3,248}, { 5,244}, { 4, 11}, {11, 69}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 8, 20}, { 3,248}, { 6, 15}, { 4,246}, {11,197}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6, 16}, { 3,248}, { 5,243}, { 4, 11}, { 9, 23}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7, 18}, { 3,248}, { 5,245}, { 4,246}, {11,117}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,241}, { 3,248}, { 5,244}, { 4, 11}, {11, 53}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 7,239}, { 3,248}, { 6, 14}, { 4,246}, {11,181}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,240}, { 3,248}, { 5,243}, { 4, 11}, {10,226}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7,237}, { 3,248}, { 5,245}, { 4,246}, {11,149}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,242}, { 3,248}, { 5,244}, { 4, 11}, {11, 85}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 8, 22}, { 3,248}, { 6, 15}, { 4,246}, {11,213}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6, 16}, { 3,248}, { 5,243}, { 4, 11}, { 8,236}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7, 17}, { 3,248}, { 5,245}, { 4,246}, {11,109}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,241}, { 3,248}, { 5,244}, { 4, 11}, {11, 45}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 7,238}, { 3,248}, { 6, 14}, { 4,246}, {11,173}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,240}, { 3,248}, { 5,243}, { 4, 11}, { 9,233}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7, 19}, { 3,248}, { 5,245}, { 4,246}, {11,141}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,242}, { 3,248}, { 5,244}, { 4, 11}, {11, 77}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 8, 21}, { 3,248}, { 6, 15}, { 4,246}, {11,205}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6, 16}, { 3,248}, { 5,243}, { 4, 11}, { 9, 25}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7, 18}, { 3,248}, { 5,245}, { 4,246}, {11,125}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,241}, { 3,248}, { 5,244}, { 4, 11}, {11, 61}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 7,239}, { 3,248}, { 6, 14}, { 4,246}, {11,189}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,240}, { 3,248}, { 5,243}, { 4, 11}, {10,230}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7,237}, { 3,248}, { 5,245}, { 4,246}, {11,157}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,242}, { 3,248}, { 5,244}, { 4, 11}, {11, 93}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 8,234}, { 3,248}, { 6, 15}, { 4,246}, {11,221}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6, 16}, { 3,248}, { 5,243}, { 4, 11}, { 8,235}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7, 17}, { 3,248}, { 5,245}, { 4,246}, {11,105}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,241}, { 3,248}, { 5,244}, { 4, 11}, {11, 41}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 7,238}, { 3,248}, { 6, 14}, { 4,246}, {11,169}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,240}, { 3,248}, { 5,243}, { 4, 11}, { 9,232}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7, 19}, { 3,248}, { 5,245}, { 4,246}, {11,137}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,242}, { 3,248}, { 5,244}, { 4, 11}, {11, 73}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 8, 20}, { 3,248}, { 6, 15}, { 4,246}, {11,201}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6, 16}, { 3,248}, { 5,243}, { 4, 11}, { 9, 24}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7, 18}, { 3,248}, { 5,245}, { 4,246}, {11,121}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,241}, { 3,248}, { 5,244}, { 4, 11}, {11, 57}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 7,239}, { 3,248}, { 6, 14}, { 4,246}, {11,185}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,240}, { 3,248}, { 5,243}, { 4, 11}, {10,228}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7,237}, { 3,248}, { 5,245}, { 4,246}, {11,153}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,242}, { 3,248}, { 5,244}, { 4, 11}, {11, 89}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 8, 22}, { 3,248}, { 6, 15}, { 4,246}, {11,217}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6, 16}, { 3,248}, { 5,243}, { 4, 11}, { 8,236}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7, 17}, { 3,248}, { 5,245}, { 4,246}, {11,113}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,241}, { 3,248}, { 5,244}, { 4, 11}, {11, 49}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 7,238}, { 3,248}, { 6, 14}, { 4,246}, {11,177}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,240}, { 3,248}, { 5,243}, { 4, 11}, {10, 28}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7, 19}, { 3,248}, { 5,245}, { 4,246}, {11,145}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,242}, { 3,248}, { 5,244}, { 4, 11}, {11, 81}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 8, 21}, { 3,248}, { 6, 15}, { 4,246}, {11,209}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6, 16}, { 3,248}, { 5,243}, { 4, 11}, { 9, 26}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7, 18}, { 3,248}, { 5,245}, { 4,246}, {11,129}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,241}, { 3,248}, { 5,244}, { 4, 11}, {11, 65}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 7,239}, { 3,248}, { 6, 14}, { 4,246}, {11,193}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,240}, { 3,248}, { 5,243}, { 4, 11}, {11, 33}, 
    { 3,  8}, { 5, 12}, { 4, 10}, { 7,237}, { 3,248}, { 5,245}, { 4,246}, {11,161}, 
    { 3,  8}, { 4,247}, { 4,  9}, { 6,242}, { 3,248}, { 5,244}, { 4, 11}, {11, 97}, 
    { 3,  8}, { 5, 13}, { 4, 10}, { 8,234}, { 3,248}, { 6, 15}, { 4,246}, {11,225}, 
  }
  #endif
};

static radaudio_huffman rada_nonzero_coefficient_big_huff2 =
{
  #ifdef HUFFMAN_ENCODE
  {
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 4}, {   8, 4}, {   4, 4}, {   6, 5}, {  22, 5}, {  14, 5}, {  30, 5}, {   5, 6}, 
    {  37, 6}, {  21, 6}, {  53, 6}, {  13, 6}, {  35, 7}, {  99, 7}, {  19, 7}, {  83, 7}, 
    { 107, 8}, { 235, 8}, {  27, 8}, { 155, 8}, {  91, 8}, { 219, 8}, { 135, 9}, { 391, 9}, 
    {  71, 9}, { 327, 9}, { 199, 9}, { 455, 9}, { 231,10}, { 743,10}, { 487,10}, { 999,10}, 
    {  23,10}, { 535,10}, { 279,10}, { 343,11}, {1367,11}, { 855,11}, {1879,11}, { 215,11}, 
    {1239,11}, { 727,11}, {1751,11}, { 471,11}, {1495,11}, { 983,11}, {2007,11}, {  55,11}, 
    {1079,11}, { 567,11}, {1591,11}, { 311,11}, {1335,11}, { 823,11}, {1847,11}, { 183,11}, 
    {1207,11}, { 695,11}, {1719,11}, { 439,11}, {1463,11}, { 951,11}, {1975,11}, { 119,11}, 
    {1143,11}, { 631,11}, {1655,11}, { 375,11}, {1399,11}, { 887,11}, {1911,11}, { 247,11}, 
    {1271,11}, { 759,11}, {1783,11}, { 503,11}, {1527,11}, {1015,11}, {2039,11}, {  15,11}, 
    {1039,11}, { 527,11}, {1551,11}, { 271,11}, {1295,11}, { 783,11}, {1807,11}, { 143,11}, 
    {1167,11}, { 655,11}, {1679,11}, { 399,11}, {1423,11}, { 911,11}, {1935,11}, {  79,11}, 
    {1103,11}, { 591,11}, {1615,11}, { 335,11}, {1359,11}, { 847,11}, {1871,11}, { 207,11}, 
    {1231,11}, { 719,11}, {1743,11}, { 463,11}, {1487,11}, { 975,11}, {1999,11}, {  47,11}, 
    {1071,11}, { 559,11}, {1583,11}, { 303,11}, {1327,11}, { 815,11}, {1839,11}, { 175,11}, 
    {1199,11}, { 687,11}, {1711,11}, { 431,11}, {1455,11}, { 943,11}, {1967,11}, { 111,11}, 
    {1135,11}, { 623,11}, {1647,11}, { 367,11}, {1391,11}, { 879,11}, {1903,11}, { 239,11}, 
    {1263,11}, { 751,11}, {1775,11}, { 495,11}, {1519,11}, {1007,11}, {2031,11}, {  31,11}, 
    {1055,11}, { 543,11}, {1567,11}, { 287,11}, {1311,11}, { 799,11}, {1823,11}, { 159,11}, 
    {1183,11}, { 671,11}, {1695,11}, { 415,11}, {1439,11}, { 927,11}, {1951,11}, {  95,11}, 
    {1119,11}, { 607,11}, {1631,11}, { 351,11}, {1375,11}, { 863,11}, {1887,11}, { 223,11}, 
    {1247,11}, { 735,11}, {1759,11}, { 479,11}, {1503,11}, { 991,11}, {2015,11}, {  63,11}, 
    {1087,11}, { 575,11}, {1599,11}, { 319,11}, {1343,11}, { 831,11}, {1855,11}, { 191,11}, 
    {1215,11}, { 703,11}, {1727,11}, { 447,11}, {1471,11}, { 959,11}, {1983,11}, { 127,11}, 
    {1151,11}, { 639,11}, {1663,11}, { 383,11}, {1407,11}, { 895,11}, {1919,11}, { 255,11}, 
    {1279,11}, { 767,11}, {1791,11}, { 511,11}, {1535,11}, {1023,11}, {2047,11}, { 791,10}, 
    { 151,10}, { 663,10}, { 407,10}, { 919,10}, {  87,10}, { 599,10}, {  39, 9}, { 295, 9}, 
    { 167, 9}, { 423, 9}, { 103, 9}, { 359, 9}, {  59, 8}, { 187, 8}, { 123, 8}, { 251, 8}, 
    {   7, 8}, {  51, 7}, { 115, 7}, {  11, 7}, {  75, 7}, {  43, 7}, {  45, 6}, {  29, 6}, 
    {  61, 6}, {   3, 6}, {   1, 5}, {  17, 5}, {   9, 5}, {  25, 5}, {  12, 4}, {   2, 4}, 
    {  10, 4}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
  },
  #endif
  #ifdef HUFFMAN_DECODE
  {
    { 4,  8}, { 5,242}, { 4,247}, { 6,241}, { 4, 10}, { 6, 15}, { 5, 11}, { 8,232}, 
    { 4,  9}, { 5,244}, { 4,248}, { 7,235}, { 4,246}, { 6, 19}, { 5, 13}, {11, 87}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7, 22}, { 4, 10}, { 6, 17}, { 5, 12}, {10, 40}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8, 26}, { 4,246}, { 6,239}, { 5, 14}, {11,151}, 
    { 4,  8}, { 5,242}, { 4,247}, { 7, 20}, { 4, 10}, { 6, 16}, { 5, 11}, { 9,222}, 
    { 4,  9}, { 5,244}, { 4,248}, { 7,237}, { 4,246}, { 6,238}, { 5, 13}, {11,119}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7,233}, { 4, 10}, { 6, 18}, { 5, 12}, {11, 55}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8,228}, { 4,246}, { 6,240}, { 5, 14}, {11,183}, 
    { 4,  8}, { 5,242}, { 4,247}, { 6,241}, { 4, 10}, { 6, 15}, { 5, 11}, { 9, 32}, 
    { 4,  9}, { 5,244}, { 4,248}, { 7,236}, { 4,246}, { 6, 19}, { 5, 13}, {11,103}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7, 23}, { 4, 10}, { 6, 17}, { 5, 12}, {10,220}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8, 28}, { 4,246}, { 6,239}, { 5, 14}, {11,167}, 
    { 4,  8}, { 5,242}, { 4,247}, { 7, 21}, { 4, 10}, { 6, 16}, { 5, 11}, { 9,226}, 
    { 4,  9}, { 5,244}, { 4,248}, { 8, 24}, { 4,246}, { 6,238}, { 5, 13}, {11,135}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7,234}, { 4, 10}, { 6, 18}, { 5, 12}, {11, 71}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8,230}, { 4,246}, { 6,240}, { 5, 14}, {11,199}, 
    { 4,  8}, { 5,242}, { 4,247}, { 6,241}, { 4, 10}, { 6, 15}, { 5, 11}, { 9, 30}, 
    { 4,  9}, { 5,244}, { 4,248}, { 7,235}, { 4,246}, { 6, 19}, { 5, 13}, {11, 95}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7, 22}, { 4, 10}, { 6, 17}, { 5, 12}, {10,216}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8, 27}, { 4,246}, { 6,239}, { 5, 14}, {11,159}, 
    { 4,  8}, { 5,242}, { 4,247}, { 7, 20}, { 4, 10}, { 6, 16}, { 5, 11}, { 9,224}, 
    { 4,  9}, { 5,244}, { 4,248}, { 7,237}, { 4,246}, { 6,238}, { 5, 13}, {11,127}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7,233}, { 4, 10}, { 6, 18}, { 5, 12}, {11, 63}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8,229}, { 4,246}, { 6,240}, { 5, 14}, {11,191}, 
    { 4,  8}, { 5,242}, { 4,247}, { 6,241}, { 4, 10}, { 6, 15}, { 5, 11}, { 9, 34}, 
    { 4,  9}, { 5,244}, { 4,248}, { 7,236}, { 4,246}, { 6, 19}, { 5, 13}, {11,111}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7, 23}, { 4, 10}, { 6, 17}, { 5, 12}, {11, 47}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8, 29}, { 4,246}, { 6,239}, { 5, 14}, {11,175}, 
    { 4,  8}, { 5,242}, { 4,247}, { 7, 21}, { 4, 10}, { 6, 16}, { 5, 11}, {10, 36}, 
    { 4,  9}, { 5,244}, { 4,248}, { 8, 25}, { 4,246}, { 6,238}, { 5, 13}, {11,143}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7,234}, { 4, 10}, { 6, 18}, { 5, 12}, {11, 79}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8,231}, { 4,246}, { 6,240}, { 5, 14}, {11,207}, 
    { 4,  8}, { 5,242}, { 4,247}, { 6,241}, { 4, 10}, { 6, 15}, { 5, 11}, { 8,232}, 
    { 4,  9}, { 5,244}, { 4,248}, { 7,235}, { 4,246}, { 6, 19}, { 5, 13}, {11, 91}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7, 22}, { 4, 10}, { 6, 17}, { 5, 12}, {10, 42}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8, 26}, { 4,246}, { 6,239}, { 5, 14}, {11,155}, 
    { 4,  8}, { 5,242}, { 4,247}, { 7, 20}, { 4, 10}, { 6, 16}, { 5, 11}, { 9,223}, 
    { 4,  9}, { 5,244}, { 4,248}, { 7,237}, { 4,246}, { 6,238}, { 5, 13}, {11,123}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7,233}, { 4, 10}, { 6, 18}, { 5, 12}, {11, 59}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8,228}, { 4,246}, { 6,240}, { 5, 14}, {11,187}, 
    { 4,  8}, { 5,242}, { 4,247}, { 6,241}, { 4, 10}, { 6, 15}, { 5, 11}, { 9, 33}, 
    { 4,  9}, { 5,244}, { 4,248}, { 7,236}, { 4,246}, { 6, 19}, { 5, 13}, {11,107}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7, 23}, { 4, 10}, { 6, 17}, { 5, 12}, {11, 43}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8, 28}, { 4,246}, { 6,239}, { 5, 14}, {11,171}, 
    { 4,  8}, { 5,242}, { 4,247}, { 7, 21}, { 4, 10}, { 6, 16}, { 5, 11}, { 9,227}, 
    { 4,  9}, { 5,244}, { 4,248}, { 8, 24}, { 4,246}, { 6,238}, { 5, 13}, {11,139}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7,234}, { 4, 10}, { 6, 18}, { 5, 12}, {11, 75}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8,230}, { 4,246}, { 6,240}, { 5, 14}, {11,203}, 
    { 4,  8}, { 5,242}, { 4,247}, { 6,241}, { 4, 10}, { 6, 15}, { 5, 11}, { 9, 31}, 
    { 4,  9}, { 5,244}, { 4,248}, { 7,235}, { 4,246}, { 6, 19}, { 5, 13}, {11, 99}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7, 22}, { 4, 10}, { 6, 17}, { 5, 12}, {10,218}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8, 27}, { 4,246}, { 6,239}, { 5, 14}, {11,163}, 
    { 4,  8}, { 5,242}, { 4,247}, { 7, 20}, { 4, 10}, { 6, 16}, { 5, 11}, { 9,225}, 
    { 4,  9}, { 5,244}, { 4,248}, { 7,237}, { 4,246}, { 6,238}, { 5, 13}, {11,131}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7,233}, { 4, 10}, { 6, 18}, { 5, 12}, {11, 67}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8,229}, { 4,246}, { 6,240}, { 5, 14}, {11,195}, 
    { 4,  8}, { 5,242}, { 4,247}, { 6,241}, { 4, 10}, { 6, 15}, { 5, 11}, { 9, 35}, 
    { 4,  9}, { 5,244}, { 4,248}, { 7,236}, { 4,246}, { 6, 19}, { 5, 13}, {11,115}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7, 23}, { 4, 10}, { 6, 17}, { 5, 12}, {11, 51}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8, 29}, { 4,246}, { 6,239}, { 5, 14}, {11,179}, 
    { 4,  8}, { 5,242}, { 4,247}, { 7, 21}, { 4, 10}, { 6, 16}, { 5, 11}, {10, 38}, 
    { 4,  9}, { 5,244}, { 4,248}, { 8, 25}, { 4,246}, { 6,238}, { 5, 13}, {11,147}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7,234}, { 4, 10}, { 6, 18}, { 5, 12}, {11, 83}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8,231}, { 4,246}, { 6,240}, { 5, 14}, {11,211}, 
    { 4,  8}, { 5,242}, { 4,247}, { 6,241}, { 4, 10}, { 6, 15}, { 5, 11}, { 8,232}, 
    { 4,  9}, { 5,244}, { 4,248}, { 7,235}, { 4,246}, { 6, 19}, { 5, 13}, {11, 89}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7, 22}, { 4, 10}, { 6, 17}, { 5, 12}, {10, 41}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8, 26}, { 4,246}, { 6,239}, { 5, 14}, {11,153}, 
    { 4,  8}, { 5,242}, { 4,247}, { 7, 20}, { 4, 10}, { 6, 16}, { 5, 11}, { 9,222}, 
    { 4,  9}, { 5,244}, { 4,248}, { 7,237}, { 4,246}, { 6,238}, { 5, 13}, {11,121}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7,233}, { 4, 10}, { 6, 18}, { 5, 12}, {11, 57}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8,228}, { 4,246}, { 6,240}, { 5, 14}, {11,185}, 
    { 4,  8}, { 5,242}, { 4,247}, { 6,241}, { 4, 10}, { 6, 15}, { 5, 11}, { 9, 32}, 
    { 4,  9}, { 5,244}, { 4,248}, { 7,236}, { 4,246}, { 6, 19}, { 5, 13}, {11,105}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7, 23}, { 4, 10}, { 6, 17}, { 5, 12}, {10,221}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8, 28}, { 4,246}, { 6,239}, { 5, 14}, {11,169}, 
    { 4,  8}, { 5,242}, { 4,247}, { 7, 21}, { 4, 10}, { 6, 16}, { 5, 11}, { 9,226}, 
    { 4,  9}, { 5,244}, { 4,248}, { 8, 24}, { 4,246}, { 6,238}, { 5, 13}, {11,137}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7,234}, { 4, 10}, { 6, 18}, { 5, 12}, {11, 73}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8,230}, { 4,246}, { 6,240}, { 5, 14}, {11,201}, 
    { 4,  8}, { 5,242}, { 4,247}, { 6,241}, { 4, 10}, { 6, 15}, { 5, 11}, { 9, 30}, 
    { 4,  9}, { 5,244}, { 4,248}, { 7,235}, { 4,246}, { 6, 19}, { 5, 13}, {11, 97}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7, 22}, { 4, 10}, { 6, 17}, { 5, 12}, {10,217}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8, 27}, { 4,246}, { 6,239}, { 5, 14}, {11,161}, 
    { 4,  8}, { 5,242}, { 4,247}, { 7, 20}, { 4, 10}, { 6, 16}, { 5, 11}, { 9,224}, 
    { 4,  9}, { 5,244}, { 4,248}, { 7,237}, { 4,246}, { 6,238}, { 5, 13}, {11,129}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7,233}, { 4, 10}, { 6, 18}, { 5, 12}, {11, 65}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8,229}, { 4,246}, { 6,240}, { 5, 14}, {11,193}, 
    { 4,  8}, { 5,242}, { 4,247}, { 6,241}, { 4, 10}, { 6, 15}, { 5, 11}, { 9, 34}, 
    { 4,  9}, { 5,244}, { 4,248}, { 7,236}, { 4,246}, { 6, 19}, { 5, 13}, {11,113}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7, 23}, { 4, 10}, { 6, 17}, { 5, 12}, {11, 49}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8, 29}, { 4,246}, { 6,239}, { 5, 14}, {11,177}, 
    { 4,  8}, { 5,242}, { 4,247}, { 7, 21}, { 4, 10}, { 6, 16}, { 5, 11}, {10, 37}, 
    { 4,  9}, { 5,244}, { 4,248}, { 8, 25}, { 4,246}, { 6,238}, { 5, 13}, {11,145}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7,234}, { 4, 10}, { 6, 18}, { 5, 12}, {11, 81}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8,231}, { 4,246}, { 6,240}, { 5, 14}, {11,209}, 
    { 4,  8}, { 5,242}, { 4,247}, { 6,241}, { 4, 10}, { 6, 15}, { 5, 11}, { 8,232}, 
    { 4,  9}, { 5,244}, { 4,248}, { 7,235}, { 4,246}, { 6, 19}, { 5, 13}, {11, 93}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7, 22}, { 4, 10}, { 6, 17}, { 5, 12}, {10,215}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8, 26}, { 4,246}, { 6,239}, { 5, 14}, {11,157}, 
    { 4,  8}, { 5,242}, { 4,247}, { 7, 20}, { 4, 10}, { 6, 16}, { 5, 11}, { 9,223}, 
    { 4,  9}, { 5,244}, { 4,248}, { 7,237}, { 4,246}, { 6,238}, { 5, 13}, {11,125}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7,233}, { 4, 10}, { 6, 18}, { 5, 12}, {11, 61}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8,228}, { 4,246}, { 6,240}, { 5, 14}, {11,189}, 
    { 4,  8}, { 5,242}, { 4,247}, { 6,241}, { 4, 10}, { 6, 15}, { 5, 11}, { 9, 33}, 
    { 4,  9}, { 5,244}, { 4,248}, { 7,236}, { 4,246}, { 6, 19}, { 5, 13}, {11,109}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7, 23}, { 4, 10}, { 6, 17}, { 5, 12}, {11, 45}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8, 28}, { 4,246}, { 6,239}, { 5, 14}, {11,173}, 
    { 4,  8}, { 5,242}, { 4,247}, { 7, 21}, { 4, 10}, { 6, 16}, { 5, 11}, { 9,227}, 
    { 4,  9}, { 5,244}, { 4,248}, { 8, 24}, { 4,246}, { 6,238}, { 5, 13}, {11,141}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7,234}, { 4, 10}, { 6, 18}, { 5, 12}, {11, 77}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8,230}, { 4,246}, { 6,240}, { 5, 14}, {11,205}, 
    { 4,  8}, { 5,242}, { 4,247}, { 6,241}, { 4, 10}, { 6, 15}, { 5, 11}, { 9, 31}, 
    { 4,  9}, { 5,244}, { 4,248}, { 7,235}, { 4,246}, { 6, 19}, { 5, 13}, {11,101}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7, 22}, { 4, 10}, { 6, 17}, { 5, 12}, {10,219}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8, 27}, { 4,246}, { 6,239}, { 5, 14}, {11,165}, 
    { 4,  8}, { 5,242}, { 4,247}, { 7, 20}, { 4, 10}, { 6, 16}, { 5, 11}, { 9,225}, 
    { 4,  9}, { 5,244}, { 4,248}, { 7,237}, { 4,246}, { 6,238}, { 5, 13}, {11,133}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7,233}, { 4, 10}, { 6, 18}, { 5, 12}, {11, 69}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8,229}, { 4,246}, { 6,240}, { 5, 14}, {11,197}, 
    { 4,  8}, { 5,242}, { 4,247}, { 6,241}, { 4, 10}, { 6, 15}, { 5, 11}, { 9, 35}, 
    { 4,  9}, { 5,244}, { 4,248}, { 7,236}, { 4,246}, { 6, 19}, { 5, 13}, {11,117}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7, 23}, { 4, 10}, { 6, 17}, { 5, 12}, {11, 53}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8, 29}, { 4,246}, { 6,239}, { 5, 14}, {11,181}, 
    { 4,  8}, { 5,242}, { 4,247}, { 7, 21}, { 4, 10}, { 6, 16}, { 5, 11}, {10, 39}, 
    { 4,  9}, { 5,244}, { 4,248}, { 8, 25}, { 4,246}, { 6,238}, { 5, 13}, {11,149}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7,234}, { 4, 10}, { 6, 18}, { 5, 12}, {11, 85}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8,231}, { 4,246}, { 6,240}, { 5, 14}, {11,213}, 
    { 4,  8}, { 5,242}, { 4,247}, { 6,241}, { 4, 10}, { 6, 15}, { 5, 11}, { 8,232}, 
    { 4,  9}, { 5,244}, { 4,248}, { 7,235}, { 4,246}, { 6, 19}, { 5, 13}, {11, 88}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7, 22}, { 4, 10}, { 6, 17}, { 5, 12}, {10, 40}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8, 26}, { 4,246}, { 6,239}, { 5, 14}, {11,152}, 
    { 4,  8}, { 5,242}, { 4,247}, { 7, 20}, { 4, 10}, { 6, 16}, { 5, 11}, { 9,222}, 
    { 4,  9}, { 5,244}, { 4,248}, { 7,237}, { 4,246}, { 6,238}, { 5, 13}, {11,120}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7,233}, { 4, 10}, { 6, 18}, { 5, 12}, {11, 56}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8,228}, { 4,246}, { 6,240}, { 5, 14}, {11,184}, 
    { 4,  8}, { 5,242}, { 4,247}, { 6,241}, { 4, 10}, { 6, 15}, { 5, 11}, { 9, 32}, 
    { 4,  9}, { 5,244}, { 4,248}, { 7,236}, { 4,246}, { 6, 19}, { 5, 13}, {11,104}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7, 23}, { 4, 10}, { 6, 17}, { 5, 12}, {10,220}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8, 28}, { 4,246}, { 6,239}, { 5, 14}, {11,168}, 
    { 4,  8}, { 5,242}, { 4,247}, { 7, 21}, { 4, 10}, { 6, 16}, { 5, 11}, { 9,226}, 
    { 4,  9}, { 5,244}, { 4,248}, { 8, 24}, { 4,246}, { 6,238}, { 5, 13}, {11,136}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7,234}, { 4, 10}, { 6, 18}, { 5, 12}, {11, 72}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8,230}, { 4,246}, { 6,240}, { 5, 14}, {11,200}, 
    { 4,  8}, { 5,242}, { 4,247}, { 6,241}, { 4, 10}, { 6, 15}, { 5, 11}, { 9, 30}, 
    { 4,  9}, { 5,244}, { 4,248}, { 7,235}, { 4,246}, { 6, 19}, { 5, 13}, {11, 96}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7, 22}, { 4, 10}, { 6, 17}, { 5, 12}, {10,216}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8, 27}, { 4,246}, { 6,239}, { 5, 14}, {11,160}, 
    { 4,  8}, { 5,242}, { 4,247}, { 7, 20}, { 4, 10}, { 6, 16}, { 5, 11}, { 9,224}, 
    { 4,  9}, { 5,244}, { 4,248}, { 7,237}, { 4,246}, { 6,238}, { 5, 13}, {11,128}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7,233}, { 4, 10}, { 6, 18}, { 5, 12}, {11, 64}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8,229}, { 4,246}, { 6,240}, { 5, 14}, {11,192}, 
    { 4,  8}, { 5,242}, { 4,247}, { 6,241}, { 4, 10}, { 6, 15}, { 5, 11}, { 9, 34}, 
    { 4,  9}, { 5,244}, { 4,248}, { 7,236}, { 4,246}, { 6, 19}, { 5, 13}, {11,112}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7, 23}, { 4, 10}, { 6, 17}, { 5, 12}, {11, 48}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8, 29}, { 4,246}, { 6,239}, { 5, 14}, {11,176}, 
    { 4,  8}, { 5,242}, { 4,247}, { 7, 21}, { 4, 10}, { 6, 16}, { 5, 11}, {10, 36}, 
    { 4,  9}, { 5,244}, { 4,248}, { 8, 25}, { 4,246}, { 6,238}, { 5, 13}, {11,144}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7,234}, { 4, 10}, { 6, 18}, { 5, 12}, {11, 80}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8,231}, { 4,246}, { 6,240}, { 5, 14}, {11,208}, 
    { 4,  8}, { 5,242}, { 4,247}, { 6,241}, { 4, 10}, { 6, 15}, { 5, 11}, { 8,232}, 
    { 4,  9}, { 5,244}, { 4,248}, { 7,235}, { 4,246}, { 6, 19}, { 5, 13}, {11, 92}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7, 22}, { 4, 10}, { 6, 17}, { 5, 12}, {10, 42}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8, 26}, { 4,246}, { 6,239}, { 5, 14}, {11,156}, 
    { 4,  8}, { 5,242}, { 4,247}, { 7, 20}, { 4, 10}, { 6, 16}, { 5, 11}, { 9,223}, 
    { 4,  9}, { 5,244}, { 4,248}, { 7,237}, { 4,246}, { 6,238}, { 5, 13}, {11,124}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7,233}, { 4, 10}, { 6, 18}, { 5, 12}, {11, 60}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8,228}, { 4,246}, { 6,240}, { 5, 14}, {11,188}, 
    { 4,  8}, { 5,242}, { 4,247}, { 6,241}, { 4, 10}, { 6, 15}, { 5, 11}, { 9, 33}, 
    { 4,  9}, { 5,244}, { 4,248}, { 7,236}, { 4,246}, { 6, 19}, { 5, 13}, {11,108}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7, 23}, { 4, 10}, { 6, 17}, { 5, 12}, {11, 44}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8, 28}, { 4,246}, { 6,239}, { 5, 14}, {11,172}, 
    { 4,  8}, { 5,242}, { 4,247}, { 7, 21}, { 4, 10}, { 6, 16}, { 5, 11}, { 9,227}, 
    { 4,  9}, { 5,244}, { 4,248}, { 8, 24}, { 4,246}, { 6,238}, { 5, 13}, {11,140}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7,234}, { 4, 10}, { 6, 18}, { 5, 12}, {11, 76}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8,230}, { 4,246}, { 6,240}, { 5, 14}, {11,204}, 
    { 4,  8}, { 5,242}, { 4,247}, { 6,241}, { 4, 10}, { 6, 15}, { 5, 11}, { 9, 31}, 
    { 4,  9}, { 5,244}, { 4,248}, { 7,235}, { 4,246}, { 6, 19}, { 5, 13}, {11,100}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7, 22}, { 4, 10}, { 6, 17}, { 5, 12}, {10,218}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8, 27}, { 4,246}, { 6,239}, { 5, 14}, {11,164}, 
    { 4,  8}, { 5,242}, { 4,247}, { 7, 20}, { 4, 10}, { 6, 16}, { 5, 11}, { 9,225}, 
    { 4,  9}, { 5,244}, { 4,248}, { 7,237}, { 4,246}, { 6,238}, { 5, 13}, {11,132}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7,233}, { 4, 10}, { 6, 18}, { 5, 12}, {11, 68}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8,229}, { 4,246}, { 6,240}, { 5, 14}, {11,196}, 
    { 4,  8}, { 5,242}, { 4,247}, { 6,241}, { 4, 10}, { 6, 15}, { 5, 11}, { 9, 35}, 
    { 4,  9}, { 5,244}, { 4,248}, { 7,236}, { 4,246}, { 6, 19}, { 5, 13}, {11,116}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7, 23}, { 4, 10}, { 6, 17}, { 5, 12}, {11, 52}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8, 29}, { 4,246}, { 6,239}, { 5, 14}, {11,180}, 
    { 4,  8}, { 5,242}, { 4,247}, { 7, 21}, { 4, 10}, { 6, 16}, { 5, 11}, {10, 38}, 
    { 4,  9}, { 5,244}, { 4,248}, { 8, 25}, { 4,246}, { 6,238}, { 5, 13}, {11,148}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7,234}, { 4, 10}, { 6, 18}, { 5, 12}, {11, 84}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8,231}, { 4,246}, { 6,240}, { 5, 14}, {11,212}, 
    { 4,  8}, { 5,242}, { 4,247}, { 6,241}, { 4, 10}, { 6, 15}, { 5, 11}, { 8,232}, 
    { 4,  9}, { 5,244}, { 4,248}, { 7,235}, { 4,246}, { 6, 19}, { 5, 13}, {11, 90}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7, 22}, { 4, 10}, { 6, 17}, { 5, 12}, {10, 41}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8, 26}, { 4,246}, { 6,239}, { 5, 14}, {11,154}, 
    { 4,  8}, { 5,242}, { 4,247}, { 7, 20}, { 4, 10}, { 6, 16}, { 5, 11}, { 9,222}, 
    { 4,  9}, { 5,244}, { 4,248}, { 7,237}, { 4,246}, { 6,238}, { 5, 13}, {11,122}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7,233}, { 4, 10}, { 6, 18}, { 5, 12}, {11, 58}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8,228}, { 4,246}, { 6,240}, { 5, 14}, {11,186}, 
    { 4,  8}, { 5,242}, { 4,247}, { 6,241}, { 4, 10}, { 6, 15}, { 5, 11}, { 9, 32}, 
    { 4,  9}, { 5,244}, { 4,248}, { 7,236}, { 4,246}, { 6, 19}, { 5, 13}, {11,106}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7, 23}, { 4, 10}, { 6, 17}, { 5, 12}, {10,221}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8, 28}, { 4,246}, { 6,239}, { 5, 14}, {11,170}, 
    { 4,  8}, { 5,242}, { 4,247}, { 7, 21}, { 4, 10}, { 6, 16}, { 5, 11}, { 9,226}, 
    { 4,  9}, { 5,244}, { 4,248}, { 8, 24}, { 4,246}, { 6,238}, { 5, 13}, {11,138}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7,234}, { 4, 10}, { 6, 18}, { 5, 12}, {11, 74}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8,230}, { 4,246}, { 6,240}, { 5, 14}, {11,202}, 
    { 4,  8}, { 5,242}, { 4,247}, { 6,241}, { 4, 10}, { 6, 15}, { 5, 11}, { 9, 30}, 
    { 4,  9}, { 5,244}, { 4,248}, { 7,235}, { 4,246}, { 6, 19}, { 5, 13}, {11, 98}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7, 22}, { 4, 10}, { 6, 17}, { 5, 12}, {10,217}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8, 27}, { 4,246}, { 6,239}, { 5, 14}, {11,162}, 
    { 4,  8}, { 5,242}, { 4,247}, { 7, 20}, { 4, 10}, { 6, 16}, { 5, 11}, { 9,224}, 
    { 4,  9}, { 5,244}, { 4,248}, { 7,237}, { 4,246}, { 6,238}, { 5, 13}, {11,130}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7,233}, { 4, 10}, { 6, 18}, { 5, 12}, {11, 66}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8,229}, { 4,246}, { 6,240}, { 5, 14}, {11,194}, 
    { 4,  8}, { 5,242}, { 4,247}, { 6,241}, { 4, 10}, { 6, 15}, { 5, 11}, { 9, 34}, 
    { 4,  9}, { 5,244}, { 4,248}, { 7,236}, { 4,246}, { 6, 19}, { 5, 13}, {11,114}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7, 23}, { 4, 10}, { 6, 17}, { 5, 12}, {11, 50}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8, 29}, { 4,246}, { 6,239}, { 5, 14}, {11,178}, 
    { 4,  8}, { 5,242}, { 4,247}, { 7, 21}, { 4, 10}, { 6, 16}, { 5, 11}, {10, 37}, 
    { 4,  9}, { 5,244}, { 4,248}, { 8, 25}, { 4,246}, { 6,238}, { 5, 13}, {11,146}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7,234}, { 4, 10}, { 6, 18}, { 5, 12}, {11, 82}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8,231}, { 4,246}, { 6,240}, { 5, 14}, {11,210}, 
    { 4,  8}, { 5,242}, { 4,247}, { 6,241}, { 4, 10}, { 6, 15}, { 5, 11}, { 8,232}, 
    { 4,  9}, { 5,244}, { 4,248}, { 7,235}, { 4,246}, { 6, 19}, { 5, 13}, {11, 94}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7, 22}, { 4, 10}, { 6, 17}, { 5, 12}, {10,215}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8, 26}, { 4,246}, { 6,239}, { 5, 14}, {11,158}, 
    { 4,  8}, { 5,242}, { 4,247}, { 7, 20}, { 4, 10}, { 6, 16}, { 5, 11}, { 9,223}, 
    { 4,  9}, { 5,244}, { 4,248}, { 7,237}, { 4,246}, { 6,238}, { 5, 13}, {11,126}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7,233}, { 4, 10}, { 6, 18}, { 5, 12}, {11, 62}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8,228}, { 4,246}, { 6,240}, { 5, 14}, {11,190}, 
    { 4,  8}, { 5,242}, { 4,247}, { 6,241}, { 4, 10}, { 6, 15}, { 5, 11}, { 9, 33}, 
    { 4,  9}, { 5,244}, { 4,248}, { 7,236}, { 4,246}, { 6, 19}, { 5, 13}, {11,110}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7, 23}, { 4, 10}, { 6, 17}, { 5, 12}, {11, 46}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8, 28}, { 4,246}, { 6,239}, { 5, 14}, {11,174}, 
    { 4,  8}, { 5,242}, { 4,247}, { 7, 21}, { 4, 10}, { 6, 16}, { 5, 11}, { 9,227}, 
    { 4,  9}, { 5,244}, { 4,248}, { 8, 24}, { 4,246}, { 6,238}, { 5, 13}, {11,142}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7,234}, { 4, 10}, { 6, 18}, { 5, 12}, {11, 78}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8,230}, { 4,246}, { 6,240}, { 5, 14}, {11,206}, 
    { 4,  8}, { 5,242}, { 4,247}, { 6,241}, { 4, 10}, { 6, 15}, { 5, 11}, { 9, 31}, 
    { 4,  9}, { 5,244}, { 4,248}, { 7,235}, { 4,246}, { 6, 19}, { 5, 13}, {11,102}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7, 22}, { 4, 10}, { 6, 17}, { 5, 12}, {10,219}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8, 27}, { 4,246}, { 6,239}, { 5, 14}, {11,166}, 
    { 4,  8}, { 5,242}, { 4,247}, { 7, 20}, { 4, 10}, { 6, 16}, { 5, 11}, { 9,225}, 
    { 4,  9}, { 5,244}, { 4,248}, { 7,237}, { 4,246}, { 6,238}, { 5, 13}, {11,134}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7,233}, { 4, 10}, { 6, 18}, { 5, 12}, {11, 70}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8,229}, { 4,246}, { 6,240}, { 5, 14}, {11,198}, 
    { 4,  8}, { 5,242}, { 4,247}, { 6,241}, { 4, 10}, { 6, 15}, { 5, 11}, { 9, 35}, 
    { 4,  9}, { 5,244}, { 4,248}, { 7,236}, { 4,246}, { 6, 19}, { 5, 13}, {11,118}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7, 23}, { 4, 10}, { 6, 17}, { 5, 12}, {11, 54}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8, 29}, { 4,246}, { 6,239}, { 5, 14}, {11,182}, 
    { 4,  8}, { 5,242}, { 4,247}, { 7, 21}, { 4, 10}, { 6, 16}, { 5, 11}, {10, 39}, 
    { 4,  9}, { 5,244}, { 4,248}, { 8, 25}, { 4,246}, { 6,238}, { 5, 13}, {11,150}, 
    { 4,  8}, { 5,243}, { 4,247}, { 7,234}, { 4, 10}, { 6, 18}, { 5, 12}, {11, 86}, 
    { 4,  9}, { 5,245}, { 4,248}, { 8,231}, { 4,246}, { 6,240}, { 5, 14}, {11,214}, 
  }
  #endif
};

static radaudio_huffman rada_nonzero_coefficient_big_huff3 =
{
  #ifdef HUFFMAN_ENCODE
  {
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 4}, {   4, 5}, {  20, 5}, {  12, 5}, {  28, 5}, {   2, 5}, {  18, 5}, {  30, 6}, 
    {  62, 6}, {   1, 6}, {  33, 6}, {  17, 6}, {  49, 6}, {  53, 7}, { 117, 7}, {  13, 7}, 
    {  77, 7}, {  45, 7}, { 109, 7}, {  29, 7}, {  93, 7}, {  51, 8}, { 179, 8}, { 115, 8}, 
    { 243, 8}, {  11, 8}, { 139, 8}, {  75, 8}, { 203, 8}, {  43, 8}, { 123, 9}, { 379, 9}, 
    { 251, 9}, { 507, 9}, {   7, 9}, { 263, 9}, { 135, 9}, { 391, 9}, {  71, 9}, { 327, 9}, 
    { 151,10}, { 663,10}, { 407,10}, { 919,10}, {  87,10}, { 599,10}, { 343,10}, { 855,10}, 
    { 215,10}, { 727,10}, { 471,10}, { 983,10}, {  55,10}, { 759,11}, {1783,11}, { 503,11}, 
    {1527,11}, {1015,11}, {2039,11}, {  15,11}, {1039,11}, { 527,11}, {1551,11}, { 271,11}, 
    {1295,11}, { 783,11}, {1807,11}, { 143,11}, {1167,11}, { 655,11}, {1679,11}, { 399,11}, 
    {1423,11}, { 911,11}, {1935,11}, {  79,11}, {1103,11}, { 591,11}, {1615,11}, { 335,11}, 
    {1359,11}, { 847,11}, {1871,11}, { 207,11}, {1231,11}, { 719,11}, {1743,11}, { 463,11}, 
    {1487,11}, { 975,11}, {1999,11}, {  47,11}, {1071,11}, { 559,11}, {1583,11}, { 303,11}, 
    {1327,11}, { 815,11}, {1839,11}, { 175,11}, {1199,11}, { 687,11}, {1711,11}, { 431,11}, 
    { 199, 9}, {1455,11}, { 943,11}, {1967,11}, { 111,11}, {1135,11}, { 623,11}, {1647,11}, 
    { 367,11}, {1391,11}, { 879,11}, {1903,11}, { 239,11}, {1263,11}, { 751,11}, {1775,11}, 
    { 495,11}, {1519,11}, {1007,11}, {2031,11}, {  31,11}, {1055,11}, { 543,11}, {1567,11}, 
    { 287,11}, {1311,11}, { 799,11}, {1823,11}, { 159,11}, {1183,11}, { 671,11}, {1695,11}, 
    { 455, 9}, { 415,11}, {1439,11}, { 927,11}, {1951,11}, {  95,11}, {1119,11}, { 607,11}, 
    {1631,11}, { 351,11}, {1375,11}, { 863,11}, {1887,11}, { 223,11}, {1247,11}, { 735,11}, 
    {1759,11}, { 479,11}, {1503,11}, { 991,11}, {2015,11}, {  63,11}, {1087,11}, { 575,11}, 
    {1599,11}, { 319,11}, {1343,11}, { 831,11}, {1855,11}, { 191,11}, {1215,11}, { 703,11}, 
    {1727,11}, { 447,11}, {1471,11}, { 959,11}, {1983,11}, { 127,11}, {1151,11}, { 639,11}, 
    {1663,11}, { 383,11}, {1407,11}, { 895,11}, {1919,11}, { 255,11}, {1279,11}, { 767,11}, 
    {1791,11}, { 511,11}, {1535,11}, {1023,11}, {2047,11}, { 567,10}, { 311,10}, { 823,10}, 
    { 183,10}, { 695,10}, { 439,10}, { 951,10}, { 119,10}, { 631,10}, { 375,10}, { 887,10}, 
    { 247,10}, {  39, 9}, { 295, 9}, { 167, 9}, { 423, 9}, { 103, 9}, { 359, 9}, { 231, 9}, 
    { 487, 9}, {  23, 9}, { 279, 9}, { 171, 8}, { 107, 8}, { 235, 8}, {  27, 8}, { 155, 8}, 
    {  91, 8}, { 219, 8}, {  59, 8}, { 187, 8}, {  61, 7}, { 125, 7}, {   3, 7}, {  67, 7}, 
    {  35, 7}, {  99, 7}, {  19, 7}, {  83, 7}, {   9, 6}, {  41, 6}, {  25, 6}, {  57, 6}, 
    {   5, 6}, {  37, 6}, {  21, 6}, {  10, 5}, {  26, 5}, {   6, 5}, {  22, 5}, {  14, 5}, 
    {   8, 4}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
  },
  #endif
  #ifdef HUFFMAN_DECODE
  {
    { 4,  8}, { 6, 17}, { 5, 13}, { 7,230}, { 5,  9}, { 6,240}, { 5,245}, { 9, 42}, 
    { 4,248}, { 6,236}, { 5,243}, { 8, 33}, { 5, 11}, { 7, 23}, { 5,247}, {11, 67}, 
    { 4,  8}, { 6, 19}, { 5, 14}, { 7,234}, { 5, 10}, { 6,242}, { 5,246}, { 9,217}, 
    { 4,248}, { 6,238}, { 5,244}, { 8,222}, { 5, 12}, { 7, 27}, { 6, 15}, {11,132}, 
    { 4,  8}, { 6, 18}, { 5, 13}, { 7,232}, { 5,  9}, { 6,241}, { 5,245}, { 9,209}, 
    { 4,248}, { 6,237}, { 5,243}, { 8, 37}, { 5, 11}, { 7, 25}, { 5,247}, {11, 99}, 
    { 4,  8}, { 6, 20}, { 5, 14}, { 8, 29}, { 5, 10}, { 7, 21}, { 5,246}, {10, 60}, 
    { 4,248}, { 6,239}, { 5,244}, { 8,226}, { 5, 12}, { 7,228}, { 6, 16}, {11,165}, 
    { 4,  8}, { 6, 17}, { 5, 13}, { 7,231}, { 5,  9}, { 6,240}, { 5,245}, { 9, 46}, 
    { 4,248}, { 6,236}, { 5,243}, { 8, 35}, { 5, 11}, { 7, 24}, { 5,247}, {11, 83}, 
    { 4,  8}, { 6, 19}, { 5, 14}, { 7,235}, { 5, 10}, { 6,242}, { 5,246}, {10, 52}, 
    { 4,248}, { 6,238}, { 5,244}, { 8,224}, { 5, 12}, { 7, 28}, { 6, 15}, {11,149}, 
    { 4,  8}, { 6, 18}, { 5, 13}, { 7,233}, { 5,  9}, { 6,241}, { 5,245}, { 9,213}, 
    { 4,248}, { 6,237}, { 5,243}, { 8,220}, { 5, 11}, { 7, 26}, { 5,247}, {11,116}, 
    { 4,  8}, { 6, 20}, { 5, 14}, { 8, 31}, { 5, 10}, { 7, 22}, { 5,246}, {10,204}, 
    { 4,248}, { 6,239}, { 5,244}, { 9, 38}, { 5, 12}, { 7,229}, { 6, 16}, {11,181}, 
    { 4,  8}, { 6, 17}, { 5, 13}, { 7,230}, { 5,  9}, { 6,240}, { 5,245}, { 9, 44}, 
    { 4,248}, { 6,236}, { 5,243}, { 8, 34}, { 5, 11}, { 7, 23}, { 5,247}, {11, 75}, 
    { 4,  8}, { 6, 19}, { 5, 14}, { 7,234}, { 5, 10}, { 6,242}, { 5,246}, {10, 48}, 
    { 4,248}, { 6,238}, { 5,244}, { 8,223}, { 5, 12}, { 7, 27}, { 6, 15}, {11,140}, 
    { 4,  8}, { 6, 18}, { 5, 13}, { 7,232}, { 5,  9}, { 6,241}, { 5,245}, { 9,211}, 
    { 4,248}, { 6,237}, { 5,243}, { 8,219}, { 5, 11}, { 7, 25}, { 5,247}, {11,107}, 
    { 4,  8}, { 6, 20}, { 5, 14}, { 8, 30}, { 5, 10}, { 7, 21}, { 5,246}, {10,200}, 
    { 4,248}, { 6,239}, { 5,244}, { 8,227}, { 5, 12}, { 7,228}, { 6, 16}, {11,173}, 
    { 4,  8}, { 6, 17}, { 5, 13}, { 7,231}, { 5,  9}, { 6,240}, { 5,245}, { 9,112}, 
    { 4,248}, { 6,236}, { 5,243}, { 8, 36}, { 5, 11}, { 7, 24}, { 5,247}, {11, 91}, 
    { 4,  8}, { 6, 19}, { 5, 14}, { 7,235}, { 5, 10}, { 6,242}, { 5,246}, {10, 56}, 
    { 4,248}, { 6,238}, { 5,244}, { 8,225}, { 5, 12}, { 7, 28}, { 6, 15}, {11,157}, 
    { 4,  8}, { 6, 18}, { 5, 13}, { 7,233}, { 5,  9}, { 6,241}, { 5,245}, { 9,215}, 
    { 4,248}, { 6,237}, { 5,243}, { 8,221}, { 5, 11}, { 7, 26}, { 5,247}, {11,124}, 
    { 4,  8}, { 6, 20}, { 5, 14}, { 8, 32}, { 5, 10}, { 7, 22}, { 5,246}, {10,208}, 
    { 4,248}, { 6,239}, { 5,244}, { 9, 40}, { 5, 12}, { 7,229}, { 6, 16}, {11,189}, 
    { 4,  8}, { 6, 17}, { 5, 13}, { 7,230}, { 5,  9}, { 6,240}, { 5,245}, { 9, 43}, 
    { 4,248}, { 6,236}, { 5,243}, { 8, 33}, { 5, 11}, { 7, 23}, { 5,247}, {11, 71}, 
    { 4,  8}, { 6, 19}, { 5, 14}, { 7,234}, { 5, 10}, { 6,242}, { 5,246}, { 9,218}, 
    { 4,248}, { 6,238}, { 5,244}, { 8,222}, { 5, 12}, { 7, 27}, { 6, 15}, {11,136}, 
    { 4,  8}, { 6, 18}, { 5, 13}, { 7,232}, { 5,  9}, { 6,241}, { 5,245}, { 9,210}, 
    { 4,248}, { 6,237}, { 5,243}, { 8, 37}, { 5, 11}, { 7, 25}, { 5,247}, {11,103}, 
    { 4,  8}, { 6, 20}, { 5, 14}, { 8, 29}, { 5, 10}, { 7, 21}, { 5,246}, {10,198}, 
    { 4,248}, { 6,239}, { 5,244}, { 8,226}, { 5, 12}, { 7,228}, { 6, 16}, {11,169}, 
    { 4,  8}, { 6, 17}, { 5, 13}, { 7,231}, { 5,  9}, { 6,240}, { 5,245}, { 9, 47}, 
    { 4,248}, { 6,236}, { 5,243}, { 8, 35}, { 5, 11}, { 7, 24}, { 5,247}, {11, 87}, 
    { 4,  8}, { 6, 19}, { 5, 14}, { 7,235}, { 5, 10}, { 6,242}, { 5,246}, {10, 54}, 
    { 4,248}, { 6,238}, { 5,244}, { 8,224}, { 5, 12}, { 7, 28}, { 6, 15}, {11,153}, 
    { 4,  8}, { 6, 18}, { 5, 13}, { 7,233}, { 5,  9}, { 6,241}, { 5,245}, { 9,214}, 
    { 4,248}, { 6,237}, { 5,243}, { 8,220}, { 5, 11}, { 7, 26}, { 5,247}, {11,120}, 
    { 4,  8}, { 6, 20}, { 5, 14}, { 8, 31}, { 5, 10}, { 7, 22}, { 5,246}, {10,206}, 
    { 4,248}, { 6,239}, { 5,244}, { 9, 39}, { 5, 12}, { 7,229}, { 6, 16}, {11,185}, 
    { 4,  8}, { 6, 17}, { 5, 13}, { 7,230}, { 5,  9}, { 6,240}, { 5,245}, { 9, 45}, 
    { 4,248}, { 6,236}, { 5,243}, { 8, 34}, { 5, 11}, { 7, 23}, { 5,247}, {11, 79}, 
    { 4,  8}, { 6, 19}, { 5, 14}, { 7,234}, { 5, 10}, { 6,242}, { 5,246}, {10, 50}, 
    { 4,248}, { 6,238}, { 5,244}, { 8,223}, { 5, 12}, { 7, 27}, { 6, 15}, {11,145}, 
    { 4,  8}, { 6, 18}, { 5, 13}, { 7,232}, { 5,  9}, { 6,241}, { 5,245}, { 9,212}, 
    { 4,248}, { 6,237}, { 5,243}, { 8,219}, { 5, 11}, { 7, 25}, { 5,247}, {11,111}, 
    { 4,  8}, { 6, 20}, { 5, 14}, { 8, 30}, { 5, 10}, { 7, 21}, { 5,246}, {10,202}, 
    { 4,248}, { 6,239}, { 5,244}, { 8,227}, { 5, 12}, { 7,228}, { 6, 16}, {11,177}, 
    { 4,  8}, { 6, 17}, { 5, 13}, { 7,231}, { 5,  9}, { 6,240}, { 5,245}, { 9,144}, 
    { 4,248}, { 6,236}, { 5,243}, { 8, 36}, { 5, 11}, { 7, 24}, { 5,247}, {11, 95}, 
    { 4,  8}, { 6, 19}, { 5, 14}, { 7,235}, { 5, 10}, { 6,242}, { 5,246}, {10, 58}, 
    { 4,248}, { 6,238}, { 5,244}, { 8,225}, { 5, 12}, { 7, 28}, { 6, 15}, {11,161}, 
    { 4,  8}, { 6, 18}, { 5, 13}, { 7,233}, { 5,  9}, { 6,241}, { 5,245}, { 9,216}, 
    { 4,248}, { 6,237}, { 5,243}, { 8,221}, { 5, 11}, { 7, 26}, { 5,247}, {11,128}, 
    { 4,  8}, { 6, 20}, { 5, 14}, { 8, 32}, { 5, 10}, { 7, 22}, { 5,246}, {11, 63}, 
    { 4,248}, { 6,239}, { 5,244}, { 9, 41}, { 5, 12}, { 7,229}, { 6, 16}, {11,193}, 
    { 4,  8}, { 6, 17}, { 5, 13}, { 7,230}, { 5,  9}, { 6,240}, { 5,245}, { 9, 42}, 
    { 4,248}, { 6,236}, { 5,243}, { 8, 33}, { 5, 11}, { 7, 23}, { 5,247}, {11, 69}, 
    { 4,  8}, { 6, 19}, { 5, 14}, { 7,234}, { 5, 10}, { 6,242}, { 5,246}, { 9,217}, 
    { 4,248}, { 6,238}, { 5,244}, { 8,222}, { 5, 12}, { 7, 27}, { 6, 15}, {11,134}, 
    { 4,  8}, { 6, 18}, { 5, 13}, { 7,232}, { 5,  9}, { 6,241}, { 5,245}, { 9,209}, 
    { 4,248}, { 6,237}, { 5,243}, { 8, 37}, { 5, 11}, { 7, 25}, { 5,247}, {11,101}, 
    { 4,  8}, { 6, 20}, { 5, 14}, { 8, 29}, { 5, 10}, { 7, 21}, { 5,246}, {10,197}, 
    { 4,248}, { 6,239}, { 5,244}, { 8,226}, { 5, 12}, { 7,228}, { 6, 16}, {11,167}, 
    { 4,  8}, { 6, 17}, { 5, 13}, { 7,231}, { 5,  9}, { 6,240}, { 5,245}, { 9, 46}, 
    { 4,248}, { 6,236}, { 5,243}, { 8, 35}, { 5, 11}, { 7, 24}, { 5,247}, {11, 85}, 
    { 4,  8}, { 6, 19}, { 5, 14}, { 7,235}, { 5, 10}, { 6,242}, { 5,246}, {10, 53}, 
    { 4,248}, { 6,238}, { 5,244}, { 8,224}, { 5, 12}, { 7, 28}, { 6, 15}, {11,151}, 
    { 4,  8}, { 6, 18}, { 5, 13}, { 7,233}, { 5,  9}, { 6,241}, { 5,245}, { 9,213}, 
    { 4,248}, { 6,237}, { 5,243}, { 8,220}, { 5, 11}, { 7, 26}, { 5,247}, {11,118}, 
    { 4,  8}, { 6, 20}, { 5, 14}, { 8, 31}, { 5, 10}, { 7, 22}, { 5,246}, {10,205}, 
    { 4,248}, { 6,239}, { 5,244}, { 9, 38}, { 5, 12}, { 7,229}, { 6, 16}, {11,183}, 
    { 4,  8}, { 6, 17}, { 5, 13}, { 7,230}, { 5,  9}, { 6,240}, { 5,245}, { 9, 44}, 
    { 4,248}, { 6,236}, { 5,243}, { 8, 34}, { 5, 11}, { 7, 23}, { 5,247}, {11, 77}, 
    { 4,  8}, { 6, 19}, { 5, 14}, { 7,234}, { 5, 10}, { 6,242}, { 5,246}, {10, 49}, 
    { 4,248}, { 6,238}, { 5,244}, { 8,223}, { 5, 12}, { 7, 27}, { 6, 15}, {11,142}, 
    { 4,  8}, { 6, 18}, { 5, 13}, { 7,232}, { 5,  9}, { 6,241}, { 5,245}, { 9,211}, 
    { 4,248}, { 6,237}, { 5,243}, { 8,219}, { 5, 11}, { 7, 25}, { 5,247}, {11,109}, 
    { 4,  8}, { 6, 20}, { 5, 14}, { 8, 30}, { 5, 10}, { 7, 21}, { 5,246}, {10,201}, 
    { 4,248}, { 6,239}, { 5,244}, { 8,227}, { 5, 12}, { 7,228}, { 6, 16}, {11,175}, 
    { 4,  8}, { 6, 17}, { 5, 13}, { 7,231}, { 5,  9}, { 6,240}, { 5,245}, { 9,112}, 
    { 4,248}, { 6,236}, { 5,243}, { 8, 36}, { 5, 11}, { 7, 24}, { 5,247}, {11, 93}, 
    { 4,  8}, { 6, 19}, { 5, 14}, { 7,235}, { 5, 10}, { 6,242}, { 5,246}, {10, 57}, 
    { 4,248}, { 6,238}, { 5,244}, { 8,225}, { 5, 12}, { 7, 28}, { 6, 15}, {11,159}, 
    { 4,  8}, { 6, 18}, { 5, 13}, { 7,233}, { 5,  9}, { 6,241}, { 5,245}, { 9,215}, 
    { 4,248}, { 6,237}, { 5,243}, { 8,221}, { 5, 11}, { 7, 26}, { 5,247}, {11,126}, 
    { 4,  8}, { 6, 20}, { 5, 14}, { 8, 32}, { 5, 10}, { 7, 22}, { 5,246}, {11, 61}, 
    { 4,248}, { 6,239}, { 5,244}, { 9, 40}, { 5, 12}, { 7,229}, { 6, 16}, {11,191}, 
    { 4,  8}, { 6, 17}, { 5, 13}, { 7,230}, { 5,  9}, { 6,240}, { 5,245}, { 9, 43}, 
    { 4,248}, { 6,236}, { 5,243}, { 8, 33}, { 5, 11}, { 7, 23}, { 5,247}, {11, 73}, 
    { 4,  8}, { 6, 19}, { 5, 14}, { 7,234}, { 5, 10}, { 6,242}, { 5,246}, { 9,218}, 
    { 4,248}, { 6,238}, { 5,244}, { 8,222}, { 5, 12}, { 7, 27}, { 6, 15}, {11,138}, 
    { 4,  8}, { 6, 18}, { 5, 13}, { 7,232}, { 5,  9}, { 6,241}, { 5,245}, { 9,210}, 
    { 4,248}, { 6,237}, { 5,243}, { 8, 37}, { 5, 11}, { 7, 25}, { 5,247}, {11,105}, 
    { 4,  8}, { 6, 20}, { 5, 14}, { 8, 29}, { 5, 10}, { 7, 21}, { 5,246}, {10,199}, 
    { 4,248}, { 6,239}, { 5,244}, { 8,226}, { 5, 12}, { 7,228}, { 6, 16}, {11,171}, 
    { 4,  8}, { 6, 17}, { 5, 13}, { 7,231}, { 5,  9}, { 6,240}, { 5,245}, { 9, 47}, 
    { 4,248}, { 6,236}, { 5,243}, { 8, 35}, { 5, 11}, { 7, 24}, { 5,247}, {11, 89}, 
    { 4,  8}, { 6, 19}, { 5, 14}, { 7,235}, { 5, 10}, { 6,242}, { 5,246}, {10, 55}, 
    { 4,248}, { 6,238}, { 5,244}, { 8,224}, { 5, 12}, { 7, 28}, { 6, 15}, {11,155}, 
    { 4,  8}, { 6, 18}, { 5, 13}, { 7,233}, { 5,  9}, { 6,241}, { 5,245}, { 9,214}, 
    { 4,248}, { 6,237}, { 5,243}, { 8,220}, { 5, 11}, { 7, 26}, { 5,247}, {11,122}, 
    { 4,  8}, { 6, 20}, { 5, 14}, { 8, 31}, { 5, 10}, { 7, 22}, { 5,246}, {10,207}, 
    { 4,248}, { 6,239}, { 5,244}, { 9, 39}, { 5, 12}, { 7,229}, { 6, 16}, {11,187}, 
    { 4,  8}, { 6, 17}, { 5, 13}, { 7,230}, { 5,  9}, { 6,240}, { 5,245}, { 9, 45}, 
    { 4,248}, { 6,236}, { 5,243}, { 8, 34}, { 5, 11}, { 7, 23}, { 5,247}, {11, 81}, 
    { 4,  8}, { 6, 19}, { 5, 14}, { 7,234}, { 5, 10}, { 6,242}, { 5,246}, {10, 51}, 
    { 4,248}, { 6,238}, { 5,244}, { 8,223}, { 5, 12}, { 7, 27}, { 6, 15}, {11,147}, 
    { 4,  8}, { 6, 18}, { 5, 13}, { 7,232}, { 5,  9}, { 6,241}, { 5,245}, { 9,212}, 
    { 4,248}, { 6,237}, { 5,243}, { 8,219}, { 5, 11}, { 7, 25}, { 5,247}, {11,114}, 
    { 4,  8}, { 6, 20}, { 5, 14}, { 8, 30}, { 5, 10}, { 7, 21}, { 5,246}, {10,203}, 
    { 4,248}, { 6,239}, { 5,244}, { 8,227}, { 5, 12}, { 7,228}, { 6, 16}, {11,179}, 
    { 4,  8}, { 6, 17}, { 5, 13}, { 7,231}, { 5,  9}, { 6,240}, { 5,245}, { 9,144}, 
    { 4,248}, { 6,236}, { 5,243}, { 8, 36}, { 5, 11}, { 7, 24}, { 5,247}, {11, 97}, 
    { 4,  8}, { 6, 19}, { 5, 14}, { 7,235}, { 5, 10}, { 6,242}, { 5,246}, {10, 59}, 
    { 4,248}, { 6,238}, { 5,244}, { 8,225}, { 5, 12}, { 7, 28}, { 6, 15}, {11,163}, 
    { 4,  8}, { 6, 18}, { 5, 13}, { 7,233}, { 5,  9}, { 6,241}, { 5,245}, { 9,216}, 
    { 4,248}, { 6,237}, { 5,243}, { 8,221}, { 5, 11}, { 7, 26}, { 5,247}, {11,130}, 
    { 4,  8}, { 6, 20}, { 5, 14}, { 8, 32}, { 5, 10}, { 7, 22}, { 5,246}, {11, 65}, 
    { 4,248}, { 6,239}, { 5,244}, { 9, 41}, { 5, 12}, { 7,229}, { 6, 16}, {11,195}, 
    { 4,  8}, { 6, 17}, { 5, 13}, { 7,230}, { 5,  9}, { 6,240}, { 5,245}, { 9, 42}, 
    { 4,248}, { 6,236}, { 5,243}, { 8, 33}, { 5, 11}, { 7, 23}, { 5,247}, {11, 68}, 
    { 4,  8}, { 6, 19}, { 5, 14}, { 7,234}, { 5, 10}, { 6,242}, { 5,246}, { 9,217}, 
    { 4,248}, { 6,238}, { 5,244}, { 8,222}, { 5, 12}, { 7, 27}, { 6, 15}, {11,133}, 
    { 4,  8}, { 6, 18}, { 5, 13}, { 7,232}, { 5,  9}, { 6,241}, { 5,245}, { 9,209}, 
    { 4,248}, { 6,237}, { 5,243}, { 8, 37}, { 5, 11}, { 7, 25}, { 5,247}, {11,100}, 
    { 4,  8}, { 6, 20}, { 5, 14}, { 8, 29}, { 5, 10}, { 7, 21}, { 5,246}, {10, 60}, 
    { 4,248}, { 6,239}, { 5,244}, { 8,226}, { 5, 12}, { 7,228}, { 6, 16}, {11,166}, 
    { 4,  8}, { 6, 17}, { 5, 13}, { 7,231}, { 5,  9}, { 6,240}, { 5,245}, { 9, 46}, 
    { 4,248}, { 6,236}, { 5,243}, { 8, 35}, { 5, 11}, { 7, 24}, { 5,247}, {11, 84}, 
    { 4,  8}, { 6, 19}, { 5, 14}, { 7,235}, { 5, 10}, { 6,242}, { 5,246}, {10, 52}, 
    { 4,248}, { 6,238}, { 5,244}, { 8,224}, { 5, 12}, { 7, 28}, { 6, 15}, {11,150}, 
    { 4,  8}, { 6, 18}, { 5, 13}, { 7,233}, { 5,  9}, { 6,241}, { 5,245}, { 9,213}, 
    { 4,248}, { 6,237}, { 5,243}, { 8,220}, { 5, 11}, { 7, 26}, { 5,247}, {11,117}, 
    { 4,  8}, { 6, 20}, { 5, 14}, { 8, 31}, { 5, 10}, { 7, 22}, { 5,246}, {10,204}, 
    { 4,248}, { 6,239}, { 5,244}, { 9, 38}, { 5, 12}, { 7,229}, { 6, 16}, {11,182}, 
    { 4,  8}, { 6, 17}, { 5, 13}, { 7,230}, { 5,  9}, { 6,240}, { 5,245}, { 9, 44}, 
    { 4,248}, { 6,236}, { 5,243}, { 8, 34}, { 5, 11}, { 7, 23}, { 5,247}, {11, 76}, 
    { 4,  8}, { 6, 19}, { 5, 14}, { 7,234}, { 5, 10}, { 6,242}, { 5,246}, {10, 48}, 
    { 4,248}, { 6,238}, { 5,244}, { 8,223}, { 5, 12}, { 7, 27}, { 6, 15}, {11,141}, 
    { 4,  8}, { 6, 18}, { 5, 13}, { 7,232}, { 5,  9}, { 6,241}, { 5,245}, { 9,211}, 
    { 4,248}, { 6,237}, { 5,243}, { 8,219}, { 5, 11}, { 7, 25}, { 5,247}, {11,108}, 
    { 4,  8}, { 6, 20}, { 5, 14}, { 8, 30}, { 5, 10}, { 7, 21}, { 5,246}, {10,200}, 
    { 4,248}, { 6,239}, { 5,244}, { 8,227}, { 5, 12}, { 7,228}, { 6, 16}, {11,174}, 
    { 4,  8}, { 6, 17}, { 5, 13}, { 7,231}, { 5,  9}, { 6,240}, { 5,245}, { 9,112}, 
    { 4,248}, { 6,236}, { 5,243}, { 8, 36}, { 5, 11}, { 7, 24}, { 5,247}, {11, 92}, 
    { 4,  8}, { 6, 19}, { 5, 14}, { 7,235}, { 5, 10}, { 6,242}, { 5,246}, {10, 56}, 
    { 4,248}, { 6,238}, { 5,244}, { 8,225}, { 5, 12}, { 7, 28}, { 6, 15}, {11,158}, 
    { 4,  8}, { 6, 18}, { 5, 13}, { 7,233}, { 5,  9}, { 6,241}, { 5,245}, { 9,215}, 
    { 4,248}, { 6,237}, { 5,243}, { 8,221}, { 5, 11}, { 7, 26}, { 5,247}, {11,125}, 
    { 4,  8}, { 6, 20}, { 5, 14}, { 8, 32}, { 5, 10}, { 7, 22}, { 5,246}, {10,208}, 
    { 4,248}, { 6,239}, { 5,244}, { 9, 40}, { 5, 12}, { 7,229}, { 6, 16}, {11,190}, 
    { 4,  8}, { 6, 17}, { 5, 13}, { 7,230}, { 5,  9}, { 6,240}, { 5,245}, { 9, 43}, 
    { 4,248}, { 6,236}, { 5,243}, { 8, 33}, { 5, 11}, { 7, 23}, { 5,247}, {11, 72}, 
    { 4,  8}, { 6, 19}, { 5, 14}, { 7,234}, { 5, 10}, { 6,242}, { 5,246}, { 9,218}, 
    { 4,248}, { 6,238}, { 5,244}, { 8,222}, { 5, 12}, { 7, 27}, { 6, 15}, {11,137}, 
    { 4,  8}, { 6, 18}, { 5, 13}, { 7,232}, { 5,  9}, { 6,241}, { 5,245}, { 9,210}, 
    { 4,248}, { 6,237}, { 5,243}, { 8, 37}, { 5, 11}, { 7, 25}, { 5,247}, {11,104}, 
    { 4,  8}, { 6, 20}, { 5, 14}, { 8, 29}, { 5, 10}, { 7, 21}, { 5,246}, {10,198}, 
    { 4,248}, { 6,239}, { 5,244}, { 8,226}, { 5, 12}, { 7,228}, { 6, 16}, {11,170}, 
    { 4,  8}, { 6, 17}, { 5, 13}, { 7,231}, { 5,  9}, { 6,240}, { 5,245}, { 9, 47}, 
    { 4,248}, { 6,236}, { 5,243}, { 8, 35}, { 5, 11}, { 7, 24}, { 5,247}, {11, 88}, 
    { 4,  8}, { 6, 19}, { 5, 14}, { 7,235}, { 5, 10}, { 6,242}, { 5,246}, {10, 54}, 
    { 4,248}, { 6,238}, { 5,244}, { 8,224}, { 5, 12}, { 7, 28}, { 6, 15}, {11,154}, 
    { 4,  8}, { 6, 18}, { 5, 13}, { 7,233}, { 5,  9}, { 6,241}, { 5,245}, { 9,214}, 
    { 4,248}, { 6,237}, { 5,243}, { 8,220}, { 5, 11}, { 7, 26}, { 5,247}, {11,121}, 
    { 4,  8}, { 6, 20}, { 5, 14}, { 8, 31}, { 5, 10}, { 7, 22}, { 5,246}, {10,206}, 
    { 4,248}, { 6,239}, { 5,244}, { 9, 39}, { 5, 12}, { 7,229}, { 6, 16}, {11,186}, 
    { 4,  8}, { 6, 17}, { 5, 13}, { 7,230}, { 5,  9}, { 6,240}, { 5,245}, { 9, 45}, 
    { 4,248}, { 6,236}, { 5,243}, { 8, 34}, { 5, 11}, { 7, 23}, { 5,247}, {11, 80}, 
    { 4,  8}, { 6, 19}, { 5, 14}, { 7,234}, { 5, 10}, { 6,242}, { 5,246}, {10, 50}, 
    { 4,248}, { 6,238}, { 5,244}, { 8,223}, { 5, 12}, { 7, 27}, { 6, 15}, {11,146}, 
    { 4,  8}, { 6, 18}, { 5, 13}, { 7,232}, { 5,  9}, { 6,241}, { 5,245}, { 9,212}, 
    { 4,248}, { 6,237}, { 5,243}, { 8,219}, { 5, 11}, { 7, 25}, { 5,247}, {11,113}, 
    { 4,  8}, { 6, 20}, { 5, 14}, { 8, 30}, { 5, 10}, { 7, 21}, { 5,246}, {10,202}, 
    { 4,248}, { 6,239}, { 5,244}, { 8,227}, { 5, 12}, { 7,228}, { 6, 16}, {11,178}, 
    { 4,  8}, { 6, 17}, { 5, 13}, { 7,231}, { 5,  9}, { 6,240}, { 5,245}, { 9,144}, 
    { 4,248}, { 6,236}, { 5,243}, { 8, 36}, { 5, 11}, { 7, 24}, { 5,247}, {11, 96}, 
    { 4,  8}, { 6, 19}, { 5, 14}, { 7,235}, { 5, 10}, { 6,242}, { 5,246}, {10, 58}, 
    { 4,248}, { 6,238}, { 5,244}, { 8,225}, { 5, 12}, { 7, 28}, { 6, 15}, {11,162}, 
    { 4,  8}, { 6, 18}, { 5, 13}, { 7,233}, { 5,  9}, { 6,241}, { 5,245}, { 9,216}, 
    { 4,248}, { 6,237}, { 5,243}, { 8,221}, { 5, 11}, { 7, 26}, { 5,247}, {11,129}, 
    { 4,  8}, { 6, 20}, { 5, 14}, { 8, 32}, { 5, 10}, { 7, 22}, { 5,246}, {11, 64}, 
    { 4,248}, { 6,239}, { 5,244}, { 9, 41}, { 5, 12}, { 7,229}, { 6, 16}, {11,194}, 
    { 4,  8}, { 6, 17}, { 5, 13}, { 7,230}, { 5,  9}, { 6,240}, { 5,245}, { 9, 42}, 
    { 4,248}, { 6,236}, { 5,243}, { 8, 33}, { 5, 11}, { 7, 23}, { 5,247}, {11, 70}, 
    { 4,  8}, { 6, 19}, { 5, 14}, { 7,234}, { 5, 10}, { 6,242}, { 5,246}, { 9,217}, 
    { 4,248}, { 6,238}, { 5,244}, { 8,222}, { 5, 12}, { 7, 27}, { 6, 15}, {11,135}, 
    { 4,  8}, { 6, 18}, { 5, 13}, { 7,232}, { 5,  9}, { 6,241}, { 5,245}, { 9,209}, 
    { 4,248}, { 6,237}, { 5,243}, { 8, 37}, { 5, 11}, { 7, 25}, { 5,247}, {11,102}, 
    { 4,  8}, { 6, 20}, { 5, 14}, { 8, 29}, { 5, 10}, { 7, 21}, { 5,246}, {10,197}, 
    { 4,248}, { 6,239}, { 5,244}, { 8,226}, { 5, 12}, { 7,228}, { 6, 16}, {11,168}, 
    { 4,  8}, { 6, 17}, { 5, 13}, { 7,231}, { 5,  9}, { 6,240}, { 5,245}, { 9, 46}, 
    { 4,248}, { 6,236}, { 5,243}, { 8, 35}, { 5, 11}, { 7, 24}, { 5,247}, {11, 86}, 
    { 4,  8}, { 6, 19}, { 5, 14}, { 7,235}, { 5, 10}, { 6,242}, { 5,246}, {10, 53}, 
    { 4,248}, { 6,238}, { 5,244}, { 8,224}, { 5, 12}, { 7, 28}, { 6, 15}, {11,152}, 
    { 4,  8}, { 6, 18}, { 5, 13}, { 7,233}, { 5,  9}, { 6,241}, { 5,245}, { 9,213}, 
    { 4,248}, { 6,237}, { 5,243}, { 8,220}, { 5, 11}, { 7, 26}, { 5,247}, {11,119}, 
    { 4,  8}, { 6, 20}, { 5, 14}, { 8, 31}, { 5, 10}, { 7, 22}, { 5,246}, {10,205}, 
    { 4,248}, { 6,239}, { 5,244}, { 9, 38}, { 5, 12}, { 7,229}, { 6, 16}, {11,184}, 
    { 4,  8}, { 6, 17}, { 5, 13}, { 7,230}, { 5,  9}, { 6,240}, { 5,245}, { 9, 44}, 
    { 4,248}, { 6,236}, { 5,243}, { 8, 34}, { 5, 11}, { 7, 23}, { 5,247}, {11, 78}, 
    { 4,  8}, { 6, 19}, { 5, 14}, { 7,234}, { 5, 10}, { 6,242}, { 5,246}, {10, 49}, 
    { 4,248}, { 6,238}, { 5,244}, { 8,223}, { 5, 12}, { 7, 27}, { 6, 15}, {11,143}, 
    { 4,  8}, { 6, 18}, { 5, 13}, { 7,232}, { 5,  9}, { 6,241}, { 5,245}, { 9,211}, 
    { 4,248}, { 6,237}, { 5,243}, { 8,219}, { 5, 11}, { 7, 25}, { 5,247}, {11,110}, 
    { 4,  8}, { 6, 20}, { 5, 14}, { 8, 30}, { 5, 10}, { 7, 21}, { 5,246}, {10,201}, 
    { 4,248}, { 6,239}, { 5,244}, { 8,227}, { 5, 12}, { 7,228}, { 6, 16}, {11,176}, 
    { 4,  8}, { 6, 17}, { 5, 13}, { 7,231}, { 5,  9}, { 6,240}, { 5,245}, { 9,112}, 
    { 4,248}, { 6,236}, { 5,243}, { 8, 36}, { 5, 11}, { 7, 24}, { 5,247}, {11, 94}, 
    { 4,  8}, { 6, 19}, { 5, 14}, { 7,235}, { 5, 10}, { 6,242}, { 5,246}, {10, 57}, 
    { 4,248}, { 6,238}, { 5,244}, { 8,225}, { 5, 12}, { 7, 28}, { 6, 15}, {11,160}, 
    { 4,  8}, { 6, 18}, { 5, 13}, { 7,233}, { 5,  9}, { 6,241}, { 5,245}, { 9,215}, 
    { 4,248}, { 6,237}, { 5,243}, { 8,221}, { 5, 11}, { 7, 26}, { 5,247}, {11,127}, 
    { 4,  8}, { 6, 20}, { 5, 14}, { 8, 32}, { 5, 10}, { 7, 22}, { 5,246}, {11, 62}, 
    { 4,248}, { 6,239}, { 5,244}, { 9, 40}, { 5, 12}, { 7,229}, { 6, 16}, {11,192}, 
    { 4,  8}, { 6, 17}, { 5, 13}, { 7,230}, { 5,  9}, { 6,240}, { 5,245}, { 9, 43}, 
    { 4,248}, { 6,236}, { 5,243}, { 8, 33}, { 5, 11}, { 7, 23}, { 5,247}, {11, 74}, 
    { 4,  8}, { 6, 19}, { 5, 14}, { 7,234}, { 5, 10}, { 6,242}, { 5,246}, { 9,218}, 
    { 4,248}, { 6,238}, { 5,244}, { 8,222}, { 5, 12}, { 7, 27}, { 6, 15}, {11,139}, 
    { 4,  8}, { 6, 18}, { 5, 13}, { 7,232}, { 5,  9}, { 6,241}, { 5,245}, { 9,210}, 
    { 4,248}, { 6,237}, { 5,243}, { 8, 37}, { 5, 11}, { 7, 25}, { 5,247}, {11,106}, 
    { 4,  8}, { 6, 20}, { 5, 14}, { 8, 29}, { 5, 10}, { 7, 21}, { 5,246}, {10,199}, 
    { 4,248}, { 6,239}, { 5,244}, { 8,226}, { 5, 12}, { 7,228}, { 6, 16}, {11,172}, 
    { 4,  8}, { 6, 17}, { 5, 13}, { 7,231}, { 5,  9}, { 6,240}, { 5,245}, { 9, 47}, 
    { 4,248}, { 6,236}, { 5,243}, { 8, 35}, { 5, 11}, { 7, 24}, { 5,247}, {11, 90}, 
    { 4,  8}, { 6, 19}, { 5, 14}, { 7,235}, { 5, 10}, { 6,242}, { 5,246}, {10, 55}, 
    { 4,248}, { 6,238}, { 5,244}, { 8,224}, { 5, 12}, { 7, 28}, { 6, 15}, {11,156}, 
    { 4,  8}, { 6, 18}, { 5, 13}, { 7,233}, { 5,  9}, { 6,241}, { 5,245}, { 9,214}, 
    { 4,248}, { 6,237}, { 5,243}, { 8,220}, { 5, 11}, { 7, 26}, { 5,247}, {11,123}, 
    { 4,  8}, { 6, 20}, { 5, 14}, { 8, 31}, { 5, 10}, { 7, 22}, { 5,246}, {10,207}, 
    { 4,248}, { 6,239}, { 5,244}, { 9, 39}, { 5, 12}, { 7,229}, { 6, 16}, {11,188}, 
    { 4,  8}, { 6, 17}, { 5, 13}, { 7,230}, { 5,  9}, { 6,240}, { 5,245}, { 9, 45}, 
    { 4,248}, { 6,236}, { 5,243}, { 8, 34}, { 5, 11}, { 7, 23}, { 5,247}, {11, 82}, 
    { 4,  8}, { 6, 19}, { 5, 14}, { 7,234}, { 5, 10}, { 6,242}, { 5,246}, {10, 51}, 
    { 4,248}, { 6,238}, { 5,244}, { 8,223}, { 5, 12}, { 7, 27}, { 6, 15}, {11,148}, 
    { 4,  8}, { 6, 18}, { 5, 13}, { 7,232}, { 5,  9}, { 6,241}, { 5,245}, { 9,212}, 
    { 4,248}, { 6,237}, { 5,243}, { 8,219}, { 5, 11}, { 7, 25}, { 5,247}, {11,115}, 
    { 4,  8}, { 6, 20}, { 5, 14}, { 8, 30}, { 5, 10}, { 7, 21}, { 5,246}, {10,203}, 
    { 4,248}, { 6,239}, { 5,244}, { 8,227}, { 5, 12}, { 7,228}, { 6, 16}, {11,180}, 
    { 4,  8}, { 6, 17}, { 5, 13}, { 7,231}, { 5,  9}, { 6,240}, { 5,245}, { 9,144}, 
    { 4,248}, { 6,236}, { 5,243}, { 8, 36}, { 5, 11}, { 7, 24}, { 5,247}, {11, 98}, 
    { 4,  8}, { 6, 19}, { 5, 14}, { 7,235}, { 5, 10}, { 6,242}, { 5,246}, {10, 59}, 
    { 4,248}, { 6,238}, { 5,244}, { 8,225}, { 5, 12}, { 7, 28}, { 6, 15}, {11,164}, 
    { 4,  8}, { 6, 18}, { 5, 13}, { 7,233}, { 5,  9}, { 6,241}, { 5,245}, { 9,216}, 
    { 4,248}, { 6,237}, { 5,243}, { 8,221}, { 5, 11}, { 7, 26}, { 5,247}, {11,131}, 
    { 4,  8}, { 6, 20}, { 5, 14}, { 8, 32}, { 5, 10}, { 7, 22}, { 5,246}, {11, 66}, 
    { 4,248}, { 6,239}, { 5,244}, { 9, 41}, { 5, 12}, { 7,229}, { 6, 16}, {11,196}, 
  }
  #endif
};

static radaudio_huffman rada_nonzero_coefficient_pair_huff0 =
{
  #ifdef HUFFMAN_ENCODE
  {
    { 503,11}, {1527,11}, {1015,11}, {2039,11}, {  15,11}, {1039,11}, { 231,10}, { 219, 9}, 
    {   0, 0}, { 475, 9}, { 743,10}, { 527,11}, {1551,11}, { 271,11}, {1295,11}, { 783,11}, 
    {1807,11}, { 143,11}, {1167,11}, { 655,11}, {1679,11}, { 399,11}, {1423,11}, { 911,11}, 
    {   0, 0}, {1935,11}, {  79,11}, {1103,11}, { 591,11}, {1615,11}, { 335,11}, {1359,11}, 
    { 847,11}, {1871,11}, { 207,11}, {1231,11}, { 719,11}, {1743,11}, { 463,11}, { 487,10}, 
    {   0, 0}, { 999,10}, {1487,11}, { 975,11}, {1999,11}, {  47,11}, {1071,11}, { 559,11}, 
    {1583,11}, { 303,11}, {1327,11}, { 815,11}, {1839,11}, { 175,11}, {  23,10}, {  59, 9}, 
    {   0, 0}, { 535,10}, { 279,10}, {1199,11}, { 687,11}, {1711,11}, { 431,11}, {1455,11}, 
    { 943,11}, {1967,11}, { 111,11}, {1135,11}, { 623,11}, { 791,10}, { 315, 9}, { 187, 9}, 
    {   0, 0}, { 443, 9}, { 123, 9}, { 151,10}, {1647,11}, { 367,11}, {1391,11}, { 879,11}, 
    { 663,10}, {1903,11}, { 239,11}, {1263,11}, { 407,10}, { 379, 9}, {  11, 8}, {  29, 7}, 
    {   0, 0}, {  93, 7}, { 139, 8}, { 251, 9}, { 919,10}, { 751,11}, {1775,11}, { 495,11}, 
    {  87,10}, {1519,11}, {1007,11}, { 599,10}, { 507, 9}, {  75, 8}, {  61, 7}, {  13, 6}, 
    {   0, 0}, {   1, 5}, { 125, 7}, { 203, 8}, {   7, 9}, { 343,10}, {2031,11}, {  31,11}, 
    { 263, 9}, {1055,11}, { 855,10}, { 215,10}, {  43, 8}, {   3, 7}, {  17, 5}, {   0, 3}, 
    {   0, 0}, {   4, 3}, {   9, 5}, {  67, 7}, { 135, 9}, { 727,10}, { 471,10}, { 543,11}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    { 391, 9}, {1567,11}, { 983,10}, {  55,10}, {  71, 9}, {  35, 7}, {  25, 5}, {   2, 3}, 
    {   0, 0}, {   6, 3}, {   5, 5}, {  99, 7}, { 171, 8}, { 327, 9}, { 567,10}, { 287,11}, 
    { 311,10}, {1311,11}, { 799,11}, { 823,10}, { 199, 9}, { 107, 8}, {  19, 7}, {  21, 5}, 
    {   0, 0}, {  45, 6}, {  83, 7}, { 235, 8}, { 455, 9}, { 183,10}, {1823,11}, { 159,11}, 
    {1183,11}, { 671,11}, {1695,11}, { 415,11}, { 695,10}, {  39, 9}, {  27, 8}, {  51, 7}, 
    {   0, 0}, { 115, 7}, { 155, 8}, { 295, 9}, { 439,10}, {1439,11}, { 927,11}, {1951,11}, 
    {  95,11}, {1119,11}, { 607,11}, {1631,11}, { 351,11}, { 951,10}, { 167, 9}, { 423, 9}, 
    {   0, 0}, {  91, 8}, { 103, 9}, { 119,10}, {1375,11}, { 863,11}, {1887,11}, { 223,11}, 
    {1247,11}, { 735,11}, {1759,11}, { 479,11}, {1503,11}, { 991,11}, { 631,10}, { 375,10}, 
    {   0, 0}, { 359, 9}, { 887,10}, {2015,11}, {  63,11}, {1087,11}, { 575,11}, {1599,11}, 
    { 319,11}, {1343,11}, { 831,11}, {1855,11}, { 191,11}, {1215,11}, { 703,11}, { 247,10}, 
    {   0, 0}, { 759,10}, {1727,11}, { 447,11}, {1471,11}, { 959,11}, {1983,11}, { 127,11}, 
    {1151,11}, { 639,11}, {1663,11}, { 383,11}, {1407,11}, { 895,11}, {1919,11}, { 255,11}, 
    {   0, 0}, {1279,11}, { 767,11}, {1791,11}, { 511,11}, {1535,11}, {1023,11}, {2047,11}, 
  },
  #endif
  #ifdef HUFFMAN_DECODE
  {
    { 3,119}, { 5,105}, { 3,151}, { 7,117}, { 3,121}, { 5,154}, { 3,153}, { 9,108}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 86}, { 3,121}, { 6,103}, { 3,153}, {11,  4}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,166}, { 3,121}, { 5,167}, { 3,153}, {10, 54}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,182}, { 3,121}, { 7, 87}, { 3,153}, {11,111}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,149}, { 3,121}, { 5,154}, { 3,153}, { 9,181}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,116}, { 3,121}, { 6,169}, { 3,153}, {11, 45}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,183}, { 3,121}, { 5,167}, { 3,153}, {10,147}, 
    { 3,119}, { 5,150}, { 3,151}, { 9, 55}, { 3,121}, { 7,102}, { 3,153}, {11,220}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,123}, { 3,121}, { 5,154}, { 3,153}, { 9,148}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,101}, { 3,121}, { 6,103}, { 3,153}, {11, 26}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,170}, { 3,121}, { 5,167}, { 3,153}, {10, 96}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,201}, { 3,121}, { 7, 89}, { 3,153}, {11,192}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,155}, { 3,121}, { 5,154}, { 3,153}, { 9,202}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,165}, { 3,121}, { 6,169}, { 3,153}, {11, 66}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,185}, { 3,121}, { 5,167}, { 3,153}, {10,203}, 
    { 3,119}, { 5,150}, { 3,151}, { 9, 74}, { 3,121}, { 7,106}, { 3,153}, {11,239}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,117}, { 3,121}, { 5,154}, { 3,153}, { 9,124}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 90}, { 3,121}, { 6,103}, { 3,153}, {11, 17}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,166}, { 3,121}, { 5,167}, { 3,153}, {10, 75}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,186}, { 3,121}, { 7, 87}, { 3,153}, {11,175}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,149}, { 3,121}, { 5,154}, { 3,153}, { 9,198}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,156}, { 3,121}, { 6,169}, { 3,153}, {11, 53}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,183}, { 3,121}, { 5,167}, { 3,153}, {10,173}, 
    { 3,119}, { 5,150}, { 3,151}, { 9, 71}, { 3,121}, { 7,102}, { 3,153}, {11,228}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,123}, { 3,121}, { 5,154}, { 3,153}, { 9,164}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,107}, { 3,121}, { 6,103}, { 3,153}, {11, 34}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,170}, { 3,121}, { 5,167}, { 3,153}, {10,115}, 
    { 3,119}, { 5,150}, { 3,151}, { 9,  7}, { 3,121}, { 7, 89}, { 3,153}, {11,207}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,155}, { 3,121}, { 5,154}, { 3,153}, {10,  6}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,171}, { 3,121}, { 6,169}, { 3,153}, {11, 82}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,185}, { 3,121}, { 5,167}, { 3,153}, {10,231}, 
    { 3,119}, { 5,150}, { 3,151}, { 9, 91}, { 3,121}, { 7,106}, { 3,153}, {11,247}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,117}, { 3,121}, { 5,154}, { 3,153}, { 9,112}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 86}, { 3,121}, { 6,103}, { 3,153}, {11, 13}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,166}, { 3,121}, { 5,167}, { 3,153}, {10, 58}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,182}, { 3,121}, { 7, 87}, { 3,153}, {11,159}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,149}, { 3,121}, { 5,154}, { 3,153}, { 9,187}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,116}, { 3,121}, { 6,169}, { 3,153}, {11, 49}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,183}, { 3,121}, { 5,167}, { 3,153}, {10,160}, 
    { 3,119}, { 5,150}, { 3,151}, { 9, 70}, { 3,121}, { 7,102}, { 3,153}, {11,224}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,123}, { 3,121}, { 5,154}, { 3,153}, { 9,157}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,101}, { 3,121}, { 6,103}, { 3,153}, {11, 30}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,170}, { 3,121}, { 5,167}, { 3,153}, {10,109}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,201}, { 3,121}, { 7, 89}, { 3,153}, {11,196}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,155}, { 3,121}, { 5,154}, { 3,153}, { 9,217}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,165}, { 3,121}, { 6,169}, { 3,153}, {11, 77}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,185}, { 3,121}, { 5,167}, { 3,153}, {10,215}, 
    { 3,119}, { 5,150}, { 3,151}, { 9, 85}, { 3,121}, { 7,106}, { 3,153}, {11,243}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,117}, { 3,121}, { 5,154}, { 3,153}, { 9,144}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 90}, { 3,121}, { 6,103}, { 3,153}, {11, 21}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,166}, { 3,121}, { 5,167}, { 3,153}, {10, 84}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,186}, { 3,121}, { 7, 87}, { 3,153}, {11,179}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,149}, { 3,121}, { 5,154}, { 3,153}, { 9,199}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,156}, { 3,121}, { 6,169}, { 3,153}, {11, 62}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,183}, { 3,121}, { 5,167}, { 3,153}, {10,188}, 
    { 3,119}, { 5,150}, { 3,151}, { 9, 73}, { 3,121}, { 7,102}, { 3,153}, {11,235}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,123}, { 3,121}, { 5,154}, { 3,153}, { 9,172}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,107}, { 3,121}, { 6,103}, { 3,153}, {11, 38}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,170}, { 3,121}, { 5,167}, { 3,153}, {10,126}, 
    { 3,119}, { 5,150}, { 3,151}, { 9,  9}, { 3,121}, { 7, 89}, { 3,153}, {11,211}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,155}, { 3,121}, { 5,154}, { 3,153}, {10, 39}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,171}, { 3,121}, { 6,169}, { 3,153}, {11, 95}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,185}, { 3,121}, { 5,167}, { 3,153}, {11,  0}, 
    { 3,119}, { 5,150}, { 3,151}, { 9,100}, { 3,121}, { 7,106}, { 3,153}, {11,252}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,117}, { 3,121}, { 5,154}, { 3,153}, { 9,108}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 86}, { 3,121}, { 6,103}, { 3,153}, {11, 11}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,166}, { 3,121}, { 5,167}, { 3,153}, {10, 57}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,182}, { 3,121}, { 7, 87}, { 3,153}, {11,127}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,149}, { 3,121}, { 5,154}, { 3,153}, { 9,181}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,116}, { 3,121}, { 6,169}, { 3,153}, {11, 47}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,183}, { 3,121}, { 5,167}, { 3,153}, {10,158}, 
    { 3,119}, { 5,150}, { 3,151}, { 9, 55}, { 3,121}, { 7,102}, { 3,153}, {11,222}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,123}, { 3,121}, { 5,154}, { 3,153}, { 9,148}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,101}, { 3,121}, { 6,103}, { 3,153}, {11, 28}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,170}, { 3,121}, { 5,167}, { 3,153}, {10, 99}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,201}, { 3,121}, { 7, 89}, { 3,153}, {11,194}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,155}, { 3,121}, { 5,154}, { 3,153}, { 9,202}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,165}, { 3,121}, { 6,169}, { 3,153}, {11, 68}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,185}, { 3,121}, { 5,167}, { 3,153}, {10,214}, 
    { 3,119}, { 5,150}, { 3,151}, { 9, 74}, { 3,121}, { 7,106}, { 3,153}, {11,241}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,117}, { 3,121}, { 5,154}, { 3,153}, { 9,124}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 90}, { 3,121}, { 6,103}, { 3,153}, {11, 19}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,166}, { 3,121}, { 5,167}, { 3,153}, {10, 80}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,186}, { 3,121}, { 7, 87}, { 3,153}, {11,177}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,149}, { 3,121}, { 5,154}, { 3,153}, { 9,198}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,156}, { 3,121}, { 6,169}, { 3,153}, {11, 60}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,183}, { 3,121}, { 5,167}, { 3,153}, {10,180}, 
    { 3,119}, { 5,150}, { 3,151}, { 9, 71}, { 3,121}, { 7,102}, { 3,153}, {11,230}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,123}, { 3,121}, { 5,154}, { 3,153}, { 9,164}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,107}, { 3,121}, { 6,103}, { 3,153}, {11, 36}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,170}, { 3,121}, { 5,167}, { 3,153}, {10,125}, 
    { 3,119}, { 5,150}, { 3,151}, { 9,  7}, { 3,121}, { 7, 89}, { 3,153}, {11,209}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,155}, { 3,121}, { 5,154}, { 3,153}, {10, 10}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,171}, { 3,121}, { 6,169}, { 3,153}, {11, 93}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,185}, { 3,121}, { 5,167}, { 3,153}, {10,233}, 
    { 3,119}, { 5,150}, { 3,151}, { 9, 91}, { 3,121}, { 7,106}, { 3,153}, {11,250}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,117}, { 3,121}, { 5,154}, { 3,153}, { 9,112}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 86}, { 3,121}, { 6,103}, { 3,153}, {11, 15}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,166}, { 3,121}, { 5,167}, { 3,153}, {10, 69}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,182}, { 3,121}, { 7, 87}, { 3,153}, {11,162}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,149}, { 3,121}, { 5,154}, { 3,153}, { 9,187}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,116}, { 3,121}, { 6,169}, { 3,153}, {11, 51}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,183}, { 3,121}, { 5,167}, { 3,153}, {10,163}, 
    { 3,119}, { 5,150}, { 3,151}, { 9, 70}, { 3,121}, { 7,102}, { 3,153}, {11,226}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,123}, { 3,121}, { 5,154}, { 3,153}, { 9,157}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,101}, { 3,121}, { 6,103}, { 3,153}, {11, 32}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,170}, { 3,121}, { 5,167}, { 3,153}, {10,114}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,201}, { 3,121}, { 7, 89}, { 3,153}, {11,205}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,155}, { 3,121}, { 5,154}, { 3,153}, { 9,217}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,165}, { 3,121}, { 6,169}, { 3,153}, {11, 79}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,185}, { 3,121}, { 5,167}, { 3,153}, {10,218}, 
    { 3,119}, { 5,150}, { 3,151}, { 9, 85}, { 3,121}, { 7,106}, { 3,153}, {11,245}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,117}, { 3,121}, { 5,154}, { 3,153}, { 9,144}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 90}, { 3,121}, { 6,103}, { 3,153}, {11, 23}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,166}, { 3,121}, { 5,167}, { 3,153}, {10, 92}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,186}, { 3,121}, { 7, 87}, { 3,153}, {11,190}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,149}, { 3,121}, { 5,154}, { 3,153}, { 9,199}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,156}, { 3,121}, { 6,169}, { 3,153}, {11, 64}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,183}, { 3,121}, { 5,167}, { 3,153}, {10,197}, 
    { 3,119}, { 5,150}, { 3,151}, { 9, 73}, { 3,121}, { 7,102}, { 3,153}, {11,237}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,123}, { 3,121}, { 5,154}, { 3,153}, { 9,172}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,107}, { 3,121}, { 6,103}, { 3,153}, {11, 43}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,170}, { 3,121}, { 5,167}, { 3,153}, {10,146}, 
    { 3,119}, { 5,150}, { 3,151}, { 9,  9}, { 3,121}, { 7, 89}, { 3,153}, {11,213}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,155}, { 3,121}, { 5,154}, { 3,153}, {10, 41}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,171}, { 3,121}, { 6,169}, { 3,153}, {11, 98}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,185}, { 3,121}, { 5,167}, { 3,153}, {11,  2}, 
    { 3,119}, { 5,150}, { 3,151}, { 9,100}, { 3,121}, { 7,106}, { 3,153}, {11,254}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,117}, { 3,121}, { 5,154}, { 3,153}, { 9,108}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 86}, { 3,121}, { 6,103}, { 3,153}, {11,  5}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,166}, { 3,121}, { 5,167}, { 3,153}, {10, 54}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,182}, { 3,121}, { 7, 87}, { 3,153}, {11,113}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,149}, { 3,121}, { 5,154}, { 3,153}, { 9,181}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,116}, { 3,121}, { 6,169}, { 3,153}, {11, 46}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,183}, { 3,121}, { 5,167}, { 3,153}, {10,147}, 
    { 3,119}, { 5,150}, { 3,151}, { 9, 55}, { 3,121}, { 7,102}, { 3,153}, {11,221}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,123}, { 3,121}, { 5,154}, { 3,153}, { 9,148}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,101}, { 3,121}, { 6,103}, { 3,153}, {11, 27}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,170}, { 3,121}, { 5,167}, { 3,153}, {10, 96}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,201}, { 3,121}, { 7, 89}, { 3,153}, {11,193}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,155}, { 3,121}, { 5,154}, { 3,153}, { 9,202}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,165}, { 3,121}, { 6,169}, { 3,153}, {11, 67}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,185}, { 3,121}, { 5,167}, { 3,153}, {10,203}, 
    { 3,119}, { 5,150}, { 3,151}, { 9, 74}, { 3,121}, { 7,106}, { 3,153}, {11,240}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,117}, { 3,121}, { 5,154}, { 3,153}, { 9,124}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 90}, { 3,121}, { 6,103}, { 3,153}, {11, 18}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,166}, { 3,121}, { 5,167}, { 3,153}, {10, 75}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,186}, { 3,121}, { 7, 87}, { 3,153}, {11,176}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,149}, { 3,121}, { 5,154}, { 3,153}, { 9,198}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,156}, { 3,121}, { 6,169}, { 3,153}, {11, 59}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,183}, { 3,121}, { 5,167}, { 3,153}, {10,173}, 
    { 3,119}, { 5,150}, { 3,151}, { 9, 71}, { 3,121}, { 7,102}, { 3,153}, {11,229}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,123}, { 3,121}, { 5,154}, { 3,153}, { 9,164}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,107}, { 3,121}, { 6,103}, { 3,153}, {11, 35}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,170}, { 3,121}, { 5,167}, { 3,153}, {10,115}, 
    { 3,119}, { 5,150}, { 3,151}, { 9,  7}, { 3,121}, { 7, 89}, { 3,153}, {11,208}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,155}, { 3,121}, { 5,154}, { 3,153}, {10,  6}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,171}, { 3,121}, { 6,169}, { 3,153}, {11, 83}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,185}, { 3,121}, { 5,167}, { 3,153}, {10,231}, 
    { 3,119}, { 5,150}, { 3,151}, { 9, 91}, { 3,121}, { 7,106}, { 3,153}, {11,249}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,117}, { 3,121}, { 5,154}, { 3,153}, { 9,112}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 86}, { 3,121}, { 6,103}, { 3,153}, {11, 14}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,166}, { 3,121}, { 5,167}, { 3,153}, {10, 58}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,182}, { 3,121}, { 7, 87}, { 3,153}, {11,161}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,149}, { 3,121}, { 5,154}, { 3,153}, { 9,187}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,116}, { 3,121}, { 6,169}, { 3,153}, {11, 50}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,183}, { 3,121}, { 5,167}, { 3,153}, {10,160}, 
    { 3,119}, { 5,150}, { 3,151}, { 9, 70}, { 3,121}, { 7,102}, { 3,153}, {11,225}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,123}, { 3,121}, { 5,154}, { 3,153}, { 9,157}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,101}, { 3,121}, { 6,103}, { 3,153}, {11, 31}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,170}, { 3,121}, { 5,167}, { 3,153}, {10,109}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,201}, { 3,121}, { 7, 89}, { 3,153}, {11,204}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,155}, { 3,121}, { 5,154}, { 3,153}, { 9,217}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,165}, { 3,121}, { 6,169}, { 3,153}, {11, 78}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,185}, { 3,121}, { 5,167}, { 3,153}, {10,215}, 
    { 3,119}, { 5,150}, { 3,151}, { 9, 85}, { 3,121}, { 7,106}, { 3,153}, {11,244}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,117}, { 3,121}, { 5,154}, { 3,153}, { 9,144}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 90}, { 3,121}, { 6,103}, { 3,153}, {11, 22}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,166}, { 3,121}, { 5,167}, { 3,153}, {10, 84}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,186}, { 3,121}, { 7, 87}, { 3,153}, {11,189}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,149}, { 3,121}, { 5,154}, { 3,153}, { 9,199}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,156}, { 3,121}, { 6,169}, { 3,153}, {11, 63}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,183}, { 3,121}, { 5,167}, { 3,153}, {10,188}, 
    { 3,119}, { 5,150}, { 3,151}, { 9, 73}, { 3,121}, { 7,102}, { 3,153}, {11,236}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,123}, { 3,121}, { 5,154}, { 3,153}, { 9,172}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,107}, { 3,121}, { 6,103}, { 3,153}, {11, 42}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,170}, { 3,121}, { 5,167}, { 3,153}, {10,126}, 
    { 3,119}, { 5,150}, { 3,151}, { 9,  9}, { 3,121}, { 7, 89}, { 3,153}, {11,212}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,155}, { 3,121}, { 5,154}, { 3,153}, {10, 39}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,171}, { 3,121}, { 6,169}, { 3,153}, {11, 97}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,185}, { 3,121}, { 5,167}, { 3,153}, {11,  1}, 
    { 3,119}, { 5,150}, { 3,151}, { 9,100}, { 3,121}, { 7,106}, { 3,153}, {11,253}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,117}, { 3,121}, { 5,154}, { 3,153}, { 9,108}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 86}, { 3,121}, { 6,103}, { 3,153}, {11, 12}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,166}, { 3,121}, { 5,167}, { 3,153}, {10, 57}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,182}, { 3,121}, { 7, 87}, { 3,153}, {11,145}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,149}, { 3,121}, { 5,154}, { 3,153}, { 9,181}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,116}, { 3,121}, { 6,169}, { 3,153}, {11, 48}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,183}, { 3,121}, { 5,167}, { 3,153}, {10,158}, 
    { 3,119}, { 5,150}, { 3,151}, { 9, 55}, { 3,121}, { 7,102}, { 3,153}, {11,223}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,123}, { 3,121}, { 5,154}, { 3,153}, { 9,148}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,101}, { 3,121}, { 6,103}, { 3,153}, {11, 29}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,170}, { 3,121}, { 5,167}, { 3,153}, {10, 99}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,201}, { 3,121}, { 7, 89}, { 3,153}, {11,195}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,155}, { 3,121}, { 5,154}, { 3,153}, { 9,202}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,165}, { 3,121}, { 6,169}, { 3,153}, {11, 76}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,185}, { 3,121}, { 5,167}, { 3,153}, {10,214}, 
    { 3,119}, { 5,150}, { 3,151}, { 9, 74}, { 3,121}, { 7,106}, { 3,153}, {11,242}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,117}, { 3,121}, { 5,154}, { 3,153}, { 9,124}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 90}, { 3,121}, { 6,103}, { 3,153}, {11, 20}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,166}, { 3,121}, { 5,167}, { 3,153}, {10, 80}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,186}, { 3,121}, { 7, 87}, { 3,153}, {11,178}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,149}, { 3,121}, { 5,154}, { 3,153}, { 9,198}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,156}, { 3,121}, { 6,169}, { 3,153}, {11, 61}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,183}, { 3,121}, { 5,167}, { 3,153}, {10,180}, 
    { 3,119}, { 5,150}, { 3,151}, { 9, 71}, { 3,121}, { 7,102}, { 3,153}, {11,234}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,123}, { 3,121}, { 5,154}, { 3,153}, { 9,164}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,107}, { 3,121}, { 6,103}, { 3,153}, {11, 37}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,170}, { 3,121}, { 5,167}, { 3,153}, {10,125}, 
    { 3,119}, { 5,150}, { 3,151}, { 9,  7}, { 3,121}, { 7, 89}, { 3,153}, {11,210}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,155}, { 3,121}, { 5,154}, { 3,153}, {10, 10}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,171}, { 3,121}, { 6,169}, { 3,153}, {11, 94}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,185}, { 3,121}, { 5,167}, { 3,153}, {10,233}, 
    { 3,119}, { 5,150}, { 3,151}, { 9, 91}, { 3,121}, { 7,106}, { 3,153}, {11,251}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,117}, { 3,121}, { 5,154}, { 3,153}, { 9,112}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 86}, { 3,121}, { 6,103}, { 3,153}, {11, 16}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,166}, { 3,121}, { 5,167}, { 3,153}, {10, 69}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,182}, { 3,121}, { 7, 87}, { 3,153}, {11,174}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,149}, { 3,121}, { 5,154}, { 3,153}, { 9,187}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,116}, { 3,121}, { 6,169}, { 3,153}, {11, 52}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,183}, { 3,121}, { 5,167}, { 3,153}, {10,163}, 
    { 3,119}, { 5,150}, { 3,151}, { 9, 70}, { 3,121}, { 7,102}, { 3,153}, {11,227}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,123}, { 3,121}, { 5,154}, { 3,153}, { 9,157}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,101}, { 3,121}, { 6,103}, { 3,153}, {11, 33}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,170}, { 3,121}, { 5,167}, { 3,153}, {10,114}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,201}, { 3,121}, { 7, 89}, { 3,153}, {11,206}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,155}, { 3,121}, { 5,154}, { 3,153}, { 9,217}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,165}, { 3,121}, { 6,169}, { 3,153}, {11, 81}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,185}, { 3,121}, { 5,167}, { 3,153}, {10,218}, 
    { 3,119}, { 5,150}, { 3,151}, { 9, 85}, { 3,121}, { 7,106}, { 3,153}, {11,246}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,117}, { 3,121}, { 5,154}, { 3,153}, { 9,144}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 90}, { 3,121}, { 6,103}, { 3,153}, {11, 25}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,166}, { 3,121}, { 5,167}, { 3,153}, {10, 92}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,186}, { 3,121}, { 7, 87}, { 3,153}, {11,191}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,149}, { 3,121}, { 5,154}, { 3,153}, { 9,199}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,156}, { 3,121}, { 6,169}, { 3,153}, {11, 65}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,183}, { 3,121}, { 5,167}, { 3,153}, {10,197}, 
    { 3,119}, { 5,150}, { 3,151}, { 9, 73}, { 3,121}, { 7,102}, { 3,153}, {11,238}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,123}, { 3,121}, { 5,154}, { 3,153}, { 9,172}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,107}, { 3,121}, { 6,103}, { 3,153}, {11, 44}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,170}, { 3,121}, { 5,167}, { 3,153}, {10,146}, 
    { 3,119}, { 5,150}, { 3,151}, { 9,  9}, { 3,121}, { 7, 89}, { 3,153}, {11,219}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,155}, { 3,121}, { 5,154}, { 3,153}, {10, 41}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,171}, { 3,121}, { 6,169}, { 3,153}, {11,110}, 
    { 3,119}, { 5,118}, { 3,151}, { 7,185}, { 3,121}, { 5,167}, { 3,153}, {11,  3}, 
    { 3,119}, { 5,150}, { 3,151}, { 9,100}, { 3,121}, { 7,106}, { 3,153}, {11,255}, 
  }
  #endif
};

static radaudio_huffman rada_nonzero_coefficient_pair_huff1 =
{
  #ifdef HUFFMAN_ENCODE
  {
    {  30, 6}, { 183,10}, {  75, 9}, { 331, 9}, { 203, 9}, {  29, 8}, { 157, 8}, {  25, 7}, 
    {   0, 0}, {  89, 7}, {  93, 8}, { 221, 8}, { 459, 9}, {  43, 9}, { 299, 9}, { 695,10}, 
    { 439,10}, { 111,11}, {1135,11}, { 623,11}, {1647,11}, { 367,11}, {1391,11}, { 951,10}, 
    {   0, 0}, { 119,10}, { 879,11}, {1903,11}, { 239,11}, {1263,11}, { 751,11}, {1775,11}, 
    { 171, 9}, { 495,11}, {1519,11}, {1007,11}, {2031,11}, {  31,11}, { 631,10}, { 427, 9}, 
    {   0, 0}, { 107, 9}, { 375,10}, {1055,11}, { 543,11}, {1567,11}, { 287,11}, {1311,11}, 
    { 363, 9}, { 799,11}, {1823,11}, { 159,11}, {1183,11}, { 887,10}, { 235, 9}, { 491, 9}, 
    {   0, 0}, {  27, 9}, { 283, 9}, { 247,10}, { 759,10}, { 671,11}, {1695,11}, { 415,11}, 
    { 155, 9}, {1439,11}, { 927,11}, {1951,11}, { 503,10}, { 411, 9}, {  91, 9}, {  61, 8}, 
    {   0, 0}, { 189, 8}, { 347, 9}, { 219, 9}, {1015,10}, {  15,10}, {  95,11}, {1119,11}, 
    { 125, 8}, { 607,11}, {1631,11}, { 527,10}, { 475, 9}, {  59, 9}, { 253, 8}, {  57, 7}, 
    {   0, 0}, { 121, 7}, {   3, 8}, { 131, 8}, { 315, 9}, { 271,10}, { 351,11}, {1375,11}, 
    {  67, 8}, { 863,11}, { 783,10}, { 187, 9}, { 443, 9}, { 195, 8}, {   5, 7}, {  62, 6}, 
    {   0, 0}, {   1, 6}, {  69, 7}, {  35, 8}, { 123, 9}, { 379, 9}, { 143,10}, {1887,11}, 
    {  37, 7}, { 655,10}, { 251, 9}, { 507, 9}, { 163, 8}, { 101, 7}, {  33, 6}, {   0, 3}, 
    {   0, 0}, {   4, 3}, {  14, 5}, {  21, 7}, {  99, 8}, {   7, 9}, { 263, 9}, { 399,10}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {  85, 7}, { 911,10}, { 135, 9}, { 391, 9}, { 227, 8}, {  53, 7}, {  17, 6}, {   2, 3}, 
    {   0, 0}, {   6, 4}, {  49, 6}, { 117, 7}, {  19, 8}, {  71, 9}, { 327, 9}, {  79,10}, 
    { 147, 8}, { 223,11}, { 591,10}, { 199, 9}, { 455, 9}, {  83, 8}, {  13, 7}, {   9, 6}, 
    {   0, 0}, {  41, 6}, {  77, 7}, { 211, 8}, {  39, 9}, { 295, 9}, { 335,10}, {1247,11}, 
    {  51, 8}, { 735,11}, {1759,11}, { 847,10}, { 167, 9}, { 179, 8}, { 115, 8}, {  45, 7}, 
    {   0, 0}, { 109, 7}, { 243, 8}, { 423, 9}, { 103, 9}, { 207,10}, { 479,11}, {1503,11}, 
    { 359, 9}, { 991,11}, {2015,11}, { 719,10}, { 463,10}, { 231, 9}, { 487, 9}, {  11, 8}, 
    {   0, 0}, { 139, 8}, {  23, 9}, { 279, 9}, { 975,10}, {  63,11}, {1087,11}, { 575,11}, 
    { 151, 9}, {1599,11}, { 319,11}, {1343,11}, {  47,10}, { 559,10}, { 407, 9}, {  87, 9}, 
    {   0, 0}, { 343, 9}, { 215, 9}, { 303,10}, { 831,11}, {1855,11}, { 191,11}, {1215,11}, 
    { 471, 9}, { 703,11}, {1727,11}, { 447,11}, {1471,11}, { 959,11}, { 815,10}, {  55, 9}, 
    {   0, 0}, { 311, 9}, { 175,10}, {1983,11}, { 127,11}, {1151,11}, { 639,11}, {1663,11}, 
    { 687,10}, { 383,11}, {1407,11}, { 895,11}, {1919,11}, { 255,11}, {1279,11}, { 431,10}, 
    {   0, 0}, { 943,10}, { 767,11}, {1791,11}, { 511,11}, {1535,11}, {1023,11}, {2047,11}, 
  },
  #endif
  #ifdef HUFFMAN_DECODE
  {
    { 3,119}, { 6,105}, { 3,151}, { 8, 90}, { 3,121}, { 7,102}, { 4,153}, { 9,125}, 
    { 3,119}, { 6,167}, { 3,151}, { 8,199}, { 3,121}, { 7,166}, { 5,122}, {10, 77}, 
    { 3,119}, { 6,150}, { 3,151}, { 8,156}, { 3,121}, { 7,123}, { 4,153}, { 9,202}, 
    { 3,119}, { 7,  7}, { 3,151}, { 9, 57}, { 3,121}, { 8,  5}, { 6,  0}, {11, 37}, 
    { 3,119}, { 6,118}, { 3,151}, { 8,107}, { 3,121}, { 7,112}, { 4,153}, { 9,172}, 
    { 3,119}, { 6,169}, { 3,151}, { 9, 13}, { 3,121}, { 7,183}, { 5,122}, {10,212}, 
    { 3,119}, { 6,154}, { 3,151}, { 8,176}, { 3,121}, { 7,149}, { 4,153}, { 9,231}, 
    { 3,119}, { 7, 87}, { 3,151}, { 9, 85}, { 3,121}, { 8, 71}, { 6,103}, {11,205}, 
    { 3,119}, { 6,105}, { 3,151}, { 8, 96}, { 3,121}, { 7,106}, { 4,153}, { 9,157}, 
    { 3,119}, { 6,167}, { 3,151}, { 9,  2}, { 3,121}, { 7,170}, { 5,122}, {10,159}, 
    { 3,119}, { 6,150}, { 3,151}, { 8,165}, { 3,121}, { 7,144}, { 4,153}, { 9,215}, 
    { 3,119}, { 7,  9}, { 3,151}, { 9, 70}, { 3,121}, { 8, 10}, { 6,  0}, {11, 78}, 
    { 3,119}, { 6,118}, { 3,151}, { 8,124}, { 3,121}, { 7,117}, { 4,153}, { 9,188}, 
    { 3,119}, { 6,169}, { 3,151}, { 9, 41}, { 3,121}, { 7,185}, { 5,122}, {11, 17}, 
    { 3,119}, { 6,154}, { 3,151}, { 8,182}, { 3,121}, { 7,155}, { 4,153}, {10, 25}, 
    { 3,119}, { 7, 89}, { 3,151}, { 9,108}, { 3,121}, { 8, 80}, { 6,103}, {11,236}, 
    { 3,119}, { 6,105}, { 3,151}, { 8, 91}, { 3,121}, { 7,102}, { 4,153}, { 9,146}, 
    { 3,119}, { 6,167}, { 3,151}, { 8,201}, { 3,121}, { 7,166}, { 5,122}, {10,110}, 
    { 3,119}, { 6,150}, { 3,151}, { 8,160}, { 3,121}, { 7,123}, { 4,153}, { 9,208}, 
    { 3,119}, { 7,  7}, { 3,151}, { 9, 64}, { 3,121}, { 8,  6}, { 6,  0}, {11, 51}, 
    { 3,119}, { 6,118}, { 3,151}, { 8,116}, { 3,121}, { 7,112}, { 4,153}, { 9,180}, 
    { 3,119}, { 6,169}, { 3,151}, { 9, 32}, { 3,121}, { 7,183}, { 5,122}, {10,234}, 
    { 3,119}, { 6,154}, { 3,151}, { 8,181}, { 3,121}, { 7,149}, { 4,153}, {10,  1}, 
    { 3,119}, { 7, 87}, { 3,151}, { 9, 99}, { 3,121}, { 8, 73}, { 6,103}, {11,222}, 
    { 3,119}, { 6,105}, { 3,151}, { 8,101}, { 3,121}, { 7,106}, { 4,153}, { 9,163}, 
    { 3,119}, { 6,167}, { 3,151}, { 9,  4}, { 3,121}, { 7,170}, { 5,122}, {10,189}, 
    { 3,119}, { 6,150}, { 3,151}, { 8,171}, { 3,121}, { 7,144}, { 4,153}, { 9,218}, 
    { 3,119}, { 7,  9}, { 3,151}, { 9, 75}, { 3,121}, { 8, 11}, { 6,  0}, {11,161}, 
    { 3,119}, { 6,118}, { 3,151}, { 8,148}, { 3,121}, { 7,117}, { 4,153}, { 9,197}, 
    { 3,119}, { 6,169}, { 3,151}, { 9, 54}, { 3,121}, { 7,185}, { 5,122}, {11, 28}, 
    { 3,119}, { 6,154}, { 3,151}, { 8,186}, { 3,121}, { 7,155}, { 4,153}, {10, 59}, 
    { 3,119}, { 7, 89}, { 3,151}, { 9,114}, { 3,121}, { 8, 86}, { 6,103}, {11,245}, 
    { 3,119}, { 6,105}, { 3,151}, { 8, 90}, { 3,121}, { 7,102}, { 4,153}, { 9,126}, 
    { 3,119}, { 6,167}, { 3,151}, { 8,199}, { 3,121}, { 7,166}, { 5,122}, {10, 93}, 
    { 3,119}, { 6,150}, { 3,151}, { 8,156}, { 3,121}, { 7,123}, { 4,153}, { 9,203}, 
    { 3,119}, { 7,  7}, { 3,151}, { 9, 58}, { 3,121}, { 8,  5}, { 6,  0}, {11, 46}, 
    { 3,119}, { 6,118}, { 3,151}, { 8,107}, { 3,121}, { 7,112}, { 4,153}, { 9,173}, 
    { 3,119}, { 6,169}, { 3,151}, { 9, 14}, { 3,121}, { 7,183}, { 5,122}, {10,219}, 
    { 3,119}, { 6,154}, { 3,151}, { 8,176}, { 3,121}, { 7,149}, { 4,153}, { 9,233}, 
    { 3,119}, { 7, 87}, { 3,151}, { 9, 92}, { 3,121}, { 8, 71}, { 6,103}, {11,210}, 
    { 3,119}, { 6,105}, { 3,151}, { 8, 96}, { 3,121}, { 7,106}, { 4,153}, { 9,158}, 
    { 3,119}, { 6,167}, { 3,151}, { 9,  3}, { 3,121}, { 7,170}, { 5,122}, {10,174}, 
    { 3,119}, { 6,150}, { 3,151}, { 8,165}, { 3,121}, { 7,144}, { 4,153}, { 9,217}, 
    { 3,119}, { 7,  9}, { 3,151}, { 9, 74}, { 3,121}, { 8, 10}, { 6,  0}, {11, 94}, 
    { 3,119}, { 6,118}, { 3,151}, { 8,124}, { 3,121}, { 7,117}, { 4,153}, { 9,192}, 
    { 3,119}, { 6,169}, { 3,151}, { 9, 48}, { 3,121}, { 7,185}, { 5,122}, {11, 21}, 
    { 3,119}, { 6,154}, { 3,151}, { 8,182}, { 3,121}, { 7,155}, { 4,153}, {10, 42}, 
    { 3,119}, { 7, 89}, { 3,151}, { 9,109}, { 3,121}, { 8, 80}, { 6,103}, {11,241}, 
    { 3,119}, { 6,105}, { 3,151}, { 8, 91}, { 3,121}, { 7,102}, { 4,153}, { 9,147}, 
    { 3,119}, { 6,167}, { 3,151}, { 8,201}, { 3,121}, { 7,166}, { 5,122}, {10,127}, 
    { 3,119}, { 6,150}, { 3,151}, { 8,160}, { 3,121}, { 7,123}, { 4,153}, { 9,214}, 
    { 3,119}, { 7,  7}, { 3,151}, { 9, 69}, { 3,121}, { 8,  6}, { 6,  0}, {11, 63}, 
    { 3,119}, { 6,118}, { 3,151}, { 8,116}, { 3,121}, { 7,112}, { 4,153}, { 9,187}, 
    { 3,119}, { 6,169}, { 3,151}, { 9, 39}, { 3,121}, { 7,183}, { 5,122}, {10,247}, 
    { 3,119}, { 6,154}, { 3,151}, { 8,181}, { 3,121}, { 7,149}, { 4,153}, {10, 16}, 
    { 3,119}, { 7, 87}, { 3,151}, { 9,100}, { 3,121}, { 8, 73}, { 6,103}, {11,227}, 
    { 3,119}, { 6,105}, { 3,151}, { 8,101}, { 3,121}, { 7,106}, { 4,153}, { 9,164}, 
    { 3,119}, { 6,167}, { 3,151}, { 9, 12}, { 3,121}, { 7,170}, { 5,122}, {10,196}, 
    { 3,119}, { 6,150}, { 3,151}, { 8,171}, { 3,121}, { 7,144}, { 4,153}, { 9,224}, 
    { 3,119}, { 7,  9}, { 3,151}, { 9, 84}, { 3,121}, { 8, 11}, { 6,  0}, {11,190}, 
    { 3,119}, { 6,118}, { 3,151}, { 8,148}, { 3,121}, { 7,117}, { 4,153}, { 9,198}, 
    { 3,119}, { 6,169}, { 3,151}, { 9, 55}, { 3,121}, { 7,185}, { 5,122}, {11, 33}, 
    { 3,119}, { 6,154}, { 3,151}, { 8,186}, { 3,121}, { 7,155}, { 4,153}, {10, 68}, 
    { 3,119}, { 7, 89}, { 3,151}, { 9,115}, { 3,121}, { 8, 86}, { 6,103}, {11,252}, 
    { 3,119}, { 6,105}, { 3,151}, { 8, 90}, { 3,121}, { 7,102}, { 4,153}, { 9,125}, 
    { 3,119}, { 6,167}, { 3,151}, { 8,199}, { 3,121}, { 7,166}, { 5,122}, {10, 83}, 
    { 3,119}, { 6,150}, { 3,151}, { 8,156}, { 3,121}, { 7,123}, { 4,153}, { 9,202}, 
    { 3,119}, { 7,  7}, { 3,151}, { 9, 57}, { 3,121}, { 8,  5}, { 6,  0}, {11, 44}, 
    { 3,119}, { 6,118}, { 3,151}, { 8,107}, { 3,121}, { 7,112}, { 4,153}, { 9,172}, 
    { 3,119}, { 6,169}, { 3,151}, { 9, 13}, { 3,121}, { 7,183}, { 5,122}, {10,213}, 
    { 3,119}, { 6,154}, { 3,151}, { 8,176}, { 3,121}, { 7,149}, { 4,153}, { 9,231}, 
    { 3,119}, { 7, 87}, { 3,151}, { 9, 85}, { 3,121}, { 8, 71}, { 6,103}, {11,207}, 
    { 3,119}, { 6,105}, { 3,151}, { 8, 96}, { 3,121}, { 7,106}, { 4,153}, { 9,157}, 
    { 3,119}, { 6,167}, { 3,151}, { 9,  2}, { 3,121}, { 7,170}, { 5,122}, {10,162}, 
    { 3,119}, { 6,150}, { 3,151}, { 8,165}, { 3,121}, { 7,144}, { 4,153}, { 9,215}, 
    { 3,119}, { 7,  9}, { 3,151}, { 9, 70}, { 3,121}, { 8, 10}, { 6,  0}, {11, 81}, 
    { 3,119}, { 6,118}, { 3,151}, { 8,124}, { 3,121}, { 7,117}, { 4,153}, { 9,188}, 
    { 3,119}, { 6,169}, { 3,151}, { 9, 41}, { 3,121}, { 7,185}, { 5,122}, {11, 19}, 
    { 3,119}, { 6,154}, { 3,151}, { 8,182}, { 3,121}, { 7,155}, { 4,153}, {10, 38}, 
    { 3,119}, { 7, 89}, { 3,151}, { 9,108}, { 3,121}, { 8, 80}, { 6,103}, {11,238}, 
    { 3,119}, { 6,105}, { 3,151}, { 8, 91}, { 3,121}, { 7,102}, { 4,153}, { 9,146}, 
    { 3,119}, { 6,167}, { 3,151}, { 8,201}, { 3,121}, { 7,166}, { 5,122}, {10,113}, 
    { 3,119}, { 6,150}, { 3,151}, { 8,160}, { 3,121}, { 7,123}, { 4,153}, { 9,208}, 
    { 3,119}, { 7,  7}, { 3,151}, { 9, 64}, { 3,121}, { 8,  6}, { 6,  0}, {11, 61}, 
    { 3,119}, { 6,118}, { 3,151}, { 8,116}, { 3,121}, { 7,112}, { 4,153}, { 9,180}, 
    { 3,119}, { 6,169}, { 3,151}, { 9, 32}, { 3,121}, { 7,183}, { 5,122}, {10,240}, 
    { 3,119}, { 6,154}, { 3,151}, { 8,181}, { 3,121}, { 7,149}, { 4,153}, {10, 15}, 
    { 3,119}, { 7, 87}, { 3,151}, { 9, 99}, { 3,121}, { 8, 73}, { 6,103}, {11,225}, 
    { 3,119}, { 6,105}, { 3,151}, { 8,101}, { 3,121}, { 7,106}, { 4,153}, { 9,163}, 
    { 3,119}, { 6,167}, { 3,151}, { 9,  4}, { 3,121}, { 7,170}, { 5,122}, {10,195}, 
    { 3,119}, { 6,150}, { 3,151}, { 8,171}, { 3,121}, { 7,144}, { 4,153}, { 9,218}, 
    { 3,119}, { 7,  9}, { 3,151}, { 9, 75}, { 3,121}, { 8, 11}, { 6,  0}, {11,177}, 
    { 3,119}, { 6,118}, { 3,151}, { 8,148}, { 3,121}, { 7,117}, { 4,153}, { 9,197}, 
    { 3,119}, { 6,169}, { 3,151}, { 9, 54}, { 3,121}, { 7,185}, { 5,122}, {11, 30}, 
    { 3,119}, { 6,154}, { 3,151}, { 8,186}, { 3,121}, { 7,155}, { 4,153}, {10, 60}, 
    { 3,119}, { 7, 89}, { 3,151}, { 9,114}, { 3,121}, { 8, 86}, { 6,103}, {11,250}, 
    { 3,119}, { 6,105}, { 3,151}, { 8, 90}, { 3,121}, { 7,102}, { 4,153}, { 9,126}, 
    { 3,119}, { 6,167}, { 3,151}, { 8,199}, { 3,121}, { 7,166}, { 5,122}, {10, 98}, 
    { 3,119}, { 6,150}, { 3,151}, { 8,156}, { 3,121}, { 7,123}, { 4,153}, { 9,203}, 
    { 3,119}, { 7,  7}, { 3,151}, { 9, 58}, { 3,121}, { 8,  5}, { 6,  0}, {11, 49}, 
    { 3,119}, { 6,118}, { 3,151}, { 8,107}, { 3,121}, { 7,112}, { 4,153}, { 9,173}, 
    { 3,119}, { 6,169}, { 3,151}, { 9, 14}, { 3,121}, { 7,183}, { 5,122}, {10,230}, 
    { 3,119}, { 6,154}, { 3,151}, { 8,176}, { 3,121}, { 7,149}, { 4,153}, { 9,233}, 
    { 3,119}, { 7, 87}, { 3,151}, { 9, 92}, { 3,121}, { 8, 71}, { 6,103}, {11,220}, 
    { 3,119}, { 6,105}, { 3,151}, { 8, 96}, { 3,121}, { 7,106}, { 4,153}, { 9,158}, 
    { 3,119}, { 6,167}, { 3,151}, { 9,  3}, { 3,121}, { 7,170}, { 5,122}, {10,179}, 
    { 3,119}, { 6,150}, { 3,151}, { 8,165}, { 3,121}, { 7,144}, { 4,153}, { 9,217}, 
    { 3,119}, { 7,  9}, { 3,151}, { 9, 74}, { 3,121}, { 8, 10}, { 6,  0}, {11, 97}, 
    { 3,119}, { 6,118}, { 3,151}, { 8,124}, { 3,121}, { 7,117}, { 4,153}, { 9,192}, 
    { 3,119}, { 6,169}, { 3,151}, { 9, 48}, { 3,121}, { 7,185}, { 5,122}, {11, 26}, 
    { 3,119}, { 6,154}, { 3,151}, { 8,182}, { 3,121}, { 7,155}, { 4,153}, {10, 53}, 
    { 3,119}, { 7, 89}, { 3,151}, { 9,109}, { 3,121}, { 8, 80}, { 6,103}, {11,243}, 
    { 3,119}, { 6,105}, { 3,151}, { 8, 91}, { 3,121}, { 7,102}, { 4,153}, { 9,147}, 
    { 3,119}, { 6,167}, { 3,151}, { 8,201}, { 3,121}, { 7,166}, { 5,122}, {10,145}, 
    { 3,119}, { 6,150}, { 3,151}, { 8,160}, { 3,121}, { 7,123}, { 4,153}, { 9,214}, 
    { 3,119}, { 7,  7}, { 3,151}, { 9, 69}, { 3,121}, { 8,  6}, { 6,  0}, {11, 66}, 
    { 3,119}, { 6,118}, { 3,151}, { 8,116}, { 3,121}, { 7,112}, { 4,153}, { 9,187}, 
    { 3,119}, { 6,169}, { 3,151}, { 9, 39}, { 3,121}, { 7,183}, { 5,122}, {10,249}, 
    { 3,119}, { 6,154}, { 3,151}, { 8,181}, { 3,121}, { 7,149}, { 4,153}, {10, 23}, 
    { 3,119}, { 7, 87}, { 3,151}, { 9,100}, { 3,121}, { 8, 73}, { 6,103}, {11,229}, 
    { 3,119}, { 6,105}, { 3,151}, { 8,101}, { 3,121}, { 7,106}, { 4,153}, { 9,164}, 
    { 3,119}, { 6,167}, { 3,151}, { 9, 12}, { 3,121}, { 7,170}, { 5,122}, {10,204}, 
    { 3,119}, { 6,150}, { 3,151}, { 8,171}, { 3,121}, { 7,144}, { 4,153}, { 9,224}, 
    { 3,119}, { 7,  9}, { 3,151}, { 9, 84}, { 3,121}, { 8, 11}, { 6,  0}, {11,193}, 
    { 3,119}, { 6,118}, { 3,151}, { 8,148}, { 3,121}, { 7,117}, { 4,153}, { 9,198}, 
    { 3,119}, { 6,169}, { 3,151}, { 9, 55}, { 3,121}, { 7,185}, { 5,122}, {11, 35}, 
    { 3,119}, { 6,154}, { 3,151}, { 8,186}, { 3,121}, { 7,155}, { 4,153}, {10, 76}, 
    { 3,119}, { 7, 89}, { 3,151}, { 9,115}, { 3,121}, { 8, 86}, { 6,103}, {11,254}, 
    { 3,119}, { 6,105}, { 3,151}, { 8, 90}, { 3,121}, { 7,102}, { 4,153}, { 9,125}, 
    { 3,119}, { 6,167}, { 3,151}, { 8,199}, { 3,121}, { 7,166}, { 5,122}, {10, 77}, 
    { 3,119}, { 6,150}, { 3,151}, { 8,156}, { 3,121}, { 7,123}, { 4,153}, { 9,202}, 
    { 3,119}, { 7,  7}, { 3,151}, { 9, 57}, { 3,121}, { 8,  5}, { 6,  0}, {11, 43}, 
    { 3,119}, { 6,118}, { 3,151}, { 8,107}, { 3,121}, { 7,112}, { 4,153}, { 9,172}, 
    { 3,119}, { 6,169}, { 3,151}, { 9, 13}, { 3,121}, { 7,183}, { 5,122}, {10,212}, 
    { 3,119}, { 6,154}, { 3,151}, { 8,176}, { 3,121}, { 7,149}, { 4,153}, { 9,231}, 
    { 3,119}, { 7, 87}, { 3,151}, { 9, 85}, { 3,121}, { 8, 71}, { 6,103}, {11,206}, 
    { 3,119}, { 6,105}, { 3,151}, { 8, 96}, { 3,121}, { 7,106}, { 4,153}, { 9,157}, 
    { 3,119}, { 6,167}, { 3,151}, { 9,  2}, { 3,121}, { 7,170}, { 5,122}, {10,159}, 
    { 3,119}, { 6,150}, { 3,151}, { 8,165}, { 3,121}, { 7,144}, { 4,153}, { 9,215}, 
    { 3,119}, { 7,  9}, { 3,151}, { 9, 70}, { 3,121}, { 8, 10}, { 6,  0}, {11, 79}, 
    { 3,119}, { 6,118}, { 3,151}, { 8,124}, { 3,121}, { 7,117}, { 4,153}, { 9,188}, 
    { 3,119}, { 6,169}, { 3,151}, { 9, 41}, { 3,121}, { 7,185}, { 5,122}, {11, 18}, 
    { 3,119}, { 6,154}, { 3,151}, { 8,182}, { 3,121}, { 7,155}, { 4,153}, {10, 25}, 
    { 3,119}, { 7, 89}, { 3,151}, { 9,108}, { 3,121}, { 8, 80}, { 6,103}, {11,237}, 
    { 3,119}, { 6,105}, { 3,151}, { 8, 91}, { 3,121}, { 7,102}, { 4,153}, { 9,146}, 
    { 3,119}, { 6,167}, { 3,151}, { 8,201}, { 3,121}, { 7,166}, { 5,122}, {10,110}, 
    { 3,119}, { 6,150}, { 3,151}, { 8,160}, { 3,121}, { 7,123}, { 4,153}, { 9,208}, 
    { 3,119}, { 7,  7}, { 3,151}, { 9, 64}, { 3,121}, { 8,  6}, { 6,  0}, {11, 52}, 
    { 3,119}, { 6,118}, { 3,151}, { 8,116}, { 3,121}, { 7,112}, { 4,153}, { 9,180}, 
    { 3,119}, { 6,169}, { 3,151}, { 9, 32}, { 3,121}, { 7,183}, { 5,122}, {10,234}, 
    { 3,119}, { 6,154}, { 3,151}, { 8,181}, { 3,121}, { 7,149}, { 4,153}, {10,  1}, 
    { 3,119}, { 7, 87}, { 3,151}, { 9, 99}, { 3,121}, { 8, 73}, { 6,103}, {11,223}, 
    { 3,119}, { 6,105}, { 3,151}, { 8,101}, { 3,121}, { 7,106}, { 4,153}, { 9,163}, 
    { 3,119}, { 6,167}, { 3,151}, { 9,  4}, { 3,121}, { 7,170}, { 5,122}, {10,189}, 
    { 3,119}, { 6,150}, { 3,151}, { 8,171}, { 3,121}, { 7,144}, { 4,153}, { 9,218}, 
    { 3,119}, { 7,  9}, { 3,151}, { 9, 75}, { 3,121}, { 8, 11}, { 6,  0}, {11,175}, 
    { 3,119}, { 6,118}, { 3,151}, { 8,148}, { 3,121}, { 7,117}, { 4,153}, { 9,197}, 
    { 3,119}, { 6,169}, { 3,151}, { 9, 54}, { 3,121}, { 7,185}, { 5,122}, {11, 29}, 
    { 3,119}, { 6,154}, { 3,151}, { 8,186}, { 3,121}, { 7,155}, { 4,153}, {10, 59}, 
    { 3,119}, { 7, 89}, { 3,151}, { 9,114}, { 3,121}, { 8, 86}, { 6,103}, {11,246}, 
    { 3,119}, { 6,105}, { 3,151}, { 8, 90}, { 3,121}, { 7,102}, { 4,153}, { 9,126}, 
    { 3,119}, { 6,167}, { 3,151}, { 8,199}, { 3,121}, { 7,166}, { 5,122}, {10, 93}, 
    { 3,119}, { 6,150}, { 3,151}, { 8,156}, { 3,121}, { 7,123}, { 4,153}, { 9,203}, 
    { 3,119}, { 7,  7}, { 3,151}, { 9, 58}, { 3,121}, { 8,  5}, { 6,  0}, {11, 47}, 
    { 3,119}, { 6,118}, { 3,151}, { 8,107}, { 3,121}, { 7,112}, { 4,153}, { 9,173}, 
    { 3,119}, { 6,169}, { 3,151}, { 9, 14}, { 3,121}, { 7,183}, { 5,122}, {10,219}, 
    { 3,119}, { 6,154}, { 3,151}, { 8,176}, { 3,121}, { 7,149}, { 4,153}, { 9,233}, 
    { 3,119}, { 7, 87}, { 3,151}, { 9, 92}, { 3,121}, { 8, 71}, { 6,103}, {11,211}, 
    { 3,119}, { 6,105}, { 3,151}, { 8, 96}, { 3,121}, { 7,106}, { 4,153}, { 9,158}, 
    { 3,119}, { 6,167}, { 3,151}, { 9,  3}, { 3,121}, { 7,170}, { 5,122}, {10,174}, 
    { 3,119}, { 6,150}, { 3,151}, { 8,165}, { 3,121}, { 7,144}, { 4,153}, { 9,217}, 
    { 3,119}, { 7,  9}, { 3,151}, { 9, 74}, { 3,121}, { 8, 10}, { 6,  0}, {11, 95}, 
    { 3,119}, { 6,118}, { 3,151}, { 8,124}, { 3,121}, { 7,117}, { 4,153}, { 9,192}, 
    { 3,119}, { 6,169}, { 3,151}, { 9, 48}, { 3,121}, { 7,185}, { 5,122}, {11, 22}, 
    { 3,119}, { 6,154}, { 3,151}, { 8,182}, { 3,121}, { 7,155}, { 4,153}, {10, 42}, 
    { 3,119}, { 7, 89}, { 3,151}, { 9,109}, { 3,121}, { 8, 80}, { 6,103}, {11,242}, 
    { 3,119}, { 6,105}, { 3,151}, { 8, 91}, { 3,121}, { 7,102}, { 4,153}, { 9,147}, 
    { 3,119}, { 6,167}, { 3,151}, { 8,201}, { 3,121}, { 7,166}, { 5,122}, {10,127}, 
    { 3,119}, { 6,150}, { 3,151}, { 8,160}, { 3,121}, { 7,123}, { 4,153}, { 9,214}, 
    { 3,119}, { 7,  7}, { 3,151}, { 9, 69}, { 3,121}, { 8,  6}, { 6,  0}, {11, 65}, 
    { 3,119}, { 6,118}, { 3,151}, { 8,116}, { 3,121}, { 7,112}, { 4,153}, { 9,187}, 
    { 3,119}, { 6,169}, { 3,151}, { 9, 39}, { 3,121}, { 7,183}, { 5,122}, {10,247}, 
    { 3,119}, { 6,154}, { 3,151}, { 8,181}, { 3,121}, { 7,149}, { 4,153}, {10, 16}, 
    { 3,119}, { 7, 87}, { 3,151}, { 9,100}, { 3,121}, { 8, 73}, { 6,103}, {11,228}, 
    { 3,119}, { 6,105}, { 3,151}, { 8,101}, { 3,121}, { 7,106}, { 4,153}, { 9,164}, 
    { 3,119}, { 6,167}, { 3,151}, { 9, 12}, { 3,121}, { 7,170}, { 5,122}, {10,196}, 
    { 3,119}, { 6,150}, { 3,151}, { 8,171}, { 3,121}, { 7,144}, { 4,153}, { 9,224}, 
    { 3,119}, { 7,  9}, { 3,151}, { 9, 84}, { 3,121}, { 8, 11}, { 6,  0}, {11,191}, 
    { 3,119}, { 6,118}, { 3,151}, { 8,148}, { 3,121}, { 7,117}, { 4,153}, { 9,198}, 
    { 3,119}, { 6,169}, { 3,151}, { 9, 55}, { 3,121}, { 7,185}, { 5,122}, {11, 34}, 
    { 3,119}, { 6,154}, { 3,151}, { 8,186}, { 3,121}, { 7,155}, { 4,153}, {10, 68}, 
    { 3,119}, { 7, 89}, { 3,151}, { 9,115}, { 3,121}, { 8, 86}, { 6,103}, {11,253}, 
    { 3,119}, { 6,105}, { 3,151}, { 8, 90}, { 3,121}, { 7,102}, { 4,153}, { 9,125}, 
    { 3,119}, { 6,167}, { 3,151}, { 8,199}, { 3,121}, { 7,166}, { 5,122}, {10, 83}, 
    { 3,119}, { 6,150}, { 3,151}, { 8,156}, { 3,121}, { 7,123}, { 4,153}, { 9,202}, 
    { 3,119}, { 7,  7}, { 3,151}, { 9, 57}, { 3,121}, { 8,  5}, { 6,  0}, {11, 45}, 
    { 3,119}, { 6,118}, { 3,151}, { 8,107}, { 3,121}, { 7,112}, { 4,153}, { 9,172}, 
    { 3,119}, { 6,169}, { 3,151}, { 9, 13}, { 3,121}, { 7,183}, { 5,122}, {10,213}, 
    { 3,119}, { 6,154}, { 3,151}, { 8,176}, { 3,121}, { 7,149}, { 4,153}, { 9,231}, 
    { 3,119}, { 7, 87}, { 3,151}, { 9, 85}, { 3,121}, { 8, 71}, { 6,103}, {11,209}, 
    { 3,119}, { 6,105}, { 3,151}, { 8, 96}, { 3,121}, { 7,106}, { 4,153}, { 9,157}, 
    { 3,119}, { 6,167}, { 3,151}, { 9,  2}, { 3,121}, { 7,170}, { 5,122}, {10,162}, 
    { 3,119}, { 6,150}, { 3,151}, { 8,165}, { 3,121}, { 7,144}, { 4,153}, { 9,215}, 
    { 3,119}, { 7,  9}, { 3,151}, { 9, 70}, { 3,121}, { 8, 10}, { 6,  0}, {11, 82}, 
    { 3,119}, { 6,118}, { 3,151}, { 8,124}, { 3,121}, { 7,117}, { 4,153}, { 9,188}, 
    { 3,119}, { 6,169}, { 3,151}, { 9, 41}, { 3,121}, { 7,185}, { 5,122}, {11, 20}, 
    { 3,119}, { 6,154}, { 3,151}, { 8,182}, { 3,121}, { 7,155}, { 4,153}, {10, 38}, 
    { 3,119}, { 7, 89}, { 3,151}, { 9,108}, { 3,121}, { 8, 80}, { 6,103}, {11,239}, 
    { 3,119}, { 6,105}, { 3,151}, { 8, 91}, { 3,121}, { 7,102}, { 4,153}, { 9,146}, 
    { 3,119}, { 6,167}, { 3,151}, { 8,201}, { 3,121}, { 7,166}, { 5,122}, {10,113}, 
    { 3,119}, { 6,150}, { 3,151}, { 8,160}, { 3,121}, { 7,123}, { 4,153}, { 9,208}, 
    { 3,119}, { 7,  7}, { 3,151}, { 9, 64}, { 3,121}, { 8,  6}, { 6,  0}, {11, 62}, 
    { 3,119}, { 6,118}, { 3,151}, { 8,116}, { 3,121}, { 7,112}, { 4,153}, { 9,180}, 
    { 3,119}, { 6,169}, { 3,151}, { 9, 32}, { 3,121}, { 7,183}, { 5,122}, {10,240}, 
    { 3,119}, { 6,154}, { 3,151}, { 8,181}, { 3,121}, { 7,149}, { 4,153}, {10, 15}, 
    { 3,119}, { 7, 87}, { 3,151}, { 9, 99}, { 3,121}, { 8, 73}, { 6,103}, {11,226}, 
    { 3,119}, { 6,105}, { 3,151}, { 8,101}, { 3,121}, { 7,106}, { 4,153}, { 9,163}, 
    { 3,119}, { 6,167}, { 3,151}, { 9,  4}, { 3,121}, { 7,170}, { 5,122}, {10,195}, 
    { 3,119}, { 6,150}, { 3,151}, { 8,171}, { 3,121}, { 7,144}, { 4,153}, { 9,218}, 
    { 3,119}, { 7,  9}, { 3,151}, { 9, 75}, { 3,121}, { 8, 11}, { 6,  0}, {11,178}, 
    { 3,119}, { 6,118}, { 3,151}, { 8,148}, { 3,121}, { 7,117}, { 4,153}, { 9,197}, 
    { 3,119}, { 6,169}, { 3,151}, { 9, 54}, { 3,121}, { 7,185}, { 5,122}, {11, 31}, 
    { 3,119}, { 6,154}, { 3,151}, { 8,186}, { 3,121}, { 7,155}, { 4,153}, {10, 60}, 
    { 3,119}, { 7, 89}, { 3,151}, { 9,114}, { 3,121}, { 8, 86}, { 6,103}, {11,251}, 
    { 3,119}, { 6,105}, { 3,151}, { 8, 90}, { 3,121}, { 7,102}, { 4,153}, { 9,126}, 
    { 3,119}, { 6,167}, { 3,151}, { 8,199}, { 3,121}, { 7,166}, { 5,122}, {10, 98}, 
    { 3,119}, { 6,150}, { 3,151}, { 8,156}, { 3,121}, { 7,123}, { 4,153}, { 9,203}, 
    { 3,119}, { 7,  7}, { 3,151}, { 9, 58}, { 3,121}, { 8,  5}, { 6,  0}, {11, 50}, 
    { 3,119}, { 6,118}, { 3,151}, { 8,107}, { 3,121}, { 7,112}, { 4,153}, { 9,173}, 
    { 3,119}, { 6,169}, { 3,151}, { 9, 14}, { 3,121}, { 7,183}, { 5,122}, {10,230}, 
    { 3,119}, { 6,154}, { 3,151}, { 8,176}, { 3,121}, { 7,149}, { 4,153}, { 9,233}, 
    { 3,119}, { 7, 87}, { 3,151}, { 9, 92}, { 3,121}, { 8, 71}, { 6,103}, {11,221}, 
    { 3,119}, { 6,105}, { 3,151}, { 8, 96}, { 3,121}, { 7,106}, { 4,153}, { 9,158}, 
    { 3,119}, { 6,167}, { 3,151}, { 9,  3}, { 3,121}, { 7,170}, { 5,122}, {10,179}, 
    { 3,119}, { 6,150}, { 3,151}, { 8,165}, { 3,121}, { 7,144}, { 4,153}, { 9,217}, 
    { 3,119}, { 7,  9}, { 3,151}, { 9, 74}, { 3,121}, { 8, 10}, { 6,  0}, {11,111}, 
    { 3,119}, { 6,118}, { 3,151}, { 8,124}, { 3,121}, { 7,117}, { 4,153}, { 9,192}, 
    { 3,119}, { 6,169}, { 3,151}, { 9, 48}, { 3,121}, { 7,185}, { 5,122}, {11, 27}, 
    { 3,119}, { 6,154}, { 3,151}, { 8,182}, { 3,121}, { 7,155}, { 4,153}, {10, 53}, 
    { 3,119}, { 7, 89}, { 3,151}, { 9,109}, { 3,121}, { 8, 80}, { 6,103}, {11,244}, 
    { 3,119}, { 6,105}, { 3,151}, { 8, 91}, { 3,121}, { 7,102}, { 4,153}, { 9,147}, 
    { 3,119}, { 6,167}, { 3,151}, { 8,201}, { 3,121}, { 7,166}, { 5,122}, {10,145}, 
    { 3,119}, { 6,150}, { 3,151}, { 8,160}, { 3,121}, { 7,123}, { 4,153}, { 9,214}, 
    { 3,119}, { 7,  7}, { 3,151}, { 9, 69}, { 3,121}, { 8,  6}, { 6,  0}, {11, 67}, 
    { 3,119}, { 6,118}, { 3,151}, { 8,116}, { 3,121}, { 7,112}, { 4,153}, { 9,187}, 
    { 3,119}, { 6,169}, { 3,151}, { 9, 39}, { 3,121}, { 7,183}, { 5,122}, {10,249}, 
    { 3,119}, { 6,154}, { 3,151}, { 8,181}, { 3,121}, { 7,149}, { 4,153}, {10, 23}, 
    { 3,119}, { 7, 87}, { 3,151}, { 9,100}, { 3,121}, { 8, 73}, { 6,103}, {11,235}, 
    { 3,119}, { 6,105}, { 3,151}, { 8,101}, { 3,121}, { 7,106}, { 4,153}, { 9,164}, 
    { 3,119}, { 6,167}, { 3,151}, { 9, 12}, { 3,121}, { 7,170}, { 5,122}, {10,204}, 
    { 3,119}, { 6,150}, { 3,151}, { 8,171}, { 3,121}, { 7,144}, { 4,153}, { 9,224}, 
    { 3,119}, { 7,  9}, { 3,151}, { 9, 84}, { 3,121}, { 8, 11}, { 6,  0}, {11,194}, 
    { 3,119}, { 6,118}, { 3,151}, { 8,148}, { 3,121}, { 7,117}, { 4,153}, { 9,198}, 
    { 3,119}, { 6,169}, { 3,151}, { 9, 55}, { 3,121}, { 7,185}, { 5,122}, {11, 36}, 
    { 3,119}, { 6,154}, { 3,151}, { 8,186}, { 3,121}, { 7,155}, { 4,153}, {10, 76}, 
    { 3,119}, { 7, 89}, { 3,151}, { 9,115}, { 3,121}, { 8, 86}, { 6,103}, {11,255}, 
  }
  #endif
};

static radaudio_huffman rada_nonzero_coefficient_pair_huff2 =
{
  #ifdef HUFFMAN_ENCODE
  {
    {  35, 8}, { 207,11}, {1231,11}, { 231,10}, { 743,10}, { 155, 9}, { 411, 9}, { 163, 8}, 
    {   0, 0}, {  99, 8}, {  91, 9}, { 347, 9}, { 487,10}, { 999,10}, { 719,11}, {1743,11}, 
    { 463,11}, {1487,11}, { 975,11}, {1999,11}, {  47,11}, {1071,11}, { 559,11}, {  23,10}, 
    {   0, 0}, { 535,10}, {1583,11}, { 303,11}, {1327,11}, { 815,11}, {1839,11}, { 175,11}, 
    {1199,11}, { 687,11}, {1711,11}, { 431,11}, {1455,11}, { 943,11}, {1967,11}, { 279,10}, 
    {   0, 0}, { 791,10}, { 151,10}, { 111,11}, {1135,11}, { 623,11}, {1647,11}, { 367,11}, 
    { 663,10}, {1391,11}, { 879,11}, {1903,11}, { 239,11}, {1263,11}, { 407,10}, { 219, 9}, 
    {   0, 0}, { 475, 9}, { 919,10}, {  87,10}, { 751,11}, {1775,11}, { 495,11}, {1519,11}, 
    { 599,10}, {1007,11}, {2031,11}, {  31,11}, {1055,11}, { 343,10}, {  59, 9}, { 227, 8}, 
    {   0, 0}, {  19, 8}, { 315, 9}, { 855,10}, { 215,10}, { 543,11}, {1567,11}, { 287,11}, 
    { 187, 9}, {1311,11}, { 799,11}, {1823,11}, { 727,10}, { 443, 9}, { 147, 8}, {  53, 7}, 
    {   0, 0}, { 117, 7}, {  83, 8}, { 123, 9}, { 471,10}, { 983,10}, { 159,11}, {1183,11}, 
    { 379, 9}, { 671,11}, {  55,10}, { 567,10}, { 251, 9}, { 211, 8}, {  13, 7}, {  25, 6}, 
    {   0, 0}, {   1, 5}, {  77, 7}, {  51, 8}, { 507, 9}, { 311,10}, { 823,10}, {1695,11}, 
    { 179, 8}, { 183,10}, { 695,10}, {   7, 9}, { 115, 8}, {  45, 7}, {  57, 6}, {   0, 3}, 
    {   0, 0}, {   4, 3}, {  17, 5}, { 109, 7}, { 243, 8}, { 263, 9}, { 439,10}, { 951,10}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {  11, 8}, { 119,10}, { 631,10}, { 135, 9}, { 139, 8}, {  29, 7}, {   9, 5}, {   2, 3}, 
    {   0, 0}, {   6, 3}, {   5, 6}, {  93, 7}, {  75, 8}, { 391, 9}, { 375,10}, { 887,10}, 
    {  71, 9}, { 415,11}, { 247,10}, { 759,10}, { 327, 9}, { 203, 8}, {  61, 7}, {  37, 6}, 
    {   0, 0}, {  21, 6}, { 125, 7}, {  43, 8}, { 199, 9}, { 503,10}, {1015,10}, {1439,11}, 
    { 455, 9}, { 927,11}, {1951,11}, {  95,11}, {  15,10}, {  39, 9}, { 171, 8}, {   3, 7}, 
    {   0, 0}, {  67, 7}, { 107, 8}, { 295, 9}, { 527,10}, {1119,11}, { 607,11}, {1631,11}, 
    { 271,10}, { 351,11}, {1375,11}, { 863,11}, {1887,11}, { 783,10}, { 167, 9}, { 235, 8}, 
    {   0, 0}, {  27, 8}, { 423, 9}, { 143,10}, { 223,11}, {1247,11}, { 735,11}, {1759,11}, 
    { 655,10}, { 479,11}, {1503,11}, { 991,11}, {2015,11}, {  63,11}, { 399,10}, { 103, 9}, 
    {   0, 0}, { 359, 9}, { 911,10}, {1087,11}, { 575,11}, {1599,11}, { 319,11}, {1343,11}, 
    { 831,11}, {1855,11}, { 191,11}, {1215,11}, { 703,11}, {1727,11}, { 447,11}, {  79,10}, 
    {   0, 0}, { 591,10}, {1471,11}, { 959,11}, {1983,11}, { 127,11}, {1151,11}, { 639,11}, 
    {1663,11}, { 383,11}, {1407,11}, { 895,11}, {1919,11}, { 255,11}, {1279,11}, { 335,10}, 
    {   0, 0}, { 847,10}, { 767,11}, {1791,11}, { 511,11}, {1535,11}, {1023,11}, {2047,11}, 
  },
  #endif
  #ifdef HUFFMAN_DECODE
  {
    { 3,119}, { 5,105}, { 3,151}, { 7,183}, { 3,121}, { 6,154}, { 3,153}, { 9,115}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,144}, { 3,121}, { 7,102}, { 3,153}, {10,180}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 73}, { 3,121}, { 6,169}, { 3,153}, {10, 23}, 
    { 3,119}, { 6,103}, { 3,151}, { 8,201}, { 3,121}, { 7,149}, { 3,153}, {11, 67}, 
    { 3,119}, { 5,105}, { 3,151}, { 8,  0}, { 3,121}, { 6,167}, { 3,153}, { 9,181}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,171}, { 3,121}, { 7,117}, { 3,153}, {11, 20}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,107}, { 3,121}, { 7, 87}, { 3,153}, {10, 98}, 
    { 3,119}, { 6,118}, { 3,151}, { 9, 70}, { 3,121}, { 7,166}, { 3,153}, {11,213}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,185}, { 3,121}, { 6,154}, { 3,153}, { 9,160}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,156}, { 3,121}, { 7,106}, { 3,153}, {10,231}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 90}, { 3,121}, { 6,169}, { 3,153}, {10, 59}, 
    { 3,119}, { 6,103}, { 3,151}, { 9, 10}, { 3,121}, { 7,155}, { 3,153}, {11,179}, 
    { 3,119}, { 5,105}, { 3,151}, { 8,  9}, { 3,121}, { 6,167}, { 3,153}, { 9,215}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,186}, { 3,121}, { 7,123}, { 3,153}, {11, 43}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,116}, { 3,121}, { 7, 89}, { 3,153}, {10,145}, 
    { 3,119}, { 6,118}, { 3,151}, { 9, 91}, { 3,121}, { 7,170}, { 3,153}, {11,237}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,183}, { 3,121}, { 6,154}, { 3,153}, { 9,147}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,148}, { 3,121}, { 7,102}, { 3,153}, {10,203}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 86}, { 3,121}, { 6,169}, { 3,153}, {10, 42}, 
    { 3,119}, { 6,103}, { 3,151}, { 9,  5}, { 3,121}, { 7,149}, { 3,153}, {11, 94}, 
    { 3,119}, { 5,105}, { 3,151}, { 8,  7}, { 3,121}, { 6,167}, { 3,153}, { 9,198}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,182}, { 3,121}, { 7,117}, { 3,153}, {11, 31}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,112}, { 3,121}, { 7, 87}, { 3,153}, {10,113}, 
    { 3,119}, { 6,118}, { 3,151}, { 9, 80}, { 3,121}, { 7,166}, { 3,153}, {11,226}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,185}, { 3,121}, { 6,154}, { 3,153}, { 9,172}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,165}, { 3,121}, { 7,106}, { 3,153}, {11,  1}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,101}, { 3,121}, { 6,169}, { 3,153}, {10, 76}, 
    { 3,119}, { 6,103}, { 3,151}, { 9, 55}, { 3,121}, { 7,155}, { 3,153}, {11,204}, 
    { 3,119}, { 5,105}, { 3,151}, { 8, 71}, { 3,121}, { 6,167}, { 3,153}, {10,  3}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,199}, { 3,121}, { 7,123}, { 3,153}, {11, 52}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,124}, { 3,121}, { 7, 89}, { 3,153}, {10,162}, 
    { 3,119}, { 6,118}, { 3,151}, { 9,100}, { 3,121}, { 7,170}, { 3,153}, {11,245}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,183}, { 3,121}, { 6,154}, { 3,153}, { 9,125}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,144}, { 3,121}, { 7,102}, { 3,153}, {10,192}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 73}, { 3,121}, { 6,169}, { 3,153}, {10, 39}, 
    { 3,119}, { 6,103}, { 3,151}, { 8,201}, { 3,121}, { 7,149}, { 3,153}, {11, 79}, 
    { 3,119}, { 5,105}, { 3,151}, { 8,  0}, { 3,121}, { 6,167}, { 3,153}, { 9,187}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,171}, { 3,121}, { 7,117}, { 3,153}, {11, 27}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,107}, { 3,121}, { 7, 87}, { 3,153}, {10,109}, 
    { 3,119}, { 6,118}, { 3,151}, { 9, 74}, { 3,121}, { 7,166}, { 3,153}, {11,222}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,185}, { 3,121}, { 6,154}, { 3,153}, { 9,164}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,156}, { 3,121}, { 7,106}, { 3,153}, {10,247}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 90}, { 3,121}, { 6,169}, { 3,153}, {10, 69}, 
    { 3,119}, { 6,103}, { 3,151}, { 9, 11}, { 3,121}, { 7,155}, { 3,153}, {11,193}, 
    { 3,119}, { 5,105}, { 3,151}, { 8,  9}, { 3,121}, { 6,167}, { 3,153}, { 9,217}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,186}, { 3,121}, { 7,123}, { 3,153}, {11, 47}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,116}, { 3,121}, { 7, 89}, { 3,153}, {10,158}, 
    { 3,119}, { 6,118}, { 3,151}, { 9, 96}, { 3,121}, { 7,170}, { 3,153}, {11,241}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,183}, { 3,121}, { 6,154}, { 3,153}, { 9,157}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,148}, { 3,121}, { 7,102}, { 3,153}, {10,214}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 86}, { 3,121}, { 6,169}, { 3,153}, {10, 54}, 
    { 3,119}, { 6,103}, { 3,151}, { 9,  6}, { 3,121}, { 7,149}, { 3,153}, {11,161}, 
    { 3,119}, { 5,105}, { 3,151}, { 8,  7}, { 3,121}, { 6,167}, { 3,153}, { 9,202}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,182}, { 3,121}, { 7,117}, { 3,153}, {11, 35}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,112}, { 3,121}, { 7, 87}, { 3,153}, {10,126}, 
    { 3,119}, { 6,118}, { 3,151}, { 9, 85}, { 3,121}, { 7,166}, { 3,153}, {11,230}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,185}, { 3,121}, { 6,154}, { 3,153}, { 9,176}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,165}, { 3,121}, { 7,106}, { 3,153}, {11, 16}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,101}, { 3,121}, { 6,169}, { 3,153}, {10, 92}, 
    { 3,119}, { 6,103}, { 3,151}, { 9, 57}, { 3,121}, { 7,155}, { 3,153}, {11,209}, 
    { 3,119}, { 5,105}, { 3,151}, { 8, 71}, { 3,121}, { 6,167}, { 3,153}, {10, 12}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,199}, { 3,121}, { 7,123}, { 3,153}, {11, 62}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,124}, { 3,121}, { 7, 89}, { 3,153}, {10,173}, 
    { 3,119}, { 6,118}, { 3,151}, { 9,108}, { 3,121}, { 7,170}, { 3,153}, {11,252}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,183}, { 3,121}, { 6,154}, { 3,153}, { 9,115}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,144}, { 3,121}, { 7,102}, { 3,153}, {10,188}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 73}, { 3,121}, { 6,169}, { 3,153}, {10, 25}, 
    { 3,119}, { 6,103}, { 3,151}, { 8,201}, { 3,121}, { 7,149}, { 3,153}, {11, 77}, 
    { 3,119}, { 5,105}, { 3,151}, { 8,  0}, { 3,121}, { 6,167}, { 3,153}, { 9,181}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,171}, { 3,121}, { 7,117}, { 3,153}, {11, 22}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,107}, { 3,121}, { 7, 87}, { 3,153}, {10, 99}, 
    { 3,119}, { 6,118}, { 3,151}, { 9, 70}, { 3,121}, { 7,166}, { 3,153}, {11,220}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,185}, { 3,121}, { 6,154}, { 3,153}, { 9,160}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,156}, { 3,121}, { 7,106}, { 3,153}, {10,233}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 90}, { 3,121}, { 6,169}, { 3,153}, {10, 64}, 
    { 3,119}, { 6,103}, { 3,151}, { 9, 10}, { 3,121}, { 7,155}, { 3,153}, {11,190}, 
    { 3,119}, { 5,105}, { 3,151}, { 8,  9}, { 3,121}, { 6,167}, { 3,153}, { 9,215}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,186}, { 3,121}, { 7,123}, { 3,153}, {11, 45}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,116}, { 3,121}, { 7, 89}, { 3,153}, {10,146}, 
    { 3,119}, { 6,118}, { 3,151}, { 9, 91}, { 3,121}, { 7,170}, { 3,153}, {11,239}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,183}, { 3,121}, { 6,154}, { 3,153}, { 9,147}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,148}, { 3,121}, { 7,102}, { 3,153}, {10,208}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 86}, { 3,121}, { 6,169}, { 3,153}, {10, 48}, 
    { 3,119}, { 6,103}, { 3,151}, { 9,  5}, { 3,121}, { 7,149}, { 3,153}, {11, 97}, 
    { 3,119}, { 5,105}, { 3,151}, { 8,  7}, { 3,121}, { 6,167}, { 3,153}, { 9,198}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,182}, { 3,121}, { 7,117}, { 3,153}, {11, 33}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,112}, { 3,121}, { 7, 87}, { 3,153}, {10,114}, 
    { 3,119}, { 6,118}, { 3,151}, { 9, 80}, { 3,121}, { 7,166}, { 3,153}, {11,228}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,185}, { 3,121}, { 6,154}, { 3,153}, { 9,172}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,165}, { 3,121}, { 7,106}, { 3,153}, {11, 14}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,101}, { 3,121}, { 6,169}, { 3,153}, {10, 84}, 
    { 3,119}, { 6,103}, { 3,151}, { 9, 55}, { 3,121}, { 7,155}, { 3,153}, {11,206}, 
    { 3,119}, { 5,105}, { 3,151}, { 8, 71}, { 3,121}, { 6,167}, { 3,153}, {10,  4}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,199}, { 3,121}, { 7,123}, { 3,153}, {11, 60}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,124}, { 3,121}, { 7, 89}, { 3,153}, {10,163}, 
    { 3,119}, { 6,118}, { 3,151}, { 9,100}, { 3,121}, { 7,170}, { 3,153}, {11,250}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,183}, { 3,121}, { 6,154}, { 3,153}, { 9,125}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,144}, { 3,121}, { 7,102}, { 3,153}, {10,197}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 73}, { 3,121}, { 6,169}, { 3,153}, {10, 41}, 
    { 3,119}, { 6,103}, { 3,151}, { 8,201}, { 3,121}, { 7,149}, { 3,153}, {11, 82}, 
    { 3,119}, { 5,105}, { 3,151}, { 8,  0}, { 3,121}, { 6,167}, { 3,153}, { 9,187}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,171}, { 3,121}, { 7,117}, { 3,153}, {11, 29}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,107}, { 3,121}, { 7, 87}, { 3,153}, {10,110}, 
    { 3,119}, { 6,118}, { 3,151}, { 9, 74}, { 3,121}, { 7,166}, { 3,153}, {11,224}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,185}, { 3,121}, { 6,154}, { 3,153}, { 9,164}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,156}, { 3,121}, { 7,106}, { 3,153}, {10,249}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 90}, { 3,121}, { 6,169}, { 3,153}, {10, 75}, 
    { 3,119}, { 6,103}, { 3,151}, { 9, 11}, { 3,121}, { 7,155}, { 3,153}, {11,195}, 
    { 3,119}, { 5,105}, { 3,151}, { 8,  9}, { 3,121}, { 6,167}, { 3,153}, { 9,217}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,186}, { 3,121}, { 7,123}, { 3,153}, {11, 50}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,116}, { 3,121}, { 7, 89}, { 3,153}, {10,159}, 
    { 3,119}, { 6,118}, { 3,151}, { 9, 96}, { 3,121}, { 7,170}, { 3,153}, {11,243}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,183}, { 3,121}, { 6,154}, { 3,153}, { 9,157}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,148}, { 3,121}, { 7,102}, { 3,153}, {10,218}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 86}, { 3,121}, { 6,169}, { 3,153}, {10, 58}, 
    { 3,119}, { 6,103}, { 3,151}, { 9,  6}, { 3,121}, { 7,149}, { 3,153}, {11,177}, 
    { 3,119}, { 5,105}, { 3,151}, { 8,  7}, { 3,121}, { 6,167}, { 3,153}, { 9,202}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,182}, { 3,121}, { 7,117}, { 3,153}, {11, 37}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,112}, { 3,121}, { 7, 87}, { 3,153}, {10,127}, 
    { 3,119}, { 6,118}, { 3,151}, { 9, 85}, { 3,121}, { 7,166}, { 3,153}, {11,235}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,185}, { 3,121}, { 6,154}, { 3,153}, { 9,176}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,165}, { 3,121}, { 7,106}, { 3,153}, {11, 18}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,101}, { 3,121}, { 6,169}, { 3,153}, {10, 93}, 
    { 3,119}, { 6,103}, { 3,151}, { 9, 57}, { 3,121}, { 7,155}, { 3,153}, {11,211}, 
    { 3,119}, { 5,105}, { 3,151}, { 8, 71}, { 3,121}, { 6,167}, { 3,153}, {10, 13}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,199}, { 3,121}, { 7,123}, { 3,153}, {11, 65}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,124}, { 3,121}, { 7, 89}, { 3,153}, {10,174}, 
    { 3,119}, { 6,118}, { 3,151}, { 9,108}, { 3,121}, { 7,170}, { 3,153}, {11,254}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,183}, { 3,121}, { 6,154}, { 3,153}, { 9,115}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,144}, { 3,121}, { 7,102}, { 3,153}, {10,180}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 73}, { 3,121}, { 6,169}, { 3,153}, {10, 23}, 
    { 3,119}, { 6,103}, { 3,151}, { 8,201}, { 3,121}, { 7,149}, { 3,153}, {11, 68}, 
    { 3,119}, { 5,105}, { 3,151}, { 8,  0}, { 3,121}, { 6,167}, { 3,153}, { 9,181}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,171}, { 3,121}, { 7,117}, { 3,153}, {11, 21}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,107}, { 3,121}, { 7, 87}, { 3,153}, {10, 98}, 
    { 3,119}, { 6,118}, { 3,151}, { 9, 70}, { 3,121}, { 7,166}, { 3,153}, {11,219}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,185}, { 3,121}, { 6,154}, { 3,153}, { 9,160}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,156}, { 3,121}, { 7,106}, { 3,153}, {10,231}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 90}, { 3,121}, { 6,169}, { 3,153}, {10, 59}, 
    { 3,119}, { 6,103}, { 3,151}, { 9, 10}, { 3,121}, { 7,155}, { 3,153}, {11,189}, 
    { 3,119}, { 5,105}, { 3,151}, { 8,  9}, { 3,121}, { 6,167}, { 3,153}, { 9,215}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,186}, { 3,121}, { 7,123}, { 3,153}, {11, 44}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,116}, { 3,121}, { 7, 89}, { 3,153}, {10,145}, 
    { 3,119}, { 6,118}, { 3,151}, { 9, 91}, { 3,121}, { 7,170}, { 3,153}, {11,238}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,183}, { 3,121}, { 6,154}, { 3,153}, { 9,147}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,148}, { 3,121}, { 7,102}, { 3,153}, {10,203}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 86}, { 3,121}, { 6,169}, { 3,153}, {10, 42}, 
    { 3,119}, { 6,103}, { 3,151}, { 9,  5}, { 3,121}, { 7,149}, { 3,153}, {11, 95}, 
    { 3,119}, { 5,105}, { 3,151}, { 8,  7}, { 3,121}, { 6,167}, { 3,153}, { 9,198}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,182}, { 3,121}, { 7,117}, { 3,153}, {11, 32}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,112}, { 3,121}, { 7, 87}, { 3,153}, {10,113}, 
    { 3,119}, { 6,118}, { 3,151}, { 9, 80}, { 3,121}, { 7,166}, { 3,153}, {11,227}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,185}, { 3,121}, { 6,154}, { 3,153}, { 9,172}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,165}, { 3,121}, { 7,106}, { 3,153}, {11,  2}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,101}, { 3,121}, { 6,169}, { 3,153}, {10, 76}, 
    { 3,119}, { 6,103}, { 3,151}, { 9, 55}, { 3,121}, { 7,155}, { 3,153}, {11,205}, 
    { 3,119}, { 5,105}, { 3,151}, { 8, 71}, { 3,121}, { 6,167}, { 3,153}, {10,  3}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,199}, { 3,121}, { 7,123}, { 3,153}, {11, 53}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,124}, { 3,121}, { 7, 89}, { 3,153}, {10,162}, 
    { 3,119}, { 6,118}, { 3,151}, { 9,100}, { 3,121}, { 7,170}, { 3,153}, {11,246}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,183}, { 3,121}, { 6,154}, { 3,153}, { 9,125}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,144}, { 3,121}, { 7,102}, { 3,153}, {10,192}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 73}, { 3,121}, { 6,169}, { 3,153}, {10, 39}, 
    { 3,119}, { 6,103}, { 3,151}, { 8,201}, { 3,121}, { 7,149}, { 3,153}, {11, 81}, 
    { 3,119}, { 5,105}, { 3,151}, { 8,  0}, { 3,121}, { 6,167}, { 3,153}, { 9,187}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,171}, { 3,121}, { 7,117}, { 3,153}, {11, 28}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,107}, { 3,121}, { 7, 87}, { 3,153}, {10,109}, 
    { 3,119}, { 6,118}, { 3,151}, { 9, 74}, { 3,121}, { 7,166}, { 3,153}, {11,223}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,185}, { 3,121}, { 6,154}, { 3,153}, { 9,164}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,156}, { 3,121}, { 7,106}, { 3,153}, {10,247}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 90}, { 3,121}, { 6,169}, { 3,153}, {10, 69}, 
    { 3,119}, { 6,103}, { 3,151}, { 9, 11}, { 3,121}, { 7,155}, { 3,153}, {11,194}, 
    { 3,119}, { 5,105}, { 3,151}, { 8,  9}, { 3,121}, { 6,167}, { 3,153}, { 9,217}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,186}, { 3,121}, { 7,123}, { 3,153}, {11, 49}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,116}, { 3,121}, { 7, 89}, { 3,153}, {10,158}, 
    { 3,119}, { 6,118}, { 3,151}, { 9, 96}, { 3,121}, { 7,170}, { 3,153}, {11,242}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,183}, { 3,121}, { 6,154}, { 3,153}, { 9,157}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,148}, { 3,121}, { 7,102}, { 3,153}, {10,214}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 86}, { 3,121}, { 6,169}, { 3,153}, {10, 54}, 
    { 3,119}, { 6,103}, { 3,151}, { 9,  6}, { 3,121}, { 7,149}, { 3,153}, {11,175}, 
    { 3,119}, { 5,105}, { 3,151}, { 8,  7}, { 3,121}, { 6,167}, { 3,153}, { 9,202}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,182}, { 3,121}, { 7,117}, { 3,153}, {11, 36}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,112}, { 3,121}, { 7, 87}, { 3,153}, {10,126}, 
    { 3,119}, { 6,118}, { 3,151}, { 9, 85}, { 3,121}, { 7,166}, { 3,153}, {11,234}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,185}, { 3,121}, { 6,154}, { 3,153}, { 9,176}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,165}, { 3,121}, { 7,106}, { 3,153}, {11, 17}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,101}, { 3,121}, { 6,169}, { 3,153}, {10, 92}, 
    { 3,119}, { 6,103}, { 3,151}, { 9, 57}, { 3,121}, { 7,155}, { 3,153}, {11,210}, 
    { 3,119}, { 5,105}, { 3,151}, { 8, 71}, { 3,121}, { 6,167}, { 3,153}, {10, 12}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,199}, { 3,121}, { 7,123}, { 3,153}, {11, 63}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,124}, { 3,121}, { 7, 89}, { 3,153}, {10,173}, 
    { 3,119}, { 6,118}, { 3,151}, { 9,108}, { 3,121}, { 7,170}, { 3,153}, {11,253}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,183}, { 3,121}, { 6,154}, { 3,153}, { 9,115}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,144}, { 3,121}, { 7,102}, { 3,153}, {10,188}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 73}, { 3,121}, { 6,169}, { 3,153}, {10, 25}, 
    { 3,119}, { 6,103}, { 3,151}, { 8,201}, { 3,121}, { 7,149}, { 3,153}, {11, 78}, 
    { 3,119}, { 5,105}, { 3,151}, { 8,  0}, { 3,121}, { 6,167}, { 3,153}, { 9,181}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,171}, { 3,121}, { 7,117}, { 3,153}, {11, 26}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,107}, { 3,121}, { 7, 87}, { 3,153}, {10, 99}, 
    { 3,119}, { 6,118}, { 3,151}, { 9, 70}, { 3,121}, { 7,166}, { 3,153}, {11,221}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,185}, { 3,121}, { 6,154}, { 3,153}, { 9,160}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,156}, { 3,121}, { 7,106}, { 3,153}, {10,233}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 90}, { 3,121}, { 6,169}, { 3,153}, {10, 64}, 
    { 3,119}, { 6,103}, { 3,151}, { 9, 10}, { 3,121}, { 7,155}, { 3,153}, {11,191}, 
    { 3,119}, { 5,105}, { 3,151}, { 8,  9}, { 3,121}, { 6,167}, { 3,153}, { 9,215}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,186}, { 3,121}, { 7,123}, { 3,153}, {11, 46}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,116}, { 3,121}, { 7, 89}, { 3,153}, {10,146}, 
    { 3,119}, { 6,118}, { 3,151}, { 9, 91}, { 3,121}, { 7,170}, { 3,153}, {11,240}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,183}, { 3,121}, { 6,154}, { 3,153}, { 9,147}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,148}, { 3,121}, { 7,102}, { 3,153}, {10,208}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 86}, { 3,121}, { 6,169}, { 3,153}, {10, 48}, 
    { 3,119}, { 6,103}, { 3,151}, { 9,  5}, { 3,121}, { 7,149}, { 3,153}, {11,111}, 
    { 3,119}, { 5,105}, { 3,151}, { 8,  7}, { 3,121}, { 6,167}, { 3,153}, { 9,198}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,182}, { 3,121}, { 7,117}, { 3,153}, {11, 34}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,112}, { 3,121}, { 7, 87}, { 3,153}, {10,114}, 
    { 3,119}, { 6,118}, { 3,151}, { 9, 80}, { 3,121}, { 7,166}, { 3,153}, {11,229}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,185}, { 3,121}, { 6,154}, { 3,153}, { 9,172}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,165}, { 3,121}, { 7,106}, { 3,153}, {11, 15}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,101}, { 3,121}, { 6,169}, { 3,153}, {10, 84}, 
    { 3,119}, { 6,103}, { 3,151}, { 9, 55}, { 3,121}, { 7,155}, { 3,153}, {11,207}, 
    { 3,119}, { 5,105}, { 3,151}, { 8, 71}, { 3,121}, { 6,167}, { 3,153}, {10,  4}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,199}, { 3,121}, { 7,123}, { 3,153}, {11, 61}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,124}, { 3,121}, { 7, 89}, { 3,153}, {10,163}, 
    { 3,119}, { 6,118}, { 3,151}, { 9,100}, { 3,121}, { 7,170}, { 3,153}, {11,251}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,183}, { 3,121}, { 6,154}, { 3,153}, { 9,125}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,144}, { 3,121}, { 7,102}, { 3,153}, {10,197}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 73}, { 3,121}, { 6,169}, { 3,153}, {10, 41}, 
    { 3,119}, { 6,103}, { 3,151}, { 8,201}, { 3,121}, { 7,149}, { 3,153}, {11, 83}, 
    { 3,119}, { 5,105}, { 3,151}, { 8,  0}, { 3,121}, { 6,167}, { 3,153}, { 9,187}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,171}, { 3,121}, { 7,117}, { 3,153}, {11, 30}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,107}, { 3,121}, { 7, 87}, { 3,153}, {10,110}, 
    { 3,119}, { 6,118}, { 3,151}, { 9, 74}, { 3,121}, { 7,166}, { 3,153}, {11,225}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,185}, { 3,121}, { 6,154}, { 3,153}, { 9,164}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,156}, { 3,121}, { 7,106}, { 3,153}, {10,249}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 90}, { 3,121}, { 6,169}, { 3,153}, {10, 75}, 
    { 3,119}, { 6,103}, { 3,151}, { 9, 11}, { 3,121}, { 7,155}, { 3,153}, {11,196}, 
    { 3,119}, { 5,105}, { 3,151}, { 8,  9}, { 3,121}, { 6,167}, { 3,153}, { 9,217}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,186}, { 3,121}, { 7,123}, { 3,153}, {11, 51}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,116}, { 3,121}, { 7, 89}, { 3,153}, {10,159}, 
    { 3,119}, { 6,118}, { 3,151}, { 9, 96}, { 3,121}, { 7,170}, { 3,153}, {11,244}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,183}, { 3,121}, { 6,154}, { 3,153}, { 9,157}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,148}, { 3,121}, { 7,102}, { 3,153}, {10,218}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 86}, { 3,121}, { 6,169}, { 3,153}, {10, 58}, 
    { 3,119}, { 6,103}, { 3,151}, { 9,  6}, { 3,121}, { 7,149}, { 3,153}, {11,178}, 
    { 3,119}, { 5,105}, { 3,151}, { 8,  7}, { 3,121}, { 6,167}, { 3,153}, { 9,202}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,182}, { 3,121}, { 7,117}, { 3,153}, {11, 38}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,112}, { 3,121}, { 7, 87}, { 3,153}, {10,127}, 
    { 3,119}, { 6,118}, { 3,151}, { 9, 85}, { 3,121}, { 7,166}, { 3,153}, {11,236}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,185}, { 3,121}, { 6,154}, { 3,153}, { 9,176}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,165}, { 3,121}, { 7,106}, { 3,153}, {11, 19}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,101}, { 3,121}, { 6,169}, { 3,153}, {10, 93}, 
    { 3,119}, { 6,103}, { 3,151}, { 9, 57}, { 3,121}, { 7,155}, { 3,153}, {11,212}, 
    { 3,119}, { 5,105}, { 3,151}, { 8, 71}, { 3,121}, { 6,167}, { 3,153}, {10, 13}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,199}, { 3,121}, { 7,123}, { 3,153}, {11, 66}, 
    { 3,119}, { 5,122}, { 3,151}, { 8,124}, { 3,121}, { 7, 89}, { 3,153}, {10,174}, 
    { 3,119}, { 6,118}, { 3,151}, { 9,108}, { 3,121}, { 7,170}, { 3,153}, {11,255}, 
  }
  #endif
};

static radaudio_huffman rada_nonzero_coefficient_pair_huff3 =
{
  #ifdef HUFFMAN_ENCODE
  {
    {  59, 9}, { 271,11}, {1295,11}, { 783,11}, { 359,10}, { 871,10}, { 315, 9}, { 187, 9}, 
    {   0, 0}, { 443, 9}, { 123, 9}, { 231,10}, { 743,10}, {1807,11}, { 143,11}, {1167,11}, 
    { 655,11}, {1679,11}, { 399,11}, {1423,11}, { 911,11}, {1935,11}, {  79,11}, {1103,11}, 
    {   0, 0}, { 591,11}, {1615,11}, { 335,11}, {1359,11}, { 847,11}, {1871,11}, { 207,11}, 
    {1231,11}, { 719,11}, {1743,11}, { 463,11}, {1487,11}, { 975,11}, {1999,11}, { 487,10}, 
    {   0, 0}, { 999,10}, {  47,11}, {1071,11}, { 559,11}, {1583,11}, { 303,11}, {1327,11}, 
    { 815,11}, {1839,11}, { 175,11}, {1199,11}, { 687,11}, {1711,11}, {  23,10}, { 535,10}, 
    {   0, 0}, { 279,10}, { 791,10}, { 431,11}, {1455,11}, { 943,11}, {1967,11}, { 111,11}, 
    { 151,10}, {1135,11}, { 623,11}, {1647,11}, { 367,11}, { 663,10}, { 379, 9}, {  51, 8}, 
    {   0, 0}, { 179, 8}, { 251, 9}, { 407,10}, {1391,11}, { 879,11}, {1903,11}, { 239,11}, 
    { 919,10}, {1263,11}, { 751,11}, {1775,11}, {  87,10}, { 507, 9}, { 115, 8}, {  45, 7}, 
    {   0, 0}, { 109, 7}, { 243, 8}, {   7, 9}, { 599,10}, { 495,11}, {1519,11}, {1007,11}, 
    { 263, 9}, {2031,11}, {  31,11}, { 343,10}, { 135, 9}, {  11, 8}, {  29, 7}, {  21, 6}, 
    {   0, 0}, {   1, 5}, {  93, 7}, { 139, 8}, { 391, 9}, { 855,10}, {1055,11}, { 543,11}, 
    {  71, 9}, {1567,11}, { 215,10}, { 727,10}, {  75, 8}, {  61, 7}, {  53, 6}, {   0, 3}, 
    {   0, 0}, {   4, 3}, {  17, 5}, { 125, 7}, { 203, 8}, { 471,10}, { 983,10}, { 287,11}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, {   0, 0}, 
    { 327, 9}, {1311,11}, {  55,10}, { 567,10}, {  43, 8}, {   3, 7}, {   9, 5}, {   2, 3}, 
    {   0, 0}, {   6, 3}, {  25, 5}, {  67, 7}, { 171, 8}, { 311,10}, { 823,10}, { 799,11}, 
    { 199, 9}, {1823,11}, { 159,11}, { 183,10}, { 455, 9}, { 107, 8}, {  35, 7}, {   5, 5}, 
    {   0, 0}, {  13, 6}, {  99, 7}, { 235, 8}, {  39, 9}, { 695,10}, {1183,11}, { 671,11}, 
    { 439,10}, {1695,11}, { 415,11}, {1439,11}, { 951,10}, { 295, 9}, {  27, 8}, {  19, 7}, 
    {   0, 0}, {  83, 7}, { 155, 8}, { 167, 9}, { 119,10}, { 927,11}, {1951,11}, {  95,11}, 
    { 631,10}, {1119,11}, { 607,11}, {1631,11}, { 351,11}, { 375,10}, { 423, 9}, {  91, 8}, 
    {   0, 0}, { 219, 8}, { 103, 9}, { 887,10}, {1375,11}, { 863,11}, {1887,11}, { 223,11}, 
    {1247,11}, { 735,11}, {1759,11}, { 479,11}, {1503,11}, { 991,11}, { 247,10}, { 759,10}, 
    {   0, 0}, { 503,10}, {1015,10}, {2015,11}, {  63,11}, {1087,11}, { 575,11}, {1599,11}, 
    { 319,11}, {1343,11}, { 831,11}, {1855,11}, { 191,11}, {1215,11}, { 703,11}, {  15,10}, 
    {   0, 0}, { 527,10}, {1727,11}, { 447,11}, {1471,11}, { 959,11}, {1983,11}, { 127,11}, 
    {1151,11}, { 639,11}, {1663,11}, { 383,11}, {1407,11}, { 895,11}, {1919,11}, { 255,11}, 
    {   0, 0}, {1279,11}, { 767,11}, {1791,11}, { 511,11}, {1535,11}, {1023,11}, {2047,11}, 
  },
  #endif
  #ifdef HUFFMAN_DECODE
  {
    { 3,119}, { 5,105}, { 3,151}, { 7,149}, { 3,121}, { 5,167}, { 3,153}, { 9, 91}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,101}, { 3,121}, { 6,169}, { 3,153}, {10,231}, 
    { 3,119}, { 5,122}, { 3,151}, { 7,183}, { 3,121}, { 6,103}, { 3,153}, {10, 54}, 
    { 3,119}, { 5,154}, { 3,151}, { 8,182}, { 3,121}, { 7,102}, { 3,153}, {11, 98}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,166}, { 3,121}, { 5,167}, { 3,153}, { 9,172}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,148}, { 3,121}, { 7, 87}, { 3,153}, {11, 42}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 71}, { 3,121}, { 6,118}, { 3,153}, {10,146}, 
    { 3,119}, { 5,154}, { 3,151}, { 9,  0}, { 3,121}, { 7,117}, { 3,153}, {11,220}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,155}, { 3,121}, { 5,167}, { 3,153}, { 9,112}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,116}, { 3,121}, { 6,169}, { 3,153}, {11, 22}, 
    { 3,119}, { 5,122}, { 3,151}, { 7,185}, { 3,121}, { 6,103}, { 3,153}, {10, 84}, 
    { 3,119}, { 5,154}, { 3,151}, { 8,199}, { 3,121}, { 7,106}, { 3,153}, {11,191}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,170}, { 3,121}, { 5,167}, { 3,153}, { 9,202}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,165}, { 3,121}, { 7, 89}, { 3,153}, {11, 63}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 86}, { 3,121}, { 6,118}, { 3,153}, {10,188}, 
    { 3,119}, { 5,154}, { 3,151}, { 9, 10}, { 3,121}, { 7,123}, { 3,153}, {11,239}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,149}, { 3,121}, { 5,167}, { 3,153}, { 9,100}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,107}, { 3,121}, { 6,169}, { 3,153}, {11, 14}, 
    { 3,119}, { 5,122}, { 3,151}, { 7,183}, { 3,121}, { 6,103}, { 3,153}, {10, 64}, 
    { 3,119}, { 5,154}, { 3,151}, { 8,186}, { 3,121}, { 7,102}, { 3,153}, {11,162}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,166}, { 3,121}, { 5,167}, { 3,153}, { 9,187}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,156}, { 3,121}, { 7, 87}, { 3,153}, {11, 50}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 73}, { 3,121}, { 6,118}, { 3,153}, {10,163}, 
    { 3,119}, { 5,154}, { 3,151}, { 9,  7}, { 3,121}, { 7,117}, { 3,153}, {11,228}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,155}, { 3,121}, { 5,167}, { 3,153}, { 9,160}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,124}, { 3,121}, { 6,169}, { 3,153}, {11, 31}, 
    { 3,119}, { 5,122}, { 3,151}, { 7,185}, { 3,121}, { 6,103}, { 3,153}, {10,114}, 
    { 3,119}, { 5,154}, { 3,151}, { 8,201}, { 3,121}, { 7,106}, { 3,153}, {11,207}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,170}, { 3,121}, { 5,167}, { 3,153}, {10, 11}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,171}, { 3,121}, { 7, 89}, { 3,153}, {11, 79}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 90}, { 3,121}, { 6,118}, { 3,153}, {10,214}, 
    { 3,119}, { 5,154}, { 3,151}, { 9, 74}, { 3,121}, { 7,123}, { 3,153}, {11,247}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,149}, { 3,121}, { 5,167}, { 3,153}, { 9, 96}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,101}, { 3,121}, { 6,169}, { 3,153}, {11,  1}, 
    { 3,119}, { 5,122}, { 3,151}, { 7,183}, { 3,121}, { 6,103}, { 3,153}, {10, 57}, 
    { 3,119}, { 5,154}, { 3,151}, { 8,182}, { 3,121}, { 7,102}, { 3,153}, {11,127}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,166}, { 3,121}, { 5,167}, { 3,153}, { 9,181}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,148}, { 3,121}, { 7, 87}, { 3,153}, {11, 46}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 71}, { 3,121}, { 6,118}, { 3,153}, {10,157}, 
    { 3,119}, { 5,154}, { 3,151}, { 9,  6}, { 3,121}, { 7,117}, { 3,153}, {11,224}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,155}, { 3,121}, { 5,167}, { 3,153}, { 9,144}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,116}, { 3,121}, { 6,169}, { 3,153}, {11, 27}, 
    { 3,119}, { 5,122}, { 3,151}, { 7,185}, { 3,121}, { 6,103}, { 3,153}, {10, 99}, 
    { 3,119}, { 5,154}, { 3,151}, { 8,199}, { 3,121}, { 7,106}, { 3,153}, {11,196}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,170}, { 3,121}, { 5,167}, { 3,153}, {10,  4}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,165}, { 3,121}, { 7, 89}, { 3,153}, {11, 68}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 86}, { 3,121}, { 6,118}, { 3,153}, {10,197}, 
    { 3,119}, { 5,154}, { 3,151}, { 9, 70}, { 3,121}, { 7,123}, { 3,153}, {11,243}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,149}, { 3,121}, { 5,167}, { 3,153}, { 9,108}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,107}, { 3,121}, { 6,169}, { 3,153}, {11, 18}, 
    { 3,119}, { 5,122}, { 3,151}, { 7,183}, { 3,121}, { 6,103}, { 3,153}, {10, 75}, 
    { 3,119}, { 5,154}, { 3,151}, { 8,186}, { 3,121}, { 7,102}, { 3,153}, {11,178}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,166}, { 3,121}, { 5,167}, { 3,153}, { 9,198}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,156}, { 3,121}, { 7, 87}, { 3,153}, {11, 59}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 73}, { 3,121}, { 6,118}, { 3,153}, {10,176}, 
    { 3,119}, { 5,154}, { 3,151}, { 9,  9}, { 3,121}, { 7,117}, { 3,153}, {11,235}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,155}, { 3,121}, { 5,167}, { 3,153}, { 9,164}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,124}, { 3,121}, { 6,169}, { 3,153}, {11, 35}, 
    { 3,119}, { 5,122}, { 3,151}, { 7,185}, { 3,121}, { 6,103}, { 3,153}, {10,125}, 
    { 3,119}, { 5,154}, { 3,151}, { 8,201}, { 3,121}, { 7,106}, { 3,153}, {11,211}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,170}, { 3,121}, { 5,167}, { 3,153}, {10, 39}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,171}, { 3,121}, { 7, 89}, { 3,153}, {11, 93}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 90}, { 3,121}, { 6,118}, { 3,153}, {10,217}, 
    { 3,119}, { 5,154}, { 3,151}, { 9, 85}, { 3,121}, { 7,123}, { 3,153}, {11,252}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,149}, { 3,121}, { 5,167}, { 3,153}, { 9, 91}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,101}, { 3,121}, { 6,169}, { 3,153}, {10,233}, 
    { 3,119}, { 5,122}, { 3,151}, { 7,183}, { 3,121}, { 6,103}, { 3,153}, {10, 55}, 
    { 3,119}, { 5,154}, { 3,151}, { 8,182}, { 3,121}, { 7,102}, { 3,153}, {11,111}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,166}, { 3,121}, { 5,167}, { 3,153}, { 9,172}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,148}, { 3,121}, { 7, 87}, { 3,153}, {11, 44}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 71}, { 3,121}, { 6,118}, { 3,153}, {10,147}, 
    { 3,119}, { 5,154}, { 3,151}, { 9,  0}, { 3,121}, { 7,117}, { 3,153}, {11,222}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,155}, { 3,121}, { 5,167}, { 3,153}, { 9,112}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,116}, { 3,121}, { 6,169}, { 3,153}, {11, 25}, 
    { 3,119}, { 5,122}, { 3,151}, { 7,185}, { 3,121}, { 6,103}, { 3,153}, {10, 92}, 
    { 3,119}, { 5,154}, { 3,151}, { 8,199}, { 3,121}, { 7,106}, { 3,153}, {11,194}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,170}, { 3,121}, { 5,167}, { 3,153}, { 9,202}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,165}, { 3,121}, { 7, 89}, { 3,153}, {11, 66}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 86}, { 3,121}, { 6,118}, { 3,153}, {10,192}, 
    { 3,119}, { 5,154}, { 3,151}, { 9, 10}, { 3,121}, { 7,123}, { 3,153}, {11,241}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,149}, { 3,121}, { 5,167}, { 3,153}, { 9,100}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,107}, { 3,121}, { 6,169}, { 3,153}, {11, 16}, 
    { 3,119}, { 5,122}, { 3,151}, { 7,183}, { 3,121}, { 6,103}, { 3,153}, {10, 69}, 
    { 3,119}, { 5,154}, { 3,151}, { 8,186}, { 3,121}, { 7,102}, { 3,153}, {11,175}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,166}, { 3,121}, { 5,167}, { 3,153}, { 9,187}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,156}, { 3,121}, { 7, 87}, { 3,153}, {11, 52}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 73}, { 3,121}, { 6,118}, { 3,153}, {10,173}, 
    { 3,119}, { 5,154}, { 3,151}, { 9,  7}, { 3,121}, { 7,117}, { 3,153}, {11,230}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,155}, { 3,121}, { 5,167}, { 3,153}, { 9,160}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,124}, { 3,121}, { 6,169}, { 3,153}, {11, 33}, 
    { 3,119}, { 5,122}, { 3,151}, { 7,185}, { 3,121}, { 6,103}, { 3,153}, {10,115}, 
    { 3,119}, { 5,154}, { 3,151}, { 8,201}, { 3,121}, { 7,106}, { 3,153}, {11,209}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,170}, { 3,121}, { 5,167}, { 3,153}, {10, 12}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,171}, { 3,121}, { 7, 89}, { 3,153}, {11, 82}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 90}, { 3,121}, { 6,118}, { 3,153}, {10,215}, 
    { 3,119}, { 5,154}, { 3,151}, { 9, 74}, { 3,121}, { 7,123}, { 3,153}, {11,250}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,149}, { 3,121}, { 5,167}, { 3,153}, { 9, 96}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,101}, { 3,121}, { 6,169}, { 3,153}, {11,  3}, 
    { 3,119}, { 5,122}, { 3,151}, { 7,183}, { 3,121}, { 6,103}, { 3,153}, {10, 58}, 
    { 3,119}, { 5,154}, { 3,151}, { 8,182}, { 3,121}, { 7,102}, { 3,153}, {11,159}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,166}, { 3,121}, { 5,167}, { 3,153}, { 9,181}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,148}, { 3,121}, { 7, 87}, { 3,153}, {11, 48}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 71}, { 3,121}, { 6,118}, { 3,153}, {10,158}, 
    { 3,119}, { 5,154}, { 3,151}, { 9,  6}, { 3,121}, { 7,117}, { 3,153}, {11,226}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,155}, { 3,121}, { 5,167}, { 3,153}, { 9,144}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,116}, { 3,121}, { 6,169}, { 3,153}, {11, 29}, 
    { 3,119}, { 5,122}, { 3,151}, { 7,185}, { 3,121}, { 6,103}, { 3,153}, {10,109}, 
    { 3,119}, { 5,154}, { 3,151}, { 8,199}, { 3,121}, { 7,106}, { 3,153}, {11,205}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,170}, { 3,121}, { 5,167}, { 3,153}, {10,  5}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,165}, { 3,121}, { 7, 89}, { 3,153}, {11, 77}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 86}, { 3,121}, { 6,118}, { 3,153}, {10,203}, 
    { 3,119}, { 5,154}, { 3,151}, { 9, 70}, { 3,121}, { 7,123}, { 3,153}, {11,245}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,149}, { 3,121}, { 5,167}, { 3,153}, { 9,108}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,107}, { 3,121}, { 6,169}, { 3,153}, {11, 20}, 
    { 3,119}, { 5,122}, { 3,151}, { 7,183}, { 3,121}, { 6,103}, { 3,153}, {10, 80}, 
    { 3,119}, { 5,154}, { 3,151}, { 8,186}, { 3,121}, { 7,102}, { 3,153}, {11,189}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,166}, { 3,121}, { 5,167}, { 3,153}, { 9,198}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,156}, { 3,121}, { 7, 87}, { 3,153}, {11, 61}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 73}, { 3,121}, { 6,118}, { 3,153}, {10,180}, 
    { 3,119}, { 5,154}, { 3,151}, { 9,  9}, { 3,121}, { 7,117}, { 3,153}, {11,237}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,155}, { 3,121}, { 5,167}, { 3,153}, { 9,164}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,124}, { 3,121}, { 6,169}, { 3,153}, {11, 37}, 
    { 3,119}, { 5,122}, { 3,151}, { 7,185}, { 3,121}, { 6,103}, { 3,153}, {10,126}, 
    { 3,119}, { 5,154}, { 3,151}, { 8,201}, { 3,121}, { 7,106}, { 3,153}, {11,213}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,170}, { 3,121}, { 5,167}, { 3,153}, {10, 41}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,171}, { 3,121}, { 7, 89}, { 3,153}, {11, 95}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 90}, { 3,121}, { 6,118}, { 3,153}, {10,218}, 
    { 3,119}, { 5,154}, { 3,151}, { 9, 85}, { 3,121}, { 7,123}, { 3,153}, {11,254}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,149}, { 3,121}, { 5,167}, { 3,153}, { 9, 91}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,101}, { 3,121}, { 6,169}, { 3,153}, {10,231}, 
    { 3,119}, { 5,122}, { 3,151}, { 7,183}, { 3,121}, { 6,103}, { 3,153}, {10, 54}, 
    { 3,119}, { 5,154}, { 3,151}, { 8,182}, { 3,121}, { 7,102}, { 3,153}, {11,110}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,166}, { 3,121}, { 5,167}, { 3,153}, { 9,172}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,148}, { 3,121}, { 7, 87}, { 3,153}, {11, 43}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 71}, { 3,121}, { 6,118}, { 3,153}, {10,146}, 
    { 3,119}, { 5,154}, { 3,151}, { 9,  0}, { 3,121}, { 7,117}, { 3,153}, {11,221}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,155}, { 3,121}, { 5,167}, { 3,153}, { 9,112}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,116}, { 3,121}, { 6,169}, { 3,153}, {11, 23}, 
    { 3,119}, { 5,122}, { 3,151}, { 7,185}, { 3,121}, { 6,103}, { 3,153}, {10, 84}, 
    { 3,119}, { 5,154}, { 3,151}, { 8,199}, { 3,121}, { 7,106}, { 3,153}, {11,193}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,170}, { 3,121}, { 5,167}, { 3,153}, { 9,202}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,165}, { 3,121}, { 7, 89}, { 3,153}, {11, 65}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 86}, { 3,121}, { 6,118}, { 3,153}, {10,188}, 
    { 3,119}, { 5,154}, { 3,151}, { 9, 10}, { 3,121}, { 7,123}, { 3,153}, {11,240}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,149}, { 3,121}, { 5,167}, { 3,153}, { 9,100}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,107}, { 3,121}, { 6,169}, { 3,153}, {11, 15}, 
    { 3,119}, { 5,122}, { 3,151}, { 7,183}, { 3,121}, { 6,103}, { 3,153}, {10, 64}, 
    { 3,119}, { 5,154}, { 3,151}, { 8,186}, { 3,121}, { 7,102}, { 3,153}, {11,174}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,166}, { 3,121}, { 5,167}, { 3,153}, { 9,187}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,156}, { 3,121}, { 7, 87}, { 3,153}, {11, 51}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 73}, { 3,121}, { 6,118}, { 3,153}, {10,163}, 
    { 3,119}, { 5,154}, { 3,151}, { 9,  7}, { 3,121}, { 7,117}, { 3,153}, {11,229}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,155}, { 3,121}, { 5,167}, { 3,153}, { 9,160}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,124}, { 3,121}, { 6,169}, { 3,153}, {11, 32}, 
    { 3,119}, { 5,122}, { 3,151}, { 7,185}, { 3,121}, { 6,103}, { 3,153}, {10,114}, 
    { 3,119}, { 5,154}, { 3,151}, { 8,201}, { 3,121}, { 7,106}, { 3,153}, {11,208}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,170}, { 3,121}, { 5,167}, { 3,153}, {10, 11}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,171}, { 3,121}, { 7, 89}, { 3,153}, {11, 81}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 90}, { 3,121}, { 6,118}, { 3,153}, {10,214}, 
    { 3,119}, { 5,154}, { 3,151}, { 9, 74}, { 3,121}, { 7,123}, { 3,153}, {11,249}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,149}, { 3,121}, { 5,167}, { 3,153}, { 9, 96}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,101}, { 3,121}, { 6,169}, { 3,153}, {11,  2}, 
    { 3,119}, { 5,122}, { 3,151}, { 7,183}, { 3,121}, { 6,103}, { 3,153}, {10, 57}, 
    { 3,119}, { 5,154}, { 3,151}, { 8,182}, { 3,121}, { 7,102}, { 3,153}, {11,145}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,166}, { 3,121}, { 5,167}, { 3,153}, { 9,181}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,148}, { 3,121}, { 7, 87}, { 3,153}, {11, 47}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 71}, { 3,121}, { 6,118}, { 3,153}, {10,157}, 
    { 3,119}, { 5,154}, { 3,151}, { 9,  6}, { 3,121}, { 7,117}, { 3,153}, {11,225}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,155}, { 3,121}, { 5,167}, { 3,153}, { 9,144}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,116}, { 3,121}, { 6,169}, { 3,153}, {11, 28}, 
    { 3,119}, { 5,122}, { 3,151}, { 7,185}, { 3,121}, { 6,103}, { 3,153}, {10, 99}, 
    { 3,119}, { 5,154}, { 3,151}, { 8,199}, { 3,121}, { 7,106}, { 3,153}, {11,204}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,170}, { 3,121}, { 5,167}, { 3,153}, {10,  4}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,165}, { 3,121}, { 7, 89}, { 3,153}, {11, 76}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 86}, { 3,121}, { 6,118}, { 3,153}, {10,197}, 
    { 3,119}, { 5,154}, { 3,151}, { 9, 70}, { 3,121}, { 7,123}, { 3,153}, {11,244}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,149}, { 3,121}, { 5,167}, { 3,153}, { 9,108}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,107}, { 3,121}, { 6,169}, { 3,153}, {11, 19}, 
    { 3,119}, { 5,122}, { 3,151}, { 7,183}, { 3,121}, { 6,103}, { 3,153}, {10, 75}, 
    { 3,119}, { 5,154}, { 3,151}, { 8,186}, { 3,121}, { 7,102}, { 3,153}, {11,179}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,166}, { 3,121}, { 5,167}, { 3,153}, { 9,198}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,156}, { 3,121}, { 7, 87}, { 3,153}, {11, 60}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 73}, { 3,121}, { 6,118}, { 3,153}, {10,176}, 
    { 3,119}, { 5,154}, { 3,151}, { 9,  9}, { 3,121}, { 7,117}, { 3,153}, {11,236}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,155}, { 3,121}, { 5,167}, { 3,153}, { 9,164}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,124}, { 3,121}, { 6,169}, { 3,153}, {11, 36}, 
    { 3,119}, { 5,122}, { 3,151}, { 7,185}, { 3,121}, { 6,103}, { 3,153}, {10,125}, 
    { 3,119}, { 5,154}, { 3,151}, { 8,201}, { 3,121}, { 7,106}, { 3,153}, {11,212}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,170}, { 3,121}, { 5,167}, { 3,153}, {10, 39}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,171}, { 3,121}, { 7, 89}, { 3,153}, {11, 94}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 90}, { 3,121}, { 6,118}, { 3,153}, {10,217}, 
    { 3,119}, { 5,154}, { 3,151}, { 9, 85}, { 3,121}, { 7,123}, { 3,153}, {11,253}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,149}, { 3,121}, { 5,167}, { 3,153}, { 9, 91}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,101}, { 3,121}, { 6,169}, { 3,153}, {10,233}, 
    { 3,119}, { 5,122}, { 3,151}, { 7,183}, { 3,121}, { 6,103}, { 3,153}, {10, 55}, 
    { 3,119}, { 5,154}, { 3,151}, { 8,182}, { 3,121}, { 7,102}, { 3,153}, {11,113}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,166}, { 3,121}, { 5,167}, { 3,153}, { 9,172}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,148}, { 3,121}, { 7, 87}, { 3,153}, {11, 45}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 71}, { 3,121}, { 6,118}, { 3,153}, {10,147}, 
    { 3,119}, { 5,154}, { 3,151}, { 9,  0}, { 3,121}, { 7,117}, { 3,153}, {11,223}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,155}, { 3,121}, { 5,167}, { 3,153}, { 9,112}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,116}, { 3,121}, { 6,169}, { 3,153}, {11, 26}, 
    { 3,119}, { 5,122}, { 3,151}, { 7,185}, { 3,121}, { 6,103}, { 3,153}, {10, 92}, 
    { 3,119}, { 5,154}, { 3,151}, { 8,199}, { 3,121}, { 7,106}, { 3,153}, {11,195}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,170}, { 3,121}, { 5,167}, { 3,153}, { 9,202}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,165}, { 3,121}, { 7, 89}, { 3,153}, {11, 67}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 86}, { 3,121}, { 6,118}, { 3,153}, {10,192}, 
    { 3,119}, { 5,154}, { 3,151}, { 9, 10}, { 3,121}, { 7,123}, { 3,153}, {11,242}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,149}, { 3,121}, { 5,167}, { 3,153}, { 9,100}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,107}, { 3,121}, { 6,169}, { 3,153}, {11, 17}, 
    { 3,119}, { 5,122}, { 3,151}, { 7,183}, { 3,121}, { 6,103}, { 3,153}, {10, 69}, 
    { 3,119}, { 5,154}, { 3,151}, { 8,186}, { 3,121}, { 7,102}, { 3,153}, {11,177}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,166}, { 3,121}, { 5,167}, { 3,153}, { 9,187}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,156}, { 3,121}, { 7, 87}, { 3,153}, {11, 53}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 73}, { 3,121}, { 6,118}, { 3,153}, {10,173}, 
    { 3,119}, { 5,154}, { 3,151}, { 9,  7}, { 3,121}, { 7,117}, { 3,153}, {11,234}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,155}, { 3,121}, { 5,167}, { 3,153}, { 9,160}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,124}, { 3,121}, { 6,169}, { 3,153}, {11, 34}, 
    { 3,119}, { 5,122}, { 3,151}, { 7,185}, { 3,121}, { 6,103}, { 3,153}, {10,115}, 
    { 3,119}, { 5,154}, { 3,151}, { 8,201}, { 3,121}, { 7,106}, { 3,153}, {11,210}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,170}, { 3,121}, { 5,167}, { 3,153}, {10, 12}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,171}, { 3,121}, { 7, 89}, { 3,153}, {11, 83}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 90}, { 3,121}, { 6,118}, { 3,153}, {10,215}, 
    { 3,119}, { 5,154}, { 3,151}, { 9, 74}, { 3,121}, { 7,123}, { 3,153}, {11,251}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,149}, { 3,121}, { 5,167}, { 3,153}, { 9, 96}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,101}, { 3,121}, { 6,169}, { 3,153}, {11, 13}, 
    { 3,119}, { 5,122}, { 3,151}, { 7,183}, { 3,121}, { 6,103}, { 3,153}, {10, 58}, 
    { 3,119}, { 5,154}, { 3,151}, { 8,182}, { 3,121}, { 7,102}, { 3,153}, {11,161}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,166}, { 3,121}, { 5,167}, { 3,153}, { 9,181}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,148}, { 3,121}, { 7, 87}, { 3,153}, {11, 49}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 71}, { 3,121}, { 6,118}, { 3,153}, {10,158}, 
    { 3,119}, { 5,154}, { 3,151}, { 9,  6}, { 3,121}, { 7,117}, { 3,153}, {11,227}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,155}, { 3,121}, { 5,167}, { 3,153}, { 9,144}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,116}, { 3,121}, { 6,169}, { 3,153}, {11, 30}, 
    { 3,119}, { 5,122}, { 3,151}, { 7,185}, { 3,121}, { 6,103}, { 3,153}, {10,109}, 
    { 3,119}, { 5,154}, { 3,151}, { 8,199}, { 3,121}, { 7,106}, { 3,153}, {11,206}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,170}, { 3,121}, { 5,167}, { 3,153}, {10,  5}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,165}, { 3,121}, { 7, 89}, { 3,153}, {11, 78}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 86}, { 3,121}, { 6,118}, { 3,153}, {10,203}, 
    { 3,119}, { 5,154}, { 3,151}, { 9, 70}, { 3,121}, { 7,123}, { 3,153}, {11,246}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,149}, { 3,121}, { 5,167}, { 3,153}, { 9,108}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,107}, { 3,121}, { 6,169}, { 3,153}, {11, 21}, 
    { 3,119}, { 5,122}, { 3,151}, { 7,183}, { 3,121}, { 6,103}, { 3,153}, {10, 80}, 
    { 3,119}, { 5,154}, { 3,151}, { 8,186}, { 3,121}, { 7,102}, { 3,153}, {11,190}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,166}, { 3,121}, { 5,167}, { 3,153}, { 9,198}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,156}, { 3,121}, { 7, 87}, { 3,153}, {11, 62}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 73}, { 3,121}, { 6,118}, { 3,153}, {10,180}, 
    { 3,119}, { 5,154}, { 3,151}, { 9,  9}, { 3,121}, { 7,117}, { 3,153}, {11,238}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,155}, { 3,121}, { 5,167}, { 3,153}, { 9,164}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,124}, { 3,121}, { 6,169}, { 3,153}, {11, 38}, 
    { 3,119}, { 5,122}, { 3,151}, { 7,185}, { 3,121}, { 6,103}, { 3,153}, {10,126}, 
    { 3,119}, { 5,154}, { 3,151}, { 8,201}, { 3,121}, { 7,106}, { 3,153}, {11,219}, 
    { 3,119}, { 5,105}, { 3,151}, { 7,170}, { 3,121}, { 5,167}, { 3,153}, {10, 41}, 
    { 3,119}, { 5,150}, { 3,151}, { 8,171}, { 3,121}, { 7, 89}, { 3,153}, {11, 97}, 
    { 3,119}, { 5,122}, { 3,151}, { 8, 90}, { 3,121}, { 6,118}, { 3,153}, {10,218}, 
    { 3,119}, { 5,154}, { 3,151}, { 9, 85}, { 3,121}, { 7,123}, { 3,153}, {11,255}, 
  }
  #endif
};

// total size 84 KB for HUFFMAN_DECODE, 21 KB for HUFFMAN_ENCODE
