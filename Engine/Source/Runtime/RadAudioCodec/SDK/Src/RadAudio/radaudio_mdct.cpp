// Copyright Epic Games Tools, LLC. All Rights Reserved.
#ifndef _CRT_SECURE_NO_WARNINGS
#define _CRT_SECURE_NO_WARNINGS
#endif
#include <stdio.h>
#include <stdlib.h>
#include <stddef.h>
#include <assert.h>
#include <math.h>

#include "radaudio_mdct.h"
#include "radaudio_mdct_internal.h"
#include "radaudio_mdct_internal.inl"
#include "rrbits.h"


// The FFT alg used here was designed to be very FMA-friendly, but because we can't assume FMAs are present on
// all target HW and want consistent results everywhere, we're using FMA-less algorithms for this application.

// Notation throughout this file:
//
// Let z = a + bi. Then conj(z) = a - bi.
//
// We can swap the real and imaginary parts of z to yield s(z) = b + ai ("swap").
// Now because
//
//   iz = i(a + bi) = -b + ai
//
// we get s(z) = i z*, which is convenient to manipulate this algebraically. Now
// obviously from the definition, we have
//
//   s(s(z)) = z
//
// Regular complex arithmetic rules further give the identities
//
//   s(z + w) = i conj(z + w) = i conj(z) + i conj(w) = s(z) + s(w)
//   s(zw) = i conj(zw) = i conj(z) conj(w) = s(z) conj(w) = conj(z) s(w)
//
// Note the "swap identity" (by the multiplication rule applied twice)
//
//   s(s(z) s(w)) = s(s(z) i conj(w)) = s(i s(zw)) = conj(i) s(s(zw)) = -i zw
//
// We mostly work with split real/imaginary parts throughout this file, so these
// swaps are "free" (just a matter of renaming variables). This lets us reduce
// complex multiplications -izw to regular complex multiplications zw with some
// swapping of the real/imaginary parts. (izw can also be handled by computing
// -izw as noted, and then folding a negate into the uses.)

namespace radaudio_fft_impl {

// tables generated by radaudio_mdct_build_tables.cpp
#include "radaudio_mdct_tables.inl"

// The FFT kernel is parameterized by an "Elem" type that gives shared functionality and determines
// the vector width. The bitrev + initial passes need to work slightly differently as the vector width
// increases, which is why that is inside here as well.
namespace {
struct ElemF32
{
   static constexpr size_t kCount = 1;

   float v;

   ElemF32() {}
   explicit ElemF32(float f) : v(f) {}

   static ElemF32 load(float const* ptr)        { return ElemF32(*ptr); }
   void store(float* ptr)                       { *ptr = v; }

   ElemF32 operator+(ElemF32 b) const           { return ElemF32(v + b.v); }
   ElemF32 operator-(ElemF32 b) const           { return ElemF32(v - b.v); }
   ElemF32 operator*(ElemF32 b) const           { return ElemF32(v * b.v); }

   ElemF32 reverse() const                      { return *this; }

   static RADFORCEINLINE void radix2_twiddle(
      ElemF32& ar, ElemF32& ai, ElemF32& br, ElemF32& bi, ElemF32 wr, ElemF32 wi
   )
   {
      radix2_twiddle_unfused(ar, ai, br, bi, wr, wi);
   }

   static RADFORCEINLINE void load_deinterleave(ElemF32& re, ElemF32& im, float const* ptr)
   {
      re.v = ptr[0];
      im.v = ptr[1];
   }

   static size_t bitrev_initial_radix4(float *out, float const *in, size_t N, FftSign sign)
   {
      size_t Nbits = rrCtz64(N);
      size_t shift_amt = kMaxFFTLog2 - Nbits;
      size_t step = N / 4;

      float const * inA = in;
      float const * inB = in + burst_swizzle(2 * step); // note: 2 not 1 because it's bit-reversed
      float const * inC = in + burst_swizzle(1 * step); // note: 1 not 2 because it's bit-reversed
      float const * inD = in + burst_swizzle(3 * step);

      // This was originally written for the negative sign variant, but all we need to do
      // to toggle the sign is to swap inC and inD pointers
      if (sign == FftSign_Positive)
         swap(inC, inD);

      // Apply the initial permutation along with the initial radix-4 butterflies
      // (which are special because the twiddles are with +-1 and +-i only, i.e. trivial)
      for (size_t i = 0; i < N; i += 4)
      {
         size_t is = burst_swizzle(i); // dest index
         size_t j = s_bit_reverse[i] >> shift_amt;
         size_t js = burst_swizzle(j); // source index

         float ar = inA[js + 0*kBurstSize];
         float ai = inA[js + 1*kBurstSize];
         float br = inB[js + 0*kBurstSize];
         float bi = inB[js + 1*kBurstSize];
         float cr = inC[js + 0*kBurstSize];
         float ci = inC[js + 1*kBurstSize];
         float dr = inD[js + 0*kBurstSize];
         float di = inD[js + 1*kBurstSize];

         dft4_bfly_permuted(ar, ai, br, bi, cr, ci, dr, di);

         out[is + 0 + 0*kBurstSize] = ar;
         out[is + 0 + 1*kBurstSize] = ai;
         out[is + 1 + 0*kBurstSize] = br;
         out[is + 1 + 1*kBurstSize] = bi;
         out[is + 2 + 0*kBurstSize] = cr;
         out[is + 2 + 1*kBurstSize] = ci;
         out[is + 3 + 0*kBurstSize] = dr;
         out[is + 3 + 1*kBurstSize] = di;
      }

      if (Nbits & 1)
      {
         size_t const swiz_N = burst_swizzle(N);
         float const * twiddle_i = &s_fft_twiddles[8 + 4];
         float const * twiddle_r = &s_fft_twiddles[8 + 2];
         
         float * outA = out;
         float * outB = out + burst_swizzle(4);
         size_t swiz_dec = burst_swizzle(~size_t(4));
         static_assert(kBurstSize >= 4, "Twiddle addressing assumes kBurstSwizzle >= 4");

         for (size_t j = 0; j < swiz_N; j = (j - swiz_dec) & swiz_dec)
         {
            float ar = outA[j + 0*kBurstSize];
            float ai = outA[j + 1*kBurstSize];
            float br = outB[j + 0*kBurstSize];
            float bi = outB[j + 1*kBurstSize];

            // Twiddle index
            size_t const k = j & 3;
            radix2_twiddle_unfused(ar, ai, br, bi, twiddle_r[k], twiddle_i[k]);

            outA[j + 0*kBurstSize] = ar;
            outA[j + 1*kBurstSize] = ai;
            outB[j + 0*kBurstSize] = br;
            outB[j + 1*kBurstSize] = bi;
         }

         return 8;
      }
      else
         return 4;
   }

   static void store_interleaved(float * dest, ElemF32 re, ElemF32 im)
   {
      dest[0] = re.v;
      dest[1] = im.v;
   }
};
} // anon namespace

FftKernelSet const kernels_scalar =
{
   ElemF32::bitrev_initial_radix4,
   burst_r4_fft_single_pass<ElemF32>,
   burst_imdct_prefft<ElemF32>,
   burst_imdct_postfft<ElemF32>,
};

} // namespace radaudio_fft_impl

static radaudio_fft_impl::FftKernelSet const * choose_kernels(radaudio_cpu_features cpu)
{
   using namespace radaudio_fft_impl;

#if defined(DO_BUILD_SSE4) || defined(DO_BUILD_AVX2)
   if (cpu.has_sse2) {
   #if defined(DO_BUILD_AVX2)
      if (cpu.has_avx2)
         return &kernels_avx2;
      else
   #endif
         return &kernels_sse2;
   }
#endif

#ifdef DO_BUILD_NEON
   return &kernels_neon;
#else
   return &kernels_scalar;
#endif
}

// The complex FFT driver func.
static void radaudio_fft(float *out, float const *in, size_t N, FftSign sign, radaudio_fft_impl::FftKernelSet const * kernels)
{
   using namespace radaudio_fft_impl;

   const size_t swiz_N = burst_swizzle(N);

   RR_ASSERT(16 <= N && N <= kMaxFFTN);
   RR_ASSERT((N & (N - 1)) == 0); // checks for pow2

   size_t const initial_step = kernels->initial(out, in, N, sign);

   // For the size we support here, an iterative FFT is always fine since we're
   // comfortably in the L1D cache (the largest FFT we do is 512 complex elements,
   // which is 4K of data).

   // Iteratively do all the CT passes for increasing N (DIT order), indexed by step size (which is N/4)
   for (size_t step = initial_step; step <= N / 4; step *= 4)
         kernels->cfft_pass(out, step, swiz_N, sign);
}

// ----

static void get_mdct_twiddles(const float **out_tw_re, const float **out_tw_im, size_t N)
{
   const float *base = nullptr;
   if (N == RADAUDIO_LONG_BLOCK_LEN) {
      base = radaudio_fft_impl::s_mdct_long_twiddles;
   } else if (N == RADAUDIO_SHORT_BLOCK_LEN) {
      base = radaudio_fft_impl::s_mdct_short_twiddles;
   } else {
      RR_BREAK();
   }

   *out_tw_re = base;
   *out_tw_im = base + N/2;
}

// Computes N MDCT coeffs from 2N input values signal0:signal1
// needs N floats worth of aligned workspace at "work"
//
// N must be even, >=4.
void radaudio_mdct_fft(radaudio_cpu_features cpu, float *mdct_coef, size_t N, float const *signal0, float const *signal1, float *work)
{
   size_t M1 = N>>2;
   size_t M2 = N>>1;

   // Pre-pass turns signal0/signal1 into complex input to FFT (in w0c = mdct_coef)
   // then FFT w0c -> w1c
   // then post-pass w1c -> mdct_coef

   // The high-level reduction is MDCT -> DCT-IV -> FFT, the latter using the approach described
   // in R. Gluth, "Regular FFT-related transform kernels for DCT/DST-based polyphase filter banks" (1991)
   //
   // Consider the input signal evenly partitioned into 4 parts [a b c d]
   // to compute a MDCT, compute the DCT-IV of x = [-c^R-d a-b^R] (where ^R denotes sequence reversal)
   //
   // Then compute the N-coeff DCT-IV using a N/2-coeff complex FFT as follows: (Matlab for a column vector x)
   //   nrows = size(x,1);
   //   if mod(nrows,2) ~= 0 || nrows<4
   //       error('Only even-sized DCT-IVs of size at least 4 are supported');
   //   end
   //   % Interleave even and reversed odd sub-sequences into complex values
   //   u_prime = x(1:2:end,:) + j*flipud(x(2:2:end,:));
   //   % Calculate twiddles
   //   twiddle = exp(-j*pi/nrows*((0:nrows/2-1)' + 1/8));
   //   % The main calc
   //   f = twiddle .* fft(twiddle .* u_prime);
   //   % Build the result
   //   y = zeros(nrows,size(x,2));
   //   y(1:2:end,:) = real(f);
   //   y(2:2:end,:) = -flipud(imag(f));
   using namespace radaudio_fft_impl;

   FftKernelSet const * kernels = choose_kernels(cpu);

   const float *tw_re = nullptr;
   const float *tw_im = nullptr;
   get_mdct_twiddles(&tw_re, &tw_im, N);

   // Pre-pass
   for (size_t i = 0; i < M1; i++)
   {
      size_t j = M2 - 1 - i;
      size_t k = i*2;

      size_t is = burst_swizzle(i);
      size_t js = burst_swizzle(j);

      float w0re = tw_re[i];
      float w0im = tw_im[i];
      float w1re = tw_re[j];
      float w1im = tw_im[j];

      float re0 = signal1[M2-k-1] + signal1[M2+k+0];
      float im0 = signal0[M2+k+0] - signal0[M2-k-1];
      float re1 = signal0[M2+k+1] - signal0[M2-k-2];
      float im1 = signal1[M2-k-2] + signal1[M2+k+1];

      mdct_coef[is + 0*kBurstSize] = w0re*re0 - w0im*im0;
      mdct_coef[is + 1*kBurstSize] = w0re*im0 + w0im*re0;
      mdct_coef[js + 0*kBurstSize] = w1re*re1 - w1im*im1;
      mdct_coef[js + 1*kBurstSize] = w1re*im1 + w1im*re1;
   }

   // Size-N/2 complex FFT
   radaudio_fft(work, mdct_coef, M2, FftSign_Negative, kernels);

   // Post-pass
   for (size_t i = 0; i < M2; i++)
   {
      size_t is = burst_swizzle(i);
      size_t j = i*2;

      float wre = tw_re[i];
      float wim = tw_im[i];
      float re = work[is + 0*kBurstSize];
      float im = work[is + 1*kBurstSize];

      mdct_coef[j]     = wim*im - wre*re;
      mdct_coef[N-1-j] = wre*im + wim*re;
   }
}

// Abstractly, computes 2N IMDCT results signal0:signal1 from N input coeffs
// Practically, computes N IMDCT results [--:sig0:sig1:--] from N input coeffs
// and packs them as [sig0:sig1] in signal_both.
//
// needs N floats worth of aligned workspace at "work"
//
// both signal outputs are packed into a single buffer to allow the signal buffer
// to be used as an additional work buffer.
//
// N must be even, >=4.
void radaudio_imdct_fft_only_middle(radaudio_cpu_features cpu, float *signal_both, float *mdct_coef, size_t N)
{
   size_t M2 = N>>1;
   float *signal0 = signal_both;
   float *signal1 = signal_both + M2;

   using namespace radaudio_fft_impl;

   FftKernelSet const * kernels = choose_kernels(cpu);

   const float *tw_re = nullptr;
   const float *tw_im = nullptr;
   get_mdct_twiddles(&tw_re, &tw_im, N);

   // The first step is a DCT-IV, so we start with the interleave/twiddle dance
   // NOTE: since our twiddles are negated, we pick up a -1 scale factor here.
   // This is harmless and gets cancelled out immediately in the post-twiddle.

   kernels->imdct_pre(signal_both, mdct_coef, tw_re, tw_im, N);

   // now mdct_coef = new work buffer
   radaudio_fft(mdct_coef, signal_both, M2, FftSign_Negative, kernels);

   kernels->imdct_post(signal0, signal1, mdct_coef, tw_re, tw_im, N);
}

