// Copyright Epic Games Tools, LLC. All Rights Reserved.
#include "radaudio_common.h"

#ifdef __cplusplus
extern "C" {
#endif

#ifdef RADAUDIO_WRAP
#define WRAPPED_NAME(name) RR_STRING_JOIN(RADAUDIO_WRAP, name##_)

#define radaudio_mdct_fft					WRAPPED_NAME(radaudio_mdct_fft)
#define radaudio_imdct_fft_only_middle    	WRAPPED_NAME(radaudio_imdct_fft_only_middle)
#endif


//void radaudio_mdct_fft(float *mdct_coef, size_t N, float const *signal0, float const *signal1, float *work);
void radaudio_mdct_fft(radaudio_cpu_features cpu, float *mdct_coef, size_t N, float const *signal0, float const *signal1, float *work);

// Abstractly, computes 2N IMDCT results signal0:signal1 from N input coeffs
// Practically, computes N IMDCT results [--:sig0:sig1:--] from N input coeffs
// and packs them as [sig0:sig1] in signal_both.
//
// mdct_coeff[] is overwritten in the process. (It's used internally as workspace.)
//
// both signal outputs are packed into a single buffer to allow the signal buffer
// to be used as an additional work buffer.
//
// N must be even, >=4.
void radaudio_imdct_fft_only_middle(radaudio_cpu_features cpu, float *signal_both, float *mdct_coef, size_t N);

#ifdef __cplusplus
}
#endif
