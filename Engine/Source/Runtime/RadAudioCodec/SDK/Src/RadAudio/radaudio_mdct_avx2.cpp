// Copyright Epic Games Tools, LLC. All Rights Reserved.
/* @cdep pre
   $when($NeedsClangMachineSwitches,
      $addlocalcswitches(-mavx2 -mavx)
   )
   $when($NeedsVCMachineSwitches,
      $addlocalcswitches(/arch:AVX2)
   )   
*/

// Disable FP contraction, which allows compilers to
// automatically turn multiplies followed by additions into fused
// multiply-adds where available. This results in different results when
// FMAs are available (e.g. when compiling on x86 with AVX2 support) than
// when they are not, which breaks our compatibility guarantees.
//
// If you want FMAs you need to write them directly. (And be very
// careful to not introduce mismatches between build targets with
// and without FMA support.)

#if defined(_MSC_VER) && !defined(__clang__)
#pragma fp_contract(off)
#else
#pragma STDC FP_CONTRACT OFF
#endif

#include "radaudio_common.h"
#include "radaudio_mdct_internal.h"
#include "radaudio_mdct_internal.inl"
#include "rrbits.h"

typedef int unused_typedef; // avoid "empty translation unit" warning

#ifdef DO_BUILD_AVX2

#include <immintrin.h>

namespace radaudio_fft_impl {
namespace {
struct ElemF32x8
{
   static constexpr size_t kCount = 8;

   __m256 v;

   ElemF32x8() {}
   explicit ElemF32x8(__m256 x) : v(x) {}

   static ElemF32x8 load(float const* ptr)   { return ElemF32x8(_mm256_loadu_ps(ptr)); }
   void store(float* ptr)                    { _mm256_storeu_ps(ptr, v); }

   ElemF32x8 operator+(ElemF32x8 b) const    { return ElemF32x8(_mm256_add_ps(v, b.v)); }
   ElemF32x8 operator-(ElemF32x8 b) const    { return ElemF32x8(_mm256_sub_ps(v, b.v)); }
   ElemF32x8 operator*(ElemF32x8 b) const    { return ElemF32x8(_mm256_mul_ps(v, b.v)); }
   ElemF32x8 reverse() const                 { return ElemF32x8(_mm256_permutevar8x32_ps(v, _mm256_setr_epi32(7,6,5,4,3,2,1,0))); }

   static ElemF32x8 constant(float t)        { return ElemF32x8(_mm256_set1_ps(t)); }

   static RADFORCEINLINE void radix2_twiddle(
      ElemF32x8& ar, ElemF32x8& ai, ElemF32x8& br, ElemF32x8& bi, ElemF32x8 wr, ElemF32x8 wi
   )
   {
      radix2_twiddle_unfused(ar, ai, br, bi, wr, wi);
   }

   static RADFORCEINLINE void load_deinterleave(ElemF32x8& re, ElemF32x8& im, float const* ptr)
   {
      load_halves_deinterleave(re, im, ptr, ptr + 8);
   }

   static ElemF32x8 load4(float const* ptr)     { return ElemF32x8(_mm256_broadcast_ps((const __m128 *)ptr)); }
   static ElemF32x8 concat(__m128 a, __m128 b)  { return ElemF32x8(_mm256_insertf128_ps(_mm256_castps128_ps256(a), b, 1)); }

   static ElemF32x8 load_halves(float const* ptr_lo, float const* ptr_hi)
   {
      return concat(_mm_loadu_ps(ptr_lo), _mm_loadu_ps(ptr_hi));
   }

   static ElemF32x8 negate_upper(ElemF32x8 x)
   {
      return ElemF32x8(_mm256_xor_ps(x.v, _mm256_setr_ps(0.0f, 0.0f, 0.0f, 0.0f, -0.0f, -0.0f, -0.0f, -0.0f)));
   }

   static void initial_radix2_core_vec8(float *out, size_t N, FftSign sign)
   {
      using T = ElemF32x8;

      // Load the twiddles
      // Set up so we have the original twiddle in the lower half, and negated in the upper half
      // because we directly compute a +- t * b in one step
      T const wi = negate_upper(load4(&s_fft_twiddles[(sign == FftSign_Positive) ? 8 : 8+4]));
      T const wr = negate_upper(load4(&s_fft_twiddles[8 + 2]));

      size_t const swiz_dec = burst_swizzle(~size_t(7));
      size_t const swiz_N = burst_swizzle(N);

      // Probably want to combine this with the bit reverse.
      for (size_t j = 0; j < swiz_N;)
      {
         // Broadcast 2 copies each of the real/imag parts
         T ar = T::load4(&out[j + 0*kBurstSize]);
         T ai = T::load4(&out[j + 1*kBurstSize]);
         T br = T::load4(&out[j + 0*kBurstSize + 4]);
         T bi = T::load4(&out[j + 1*kBurstSize + 4]);

         // Twiddle and sum
         T btr = br*wr - bi*wi;
         T bti = bi*wr + br*wi;
         T abr = ar + btr;
         T abi = ai + bti;

         // Store
         abr.store(&out[j + 0*kBurstSize]);
         abi.store(&out[j + 1*kBurstSize]);

         j = (j - swiz_dec) & swiz_dec;
      }
   }

   static void initial_radix4_core_vec8(float *out, size_t N, FftSign sign)
   {
      using T = ElemF32x8;

      // Load the twiddles
      T w0i = negate_upper(load4(&s_fft_twiddles[(sign == FftSign_Positive) ? 8 : 8+4]));
      T w0r = negate_upper(load4(&s_fft_twiddles[8 + 2]));

      // Our regular kernels implicitly determine the third twiddle
      //   w2 = i*w1 = (-w1i) + i*w1r
      // and fold that into the math to avoid any explicit negations etc.
      // But in this pass we need it explicitly, so build it explicitly.
      __m128 w1i_lo = _mm_loadu_ps(&s_fft_twiddles[(sign == FftSign_Positive) ? 16 : 16+8]);
      __m128 w1r_lo = _mm_loadu_ps(&s_fft_twiddles[16 + 4]);
      T w1r, w1i;

      if (sign == FftSign_Positive)
      {
         w1r = negate_upper(concat(w1r_lo, w1i_lo));
         w1i = concat(w1i_lo, w1r_lo);
      }
      else
      {
         // Negative sign has w2 = -i*w1 so need to negate both real and imag parts
         w1r = concat(w1r_lo, w1i_lo);
         w1i = negate_upper(concat(w1i_lo, w1r_lo));
      }

      // Set up the pointers
      size_t const step = 4;
      size_t const swiz_dec = burst_swizzle(~size_t(15));
      size_t const cd_step = burst_swizzle(2 * step);
      size_t const swiz_N = burst_swizzle(N);

      for (size_t j = 0; j < swiz_N;)
      {
         // Broadcast 2 copies each of the real/imag parts
         T ar = T::load4(&out[j + 0*kBurstSize]);
         T ai = T::load4(&out[j + 1*kBurstSize]);
         T br = T::load4(&out[j + 0*kBurstSize + 4]);
         T bi = T::load4(&out[j + 1*kBurstSize + 4]);
         T cr = T::load4(&out[j + 0*kBurstSize + cd_step]);
         T ci = T::load4(&out[j + 1*kBurstSize + cd_step]);
         T dr = T::load4(&out[j + 0*kBurstSize + cd_step + 4]);
         T di = T::load4(&out[j + 1*kBurstSize + cd_step + 4]);

         // Compute first-stage results by direct mul
         T btr = br*w0r - bi*w0i;
         T bti = bi*w0r + br*w0i;
         T abr = ar + btr;
         T abi = ai + bti;

         T dtr = dr*w0r - di*w0i;
         T dti = di*w0r + dr*w0i;
         T cdr = cr + dtr;
         T cdi = ci + dti;

         // Second stage butterflies and store
         radix2_twiddle(abr, abi, cdr, cdi, w1r, w1i);

         abr.store(&out[j + 0*kBurstSize]);
         abi.store(&out[j + 1*kBurstSize]);
         cdr.store(&out[j + 0*kBurstSize + cd_step]);
         cdi.store(&out[j + 1*kBurstSize + cd_step]);

         j = (j - swiz_dec) & swiz_dec;
      }
   }

   static RADFORCEINLINE void load_halves_deinterleave(ElemF32x8& re, ElemF32x8& im, float const* ptr_lo, float const* ptr_hi)
   {
      // Load the halves
      __m256 a0 = _mm256_loadu_ps(ptr_lo); // lo.re[0], lo.im[0], ..., lo.re[3], lo.im[3]
      __m256 a1 = _mm256_loadu_ps(ptr_hi); // hi.re[0], hi.im[0], ..., hi.re[3], hi.im[3]

      // Swap the 128-bit pieces around
      __m256 b0 = _mm256_permute2f128_ps(a0, a1, 0x20); // lo.re[0], lo.im[0], lo.re[1], lo.im[1] | hi.re[0], hi.im[0], hi.re[1], hi.im[1]
      __m256 b1 = _mm256_permute2f128_ps(a0, a1, 0x31); // lo.re[2], lo.im[2], lo.re[3], lo.im[3] | hi.re[2], hi.im[2], hi.re[3], hi.im[3]

      // Deinterleave the elements into real/imag parts
      re.v = _mm256_shuffle_ps(b0, b1, _MM_SHUFFLE(2, 0, 2, 0)); // lo.re[0..3] | hi.re[0..3]
      im.v = _mm256_shuffle_ps(b0, b1, _MM_SHUFFLE(3, 1, 3, 1)); // lo.im[0..3] | hi.im[0..3]
   }

   static RADFORCEINLINE void transpose4x4(ElemF32x8& A, ElemF32x8& B, ElemF32x8& C, ElemF32x8 & D)
   {
      // Pass 1
      const __m256 t0 = _mm256_unpacklo_ps(A.v, C.v);
      const __m256 t1 = _mm256_unpacklo_ps(B.v, D.v);
      const __m256 t2 = _mm256_unpackhi_ps(A.v, C.v);
      const __m256 t3 = _mm256_unpackhi_ps(B.v, D.v);

      // Pass 2
      A.v = _mm256_unpacklo_ps(t0, t1);
      B.v = _mm256_unpackhi_ps(t0, t1);
      C.v = _mm256_unpacklo_ps(t2, t3);
      D.v = _mm256_unpackhi_ps(t2, t3);
   }

   static size_t bitrev_initial_radix4(float *out, float const *in, size_t N, FftSign sign)
   {
      size_t Nbits = rrCtz64(N);

      const size_t shift_amt = kMaxFFTLog2 - Nbits;
      const size_t step = N / 4;

      float * outA = out;
      float * outB = out + burst_swizzle(2 * step); // note: 2 not 1 because it's bit-reversed
      float * outC = out + burst_swizzle(1 * step); // note: 1 not 2 because it's bit-reversed
      float * outD = out + burst_swizzle(3 * step);

      using T = ElemF32x8;

      const size_t half_step_sw = burst_swizzle(step / 2);

      float const * inA = in;
      float const * inB = in + burst_swizzle(2 * step); // note: 2 not 1 because it's bit-reversed
      float const * inC = in + burst_swizzle(1 * step); // note: 1 not 2 because it's bit-reversed
      float const * inD = in + burst_swizzle(3 * step);

      // This was originally written for the negative sign variant, but all we need to do
      // to toggle the sign is to swap inC and inD pointers
      if (sign == FftSign_Positive)
         swap(inC, inD);

      // Apply the initial permutation along with the initial radix-4 butterflies
      // (which are special because the twiddles are with +-1 and +-i only, i.e. trivial)
      for (size_t i = 0; i < step; i += 8)
      {
         size_t is = burst_swizzle(i); // dest index
         size_t j = s_bit_reverse[i] >> shift_amt;
         size_t js = burst_swizzle(j); // source index
         size_t jsh = js + half_step_sw;

         T ar = load_halves(&inA[js + 0*kBurstSize], &inA[jsh + 0*kBurstSize]);
         T ai = load_halves(&inA[js + 1*kBurstSize], &inA[jsh + 1*kBurstSize]);
         T br = load_halves(&inB[js + 0*kBurstSize], &inB[jsh + 0*kBurstSize]);
         T bi = load_halves(&inB[js + 1*kBurstSize], &inB[jsh + 1*kBurstSize]);
         T cr = load_halves(&inC[js + 0*kBurstSize], &inC[jsh + 0*kBurstSize]);
         T ci = load_halves(&inC[js + 1*kBurstSize], &inC[jsh + 1*kBurstSize]);
         T dr = load_halves(&inD[js + 0*kBurstSize], &inD[jsh + 0*kBurstSize]);
         T di = load_halves(&inD[js + 1*kBurstSize], &inD[jsh + 1*kBurstSize]);

         // Radix-4 pass
         dft4_bfly_permuted(ar, ai, br, bi, cr, ci, dr, di);

         // Transposes
         transpose4x4(ar, br, cr, dr);
         transpose4x4(ai, bi, ci, di);

         // Output
         ar.store(&outA[is + 0*kBurstSize]);
         ai.store(&outA[is + 1*kBurstSize]);
         br.store(&outB[is + 0*kBurstSize]);
         bi.store(&outB[is + 1*kBurstSize]);
         cr.store(&outC[is + 0*kBurstSize]);
         ci.store(&outC[is + 1*kBurstSize]);
         dr.store(&outD[is + 0*kBurstSize]);
         di.store(&outD[is + 1*kBurstSize]);
      }

      if (Nbits & 1) // r2 fix-up pass with stride 4 follows
      {
         initial_radix2_core_vec8(out, N, sign);
         return 8;
      }
      else // r4 pass follows, but with stride 4 which is special for us
      {
         initial_radix4_core_vec8(out, N, sign);
         return 16;
      }
   }

   static void store_interleaved(float * dest, ElemF32x8 re, ElemF32x8 im)
   {
      // Need to do this interleave in two steps; first within 4-element groups
      __m256 a0 = _mm256_unpacklo_ps(re.v, im.v);
      __m256 a1 = _mm256_unpackhi_ps(re.v, im.v);

      // Then sort them into the right order
      __m256 b0 = _mm256_permute2f128_ps(a0, a1, 0x20);
      __m256 b1 = _mm256_permute2f128_ps(a0, a1, 0x31);

      // Finally, store
      _mm256_storeu_ps(dest + 0, b0);
      _mm256_storeu_ps(dest + 8, b1);
   }
};
} // anon namespace

FftKernelSet const kernels_avx2 =
{
   ElemF32x8::bitrev_initial_radix4,
   burst_r4_fft_single_pass<ElemF32x8>,
   burst_imdct_prefft<ElemF32x8>,
   burst_imdct_postfft<ElemF32x8>,
};

} // namespace radaudio_fft_impl

#endif // DO_BUILD_AVX2

