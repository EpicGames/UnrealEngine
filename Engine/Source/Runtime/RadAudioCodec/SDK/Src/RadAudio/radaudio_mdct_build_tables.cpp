// Copyright Epic Games Tools, LLC. All Rights Reserved.
#include <stdio.h>
#include <stdlib.h>
#include <stdint.h>
#include <string.h>
#include <math.h>
#include "radaudio_common.h"

static uint32_t bitreverse32(uint32_t n)
{
   n = ((n & 0xAAAAAAAA) >>  1) | ((n & 0x55555555) <<  1);
   n = ((n & 0xCCCCCCCC) >>  2) | ((n & 0x33333333) <<  2);
   n = ((n & 0xF0F0F0F0) >>  4) | ((n & 0x0F0F0F0F) <<  4);
   n = ((n & 0xFF00FF00) >>  8) | ((n & 0x00FF00FF) <<  8);
   return (n >> 16) | (n << 16);
}

static void compute_fft_twiddles(float *twiddle_table, int max_log2)
{
   memset(twiddle_table, 0, (2 << max_log2) * sizeof(float));

   for (size_t lev = 0; lev <= max_log2; lev++) {
      const size_t N = size_t(1) << lev;
      const double step = 2.0 * RADAUDIO_PI / (double)N;

      float *twiddle = twiddle_table + N;
      for (size_t k = 0; k < N; k++)
         twiddle[k] = (float)sin(step * (double)k);
   }
}

static void compute_mdct_twiddles(float *twiddles, int n)
{
   int m = n/2;
   double step = -RADAUDIO_PI / n;
   double scale = -sqrt(sqrt(2.0 / n));
   for (int i=0; i < m; ++i) {
      double phase = step * ((double)i + 0.125);
      twiddles[i+0*m] = (float) (scale * cos(phase));
      twiddles[i+1*m] = (float) (scale * sin(phase));
   }
}

static void radaudio_compute_fft_tables()
{
   const int max_log2 = 9;
   const int maxn = 1 << max_log2;
   const int twiddle_len = 2 * maxn;
   float fft_twiddles[twiddle_len];

   compute_fft_twiddles(fft_twiddles, max_log2);
   printf("FFT_ALIGN(float, s_fft_twiddles[%d]) = {\n", twiddle_len);
   for (int i = 0; i < twiddle_len; ++i) {
      if (i % 8 == 0) printf("   ");
      printf("%15.6a, ", fft_twiddles[i]);
      if (i % 8 == 7) printf("\n");
   }
   printf("};\n\n");

   float mdct_twiddles[RADAUDIO_LONG_BLOCK_LEN];
   for (int j = 0; j < 2; ++j) {
      int n = (j == 0) ? RADAUDIO_SHORT_BLOCK_LEN : RADAUDIO_LONG_BLOCK_LEN;
      const char *label = (j == 0) ? "short" : "long";
      printf("FFT_ALIGN(float, s_mdct_%s_twiddles[%d]) = {\n", label, n);
      compute_mdct_twiddles(mdct_twiddles, n);
      for (int i = 0; i < n; ++i) {
         if (i % 8 == 0) printf("   ");
         printf("%15.6a, ", mdct_twiddles[i]);
         if (i % 8 == 7) printf("\n");
      }
      printf("};\n\n");
   }

   printf("FFTIndex s_bit_reverse[%d] = {\n", maxn);
   for (int i = 0; i < maxn; ++i) {
      if (i % 16 == 0) printf("   ");
      printf("%3d, ", bitreverse32(i) >> (32 - max_log2));
      if (i % 16 == 15) printf("\n");
   }
   printf("};\n\n");
}

int main()
{
   radaudio_compute_fft_tables();
   return 0;
}

/* @cdep pre
   $DefaultsWinEXE64EMT
   $set(NoAutoInclude,1)
*/
// @cdep post $BuildWinEXE64EMT(,)
