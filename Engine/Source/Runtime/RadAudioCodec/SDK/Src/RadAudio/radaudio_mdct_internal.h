// Copyright Epic Games Tools, LLC. All Rights Reserved.
#ifndef RADAUDIO_MDCT_INTERNAL_H
#define RADAUDIO_MDCT_INTERNAL_H

#include <stddef.h>
#include "rrCore.h"
#include "radaudio_mdct.h"

enum FftSign
{
   FftSign_Negative = 0,   // Negative exponential (the customary "forward" FFT)
   FftSign_Positive,    // Positive exponential (customary "inverse" FFT)
};


#ifdef RADAUDIO_WRAP
#define WRAPPED_NAME(name) RR_STRING_JOIN(RADAUDIO_WRAP, name##_)

#define radaudio_fft_impl              WRAPPED_NAME(radaudio_fft_impl)

#endif


namespace radaudio_fft_impl {

static size_t const kMaxFFTLog2 = 9;
static size_t const kMaxFFTN = 1 << kMaxFFTLog2; // This is the largest straight FFT we support


// Main twiddle array:
//  sine of N values for N=1,2,4,8,...,kMaxN
//
// First radix2 level uses N/4 twiddles from N/2 twiddle array (so 1/2 the table);
// cosine table is offset by N/8 from it (another 1/4 the table); for inverse
// transform, we just start into the sine table from offset N/4 (halfway into the
// circle). That means we use all of the sine table.
//
// Second radix2 level uses N/4 twiddles from N twiddle array (1/4 the table)
// and has the cosine table offset by 1/4; for an inverse FFT the sine table is
// accessed offset by N/2. In total, we end up using 3/4 of that larger sine table.
extern FFT_ALIGN(float, s_fft_twiddles[kMaxFFTN * 2]);

// A size-N MDCT/IMDCT needs N/2 complex twiddles, which are just scaled complex
// exponentials. We store first all the real parts then all the imaginary parts.
extern FFT_ALIGN(float, s_mdct_long_twiddles[RADAUDIO_LONG_BLOCK_LEN]);
extern FFT_ALIGN(float, s_mdct_short_twiddles[RADAUDIO_SHORT_BLOCK_LEN]);

static_assert(kMaxFFTN <= 65536, "Have to increase FFTIndex size");
typedef U16 FFTIndex;

extern FFTIndex s_bit_reverse[kMaxFFTN];

// Burst-layout radix 2 FFTs
// First kBurstSize real values, then kBurstSize corresponding imaginary values, then kBurstSize reals again, and so forth.
static size_t constexpr kBurstSize = 16;
static size_t constexpr kBurstMask = kBurstSize - 1;

static constexpr inline size_t burst_swizzle(size_t i)
{
   // logically, this is:
   //  return (i & kBurstMask) + ((i & ~kBurstMask) << 1);
   // but note that i = (i & kBurstMask) + (i & ~kBurstMask)
   // and ((i & ~kBurstMask) << 1) == (i & ~kBurstMask) + (i & ~kBurstMask)
   // therefore:
   return i + (i & ~kBurstMask);
}

// Bit reverse and initial passes, returns stride of next pass to run
typedef size_t InitialPassesKernel(float * out, float const * in, size_t N, FftSign sign);
// Complex FFT radix-4 (actually 2^2) pass
typedef void CFftKernel(float * out, size_t step, size_t swiz_N, FftSign sign);
// IMDCT pre-FFT pass
typedef void ImdctPreFftKernel(float * dest, float const * coeffs, float const * tw_re, float const * tw_im, size_t N);
// IMDCT post-FFT pass
typedef void ImdctPostFftKernel(float * signal0, float * signal1, float const * dft, float const * tw_re, float const * tw_im, size_t N);

struct FftKernelSet
{
   InitialPassesKernel * initial;      // Initial bit reverse and first few passes
   CFftKernel * cfft_pass;             // Complex FFT passes
   ImdctPreFftKernel * imdct_pre;      // Pre-FFT IMDCT kernel
   ImdctPostFftKernel * imdct_post;    // Post-FFT IMDCT kernel
};

// available kernel sets
extern FftKernelSet const kernels_scalar;
extern FftKernelSet const kernels_sse2;
extern FftKernelSet const kernels_avx2;
extern FftKernelSet const kernels_neon;

} // namespace radaudio_fft_impl

#endif // RADAUDIO_MDCT_INTERNAL_H

