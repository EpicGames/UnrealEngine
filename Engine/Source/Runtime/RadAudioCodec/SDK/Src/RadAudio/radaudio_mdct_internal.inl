// Copyright Epic Games Tools, LLC. All Rights Reserved.
#include "rrbits.h"

namespace radaudio_fft_impl {

template<typename T>
static RADFORCEINLINE void swap(T& a, T& b)
{
   T t { a };
   a = b;
   b = t;
}

// Pure radix2 butterfly
// a' = a + b
// b' = a - b
template<typename T>
static RADFORCEINLINE void radix2_bfly(T& ar, T& ai, T& br, T& bi)
{
   T in_ar = ar;
   T in_ai = ai;

   ar = in_ar + br;
   ai = in_ai + bi;
   br = in_ar - br;
   bi = in_ai - bi;
}

// Twiddle bt = b * w
// a' = a + bt
// b' = a - bt
template<typename T>
static RADFORCEINLINE void radix2_twiddle_unfused(T& ar, T& ai, T& br, T& bi, T wr, T wi)
{
   T btr = br*wr - bi*wi;
   T bti = bi*wr + br*wi;
   T in_ar = ar;
   T in_ai = ai;

   ar = in_ar + btr;
   ai = in_ai + bti;
   br = in_ar - btr;
   bi = in_ai - bti;
}

// Permuted DFT4 butterfly for use in radix-2 decompositions
template<typename T>
static RADFORCEINLINE void dft4_bfly_permuted(T& ar, T& ai, T& br, T& bi, T& cr, T& ci, T& dr, T& di)
{
   // First pass of r2 butterflies
   radix2_bfly(ar, ai, br, bi);
   radix2_bfly(cr, ci, dr, di);
   
   // Second pass / output stage
   radix2_bfly(ar, ai, cr, ci);

   // need to work with twiddled dt = -i * d = -i * (dr + i*di) = di - i*dr
   // so write final bfly "by hand"
   T in_br = br;
   T in_bi = bi;
   T in_dr = dr;
   T in_di = di;
   br = in_br + in_di;
   bi = in_bi - in_dr;
   dr = in_br - in_di;
   di = in_bi + in_dr;
}

// Default bit reverse alg uses our LUT
struct DefaultBitReverse
{
	size_t shift_amt;

	DefaultBitReverse(size_t fft_nbits)
		: shift_amt(kMaxFFTLog2 - fft_nbits)
	{
	}

	size_t operator()(size_t i) const
	{
		return s_bit_reverse[i] >> shift_amt;
	}
};

template<typename T, typename TBitReverse=DefaultBitReverse>
static size_t bitrev_initial_radix4_core_vec4(float *out, float const *in, size_t N, FftSign sign)
{
   static_assert(kBurstSize >= 4, "This code requires a burst size of at least 4.");

   size_t Nbits = rrCtz64(N);
   TBitReverse bit_reverse(Nbits);

   size_t step = N / 4;
   float * outA = out;
   float * outB = out + burst_swizzle(2 * step); // note: 2 not 1 because it's bit-reversed
   float * outC = out + burst_swizzle(1 * step); // note: 1 not 2 because it's bit-reversed
   float * outD = out + burst_swizzle(3 * step);

   float const * inA = in;
   float const * inB = in + burst_swizzle(2 * step); // note: 2 not 1 because it's bit-reversed
   float const * inC = in + burst_swizzle(1 * step); // note: 1 not 2 because it's bit-reversed
   float const * inD = in + burst_swizzle(3 * step);

   // This was originally written for the negative sign variant, but all we need to do
   // to toggle the sign is to swap inC and inD pointers
   if (sign == FftSign_Positive)
      swap(inC, inD);

   // Apply the initial permutation along with the initial radix-4 butterflies
   // (which are special because the twiddles are with +-1 and +-i only, i.e. trivial)
   for (size_t i = 0; i < step; i += 4)
   {
      size_t is = burst_swizzle(i); // dest index
	   size_t j = bit_reverse(i);
      size_t js = burst_swizzle(j); // source index

      T ar = T::load(&inA[js + 0*kBurstSize]);
      T ai = T::load(&inA[js + 1*kBurstSize]);
      T br = T::load(&inB[js + 0*kBurstSize]);
      T bi = T::load(&inB[js + 1*kBurstSize]);
      T cr = T::load(&inC[js + 0*kBurstSize]);
      T ci = T::load(&inC[js + 1*kBurstSize]);
      T dr = T::load(&inD[js + 0*kBurstSize]);
      T di = T::load(&inD[js + 1*kBurstSize]);

      // Radix-4 pass
      dft4_bfly_permuted(ar, ai, br, bi, cr, ci, dr, di);

      // Transposes
      T::transpose4x4(ar, br, cr, dr);
      T::transpose4x4(ai, bi, ci, di);

      // Output
      ar.store(&outA[is + 0*kBurstSize]);
      ai.store(&outA[is + 1*kBurstSize]);
      br.store(&outB[is + 0*kBurstSize]);
      bi.store(&outB[is + 1*kBurstSize]);
      cr.store(&outC[is + 0*kBurstSize]);
      ci.store(&outC[is + 1*kBurstSize]);
      dr.store(&outD[is + 0*kBurstSize]);
      di.store(&outD[is + 1*kBurstSize]);
   }

   return Nbits;
}

template<typename T>
static void initial_radix2_core_vec4(float * out, size_t N, FftSign sign)
{
   size_t const swiz_N = burst_swizzle(N);
   T wr = T::load(&s_fft_twiddles[8 + 2]);
   T wi = T::load(&s_fft_twiddles[((sign == FftSign_Positive) ? 8 : 8 + 4)]);
   
   float * outA = out;
   float * outB = out + burst_swizzle(4);
   size_t swiz_dec = burst_swizzle(~size_t(7));

   for (size_t j = 0; j < swiz_N; j = (j - swiz_dec) & swiz_dec)
   {
      T ar = T::load(&outA[j + 0*kBurstSize]);
      T ai = T::load(&outA[j + 1*kBurstSize]);
      T br = T::load(&outB[j + 0*kBurstSize]);
      T bi = T::load(&outB[j + 1*kBurstSize]);

      T::radix2_twiddle(ar, ai, br, bi, wr, wi);

      ar.store(&outA[j + 0*kBurstSize]);
      ai.store(&outA[j + 1*kBurstSize]);
      br.store(&outB[j + 0*kBurstSize]);
      bi.store(&outB[j + 1*kBurstSize]);
   }
}

// This is initial_radix4 + initial_radix2 in one fused pass, for archs
// with plenty of regs.
template<typename T, typename TBitReverse=DefaultBitReverse>
static size_t bitrev_initial_radix8_core_vec4(float *out, float const *in, size_t N, FftSign sign)
{
   static_assert(kBurstSize >= 8, "This code requires a burst size of at least 8.");

   size_t Nbits = rrCtz64(N);
   TBitReverse bit_reverse(Nbits);

   // Twiddles for final radix2
   T wr = T::load(&s_fft_twiddles[8 + 2]);
   T wi = T::load(&s_fft_twiddles[((sign == FftSign_Positive) ? 8 : 8 + 4)]);

   size_t step = N / 4;
   float * outA = out;
   float * outB = out + burst_swizzle(2 * step); // note: 2 not 1 because it's bit-reversed
   float * outC = out + burst_swizzle(1 * step); // note: 1 not 2 because it's bit-reversed
   float * outD = out + burst_swizzle(3 * step);

   size_t group1_offs = burst_swizzle(step / 2); // offset into group 1 (=where the second vec of 4 ends up)

   float const * inA = in;
   float const * inB = in + burst_swizzle(2 * step); // note: 2 not 1 because it's bit-reversed
   float const * inC = in + burst_swizzle(1 * step); // note: 1 not 2 because it's bit-reversed
   float const * inD = in + burst_swizzle(3 * step);

   // This was originally written for the negative sign variant, but all we need to do
   // to toggle the sign is to swap inC and inD pointers
   if (sign == FftSign_Positive)
      swap(inC, inD);

   // Apply the initial permutation along with the initial radix-4 butterflies
   // (which are special because the twiddles are with +-1 and +-i only, i.e. trivial)
   for (size_t i = 0; i < step; i += 8)
   {
      // First group of 4
      size_t j0s = burst_swizzle(bit_reverse(i)); // source index 0
      T a0r = T::load(&inA[j0s + 0*kBurstSize]);
      T a0i = T::load(&inA[j0s + 1*kBurstSize]);
      T b0r = T::load(&inB[j0s + 0*kBurstSize]);
      T b0i = T::load(&inB[j0s + 1*kBurstSize]);
      T c0r = T::load(&inC[j0s + 0*kBurstSize]);
      T c0i = T::load(&inC[j0s + 1*kBurstSize]);
      T d0r = T::load(&inD[j0s + 0*kBurstSize]);
      T d0i = T::load(&inD[j0s + 1*kBurstSize]);

      // Initial radix-4 pass
      dft4_bfly_permuted(a0r, a0i, b0r, b0i, c0r, c0i, d0r, d0i);

      // Transposes
      T::transpose4x4(a0r, b0r, c0r, d0r);
      T::transpose4x4(a0i, b0i, c0i, d0i);

      // Second group of 4
      size_t j1s = j0s + group1_offs; // == burst_swizzle(bit_reverse(i + 4))
      T a1r = T::load(&inA[j1s + 0*kBurstSize]);
      T a1i = T::load(&inA[j1s + 1*kBurstSize]);
      T b1r = T::load(&inB[j1s + 0*kBurstSize]);
      T b1i = T::load(&inB[j1s + 1*kBurstSize]);
      T c1r = T::load(&inC[j1s + 0*kBurstSize]);
      T c1i = T::load(&inC[j1s + 1*kBurstSize]);
      T d1r = T::load(&inD[j1s + 0*kBurstSize]);
      T d1i = T::load(&inD[j1s + 1*kBurstSize]);

      // Initial radix-4 pass
      dft4_bfly_permuted(a1r, a1i, b1r, b1i, c1r, c1i, d1r, d1i);

      // Transposes
      T::transpose4x4(a1r, b1r, c1r, d1r);
      T::transpose4x4(a1i, b1i, c1i, d1i);

      // Final radix-2 pass between groups and output
      size_t is = burst_swizzle(i); // dest index

      T::radix2_twiddle(a0r, a0i, a1r, a1i, wr, wi);
      a0r.store(&outA[is + 0*kBurstSize]);
      a1r.store(&outA[is + 0*kBurstSize + 4]);
      a0i.store(&outA[is + 1*kBurstSize]);
      a1i.store(&outA[is + 1*kBurstSize + 4]);

      T::radix2_twiddle(b0r, b0i, b1r, b1i, wr, wi);
      b0r.store(&outB[is + 0*kBurstSize]);
      b1r.store(&outB[is + 0*kBurstSize + 4]);
      b0i.store(&outB[is + 1*kBurstSize]);
      b1i.store(&outB[is + 1*kBurstSize + 4]);

      T::radix2_twiddle(c0r, c0i, c1r, c1i, wr, wi);
      c0r.store(&outC[is + 0*kBurstSize]);
      c1r.store(&outC[is + 0*kBurstSize + 4]);
      c0i.store(&outC[is + 1*kBurstSize]);
      c1i.store(&outC[is + 1*kBurstSize + 4]);

      T::radix2_twiddle(d0r, d0i, d1r, d1i, wr, wi);
      d0r.store(&outD[is + 0*kBurstSize]);
      d1r.store(&outD[is + 0*kBurstSize + 4]);
      d0i.store(&outD[is + 1*kBurstSize]);
      d1i.store(&outD[is + 1*kBurstSize + 4]);
   }

   return Nbits;
}

template<typename T>
static void burst_r4_fft_single_pass(float * out, size_t step, size_t swiz_N, FftSign sign)
{
   float const *twiddle1_i = s_fft_twiddles + step*2;
   float const *twiddle1_r = twiddle1_i + step/2;

   float const *twiddle2_i = s_fft_twiddles + step*4;
   float const *twiddle2_r = twiddle2_i + step;

   // NOTE: this doesn't work unless step >= T::kCount
   // i.e. our initial "regular" level is determined by vector width
   const size_t twiddle_mask = step - 1;

   size_t swiz_dec = burst_swizzle(~((3 * step) | (T::kCount - 1)));
   float * outA = out;
   float * outB = out + burst_swizzle(1 * step);
   float * outC = out + burst_swizzle(2 * step);
   float * outD = out + burst_swizzle(3 * step);

   // Defaults to B/D swapped; see below
   float * outWrB = outD;
   float * outWrD = outB;

   // Advance in sine table by half the phase if negative sign requested
   // Also swap B/D outputs which effectively swaps our twiddle from +i to -i, see below.
   if (sign == FftSign_Negative)
   {
      twiddle1_i += step;
      twiddle2_i += step*2;
      swap(outWrB, outWrD);
   }

   // This is actually radix 2^2, not radix 4.
   for (size_t j = 0, k = 0; j < swiz_N; )
   {
      T ar = T::load(&outA[j + 0*kBurstSize]);
      T ai = T::load(&outA[j + 1*kBurstSize]);
      T br = T::load(&outB[j + 0*kBurstSize]);
      T bi = T::load(&outB[j + 1*kBurstSize]);
      T cr = T::load(&outC[j + 0*kBurstSize]);
      T ci = T::load(&outC[j + 1*kBurstSize]);
      T dr = T::load(&outD[j + 0*kBurstSize]);
      T di = T::load(&outD[j + 1*kBurstSize]);

      // First stage twiddle b * w0, d * w0
      T w0r = T::load(&twiddle1_r[k]);
      T w0i = T::load(&twiddle1_i[k]);

      // First stage butterflies
      T::radix2_twiddle(ar, ai, br, bi, w0r, w0i);
      T::radix2_twiddle(cr, ci, dr, di, w0r, w0i);

      // Second stage twiddle
      T w1r = T::load(&twiddle2_r[k]);
      T w1i = T::load(&twiddle2_i[k]);

      // Second stage butterfly and output
      T::radix2_twiddle(ar, ai, cr, ci, w1r, w1i);
      ar.store(&outA[j + 0*kBurstSize]);
      ai.store(&outA[j + 1*kBurstSize]);
      cr.store(&outC[j + 0*kBurstSize]);
      ci.store(&outC[j + 1*kBurstSize]);

      // w2 is exactly a quarter-circle away
      // i.e. multiply by i. w2 = i*(w1r + i*w1i) = i*w1r - w1i = (-w1i) + i*w1r
      //
      // Note "swap identity" above. We want
      //  b', d' = b +- w2 * d
      //         = b +- i (w1 * d)
      //         = b -+ -i (w1 * d)
      //         = b -+ s(s(w1) s(d))
      // <=>
      //  d', b' = b +- s(s(w1) s(d))
      //         = s(s(b) +- s(w1) s(d))
      //
      // (note d', b' swapped places).
      // Therefore, we compute a regular twiddled r2 butterfly with real/imag parts
      // of b, w1 and d swapped (on both the inputs and outputs). That takes care of
      // the s() applications. Finally for the positive FFT sign we also swap the
      // output B and D pointer (this happens outside). For the negative FFT sign
      // we need to multiply by -i not i to begin with, so we get another swap of
      // output B and D which cancels out the first one.
      T::radix2_twiddle(bi, br, di, dr, w1i, w1r);
      br.store(&outWrB[j + 0*kBurstSize]);
      bi.store(&outWrB[j + 1*kBurstSize]);
      dr.store(&outWrD[j + 0*kBurstSize]);
      di.store(&outWrD[j + 1*kBurstSize]);

      j = (j - swiz_dec) & swiz_dec;
      k = (k + T::kCount) & twiddle_mask;
   }
}

template<typename T>
static void burst_imdct_prefft(float * dest, float const * coeffs, float const * tw_re, float const * tw_im, size_t N)
{
   size_t M1 = N >> 2;
   size_t M2 = N >> 1;

   // Reference version:
   //
   // for (size_t i = 0; i < M2; ++i)
   // {
   //    size_t is = burst_swizzle(i);
   //
   //    float wre = tw_re[i];
   //    float wim = tw_im[i];
   //    float re = coeffs[i*2];
   //    float im = coeffs[N-1-i*2];
   //
   //    dest[is + 0*kBurstSize] = wre*re - wim*im;
   //    dest[is + 1*kBurstSize] = wre*im + wim*re;
   // }
   //
   // Note "re" only reads even elements and "im" only reads odd elements (from the end of the array).
   // This suggests interleaving a forward and backward pass simultaneously so we use all the values
   // in the vectors we load:
   //
   // for (size_t i = 0; i < M1; ++i)
   // {
   //    size_t j = M2 - 1 - i;
   //    size_t is = burst_swizzle(i);
   //    size_t js = burst_swizzle(j);
   //                                                      
   //    // Twiddles
   //    float wr0 = tw_re[i];
   //    float wi0 = tw_im[i];
   //    float wr1 = tw_re[j];
   //    float wi1 = tw_im[j];
   //
   //    // Pairs of coefficients
   //    float re0 = coeffs[i*2];
   //    float im1 = coeffs[i*2 + 1];
   //
   //    float re1 = coeffs[j*2];
   //    float im0 = coeffs[j*2 + 1];
   //
   //    // Do the twiddle
   //    dest[is + 0*kBurstSize] = wr0*re0 - wi0*im0;
   //    dest[is + 1*kBurstSize] = wr0*im0 + wi0*re0;
   //    dest[js + 0*kBurstSize] = wr1*re1 - wi1*im1;
   //    dest[js + 1*kBurstSize] = wr1*im1 + wi1*re1;
   // }
   //
   // This now vectorizes very cleanly with the primitives we have:

   for (size_t i = 0; i < M1; i += T::kCount)
   {
      size_t j = M2 - T::kCount - i;
      size_t is = burst_swizzle(i);
      size_t js = burst_swizzle(j);
                                                        
      // Twiddles
      T wr0 = T::load(&tw_re[i]);
      T wi0 = T::load(&tw_im[i]);
      T wr1 = T::load(&tw_re[j]); // note: should be reversed, but we defer it
      T wi1 = T::load(&tw_im[j]); // note: should be reversed, but we defer it

      // Pairs of coefficients
      T re0, im0, re1, im1;
      T::load_deinterleave(re0, im1, &coeffs[i*2]);
      T::load_deinterleave(re1, im0, &coeffs[j*2]); // note: re1/im0 should be reversed, but we defer it

      // We have several reversals to take care of here. 
      // What we actually want is:
      //
      //   oir = wr0*re0 - wi0*rev(im0);
      //   oii = wr0*rev(im0) + wi0*re0;
      //   ojr = rev(wr1)*rev(re1) - rev(wi1)*im1;
      //   oji = rev(wr1)*im1 + rev(wi1)*rev(re1);
      //
      // and then ojr and oji get stored reversed as well. If we factor that final reversal in, we get:
      //
      //   ojr' = rev(rev(wr1)*rev(re1) - rev(wi1)*im1)
      //        = rev(rev(wr1 * re1) - rev(wi1)*im1)
      //        = wr1 * re1 - wi1*rev(im1)
      //   oji' = rev(rev(wr1)*im1 + rev(wi1)*rev(re1))
      //        = rev(rev(wr1*rev(im1)) + rev(wi1*re1))
      //        = wr1*rev(im1) + wi1*re1
      //
      // in other words, almost all the reversals cancel each other and we end up with mostly in-order
      // operation with just reversals on the two imaginary inputs:
      im0 = im0.reverse(); // NOTE optimize (should be able to fold into the load shuffles)
      im1 = im1.reverse();

      // Do the twiddle
      T oir = wr0*re0 - wi0*im0;
      T oii = wr0*im0 + wi0*re0;
      T ojr = wr1*re1 - wi1*im1;
      T oji = wr1*im1 + wi1*re1;

      oir.store(&dest[is + 0*kBurstSize]);
      oii.store(&dest[is + 1*kBurstSize]);
      ojr.store(&dest[js + 0*kBurstSize]);
      oji.store(&dest[js + 1*kBurstSize]);
   }
}

template<typename T>
static void burst_imdct_postfft(float * signal0, float * signal1, float const * dft, float const * tw_re, float const * tw_im, size_t N)
{
   size_t M1 = N >> 2;
   size_t M2 = N >> 1;

   for (size_t i = 0; i < M1; i += T::kCount)
   {
      size_t j = M2 - T::kCount - i; // reversed index
      size_t is = burst_swizzle(i);
      size_t js = burst_swizzle(j);

      // Twiddles
      T w0re = T::load(&tw_re[i]);
      T w0im = T::load(&tw_im[i]);
      T w1re = T::load(&tw_re[j]); // note: should be reversed, but we defer it
      T w1im = T::load(&tw_im[j]); // note: should be reversed, but we defer it

      // FFT outputs
      T re0 = T::load(&dft[is + 0*kBurstSize]);
      T im0 = T::load(&dft[is + 1*kBurstSize]);
      T re1 = T::load(&dft[js + 0*kBurstSize]); // note: should be reversed, but we defer it
      T im1 = T::load(&dft[js + 1*kBurstSize]); // note: should be reversed, but we defer it

      // We have several reversals to take care of here.
      // w1re, w1im, re1, im1 should all get reversed on use,
      // and the outputs out1e and out1o also get stored in reversed
      // form.
      //
      // We start out with
      //
      //   out0e = w0re*im0 + w0im*re0
      //   out0o = rev(w1im)*rev(im1) - rev(w1re)*rev(re1)
      //   out1e = rev(w1re)*rev(im1) + rev(w1im)*rev(re1)
      //   out1o = w0im*im0 - w0re*re0
      //
      // and we finally want out1e' = rev(out1e), out1o' = rev(out1o).
      // Simplifying that, we get:
      //
      //   out0e  = w0re*im0 + w0im*re0
      //   out0o  = rev(w1im*im1 - w1re*re1)
      //   out1e' = w1re*im1 + w1im*re1
      //   out1o' = rev(w0im*im0 - w0re*re0)

      // Simplified twiddles (see above)
      T out0e = w0re*im0 + w0im*re0;
      T out0o = (w1im*im1 - w1re*re1).reverse();
      T out1e = w1re*im1 + w1im*re1;
      T out1o = (w0im*im0 - w0re*re0).reverse();

      // Write outputs
      T::store_interleaved(&signal0[i*2], out0e, out0o);
      T::store_interleaved(&signal1[M2 - T::kCount*2 - i*2], out1e, out1o);
   }
}

} // namespace radaudio_fft_impl

