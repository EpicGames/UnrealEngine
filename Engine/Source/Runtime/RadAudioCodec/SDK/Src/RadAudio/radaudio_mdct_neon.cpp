// Copyright Epic Games Tools, LLC. All Rights Reserved.
#include "radaudio_common.h"
#include "radaudio_mdct_internal.h"
#include "radaudio_mdct_internal.inl"
#include "rrbits.h"

typedef int unused_typedef; // avoid "empty translation unit" warning

#ifdef DO_BUILD_NEON

#include <arm_neon.h>

namespace radaudio_fft_impl {

#if defined(__clang__)

#ifdef __RAD64__
#define BITREVERSE_WIDTH      64
#define BITREVERSE_BUILTIN(x)	__builtin_bitreverse64(x)
#else
#define BITREVERSE_WIDTH      32
#define BITREVERSE_BUILTIN(x)	__builtin_bitreverse32(x)
#endif

// Can use the ARM RBIT instruction and skip the bit reverse LUT
struct NeonBitReverse
{
   size_t shift_amt;

   NeonBitReverse(size_t fft_nbits)
      : shift_amt(BITREVERSE_WIDTH - fft_nbits)
   {
   }

   size_t operator()(size_t i) const
   {
      return BITREVERSE_BUILTIN(i) >> shift_amt;
   }
};

#else

typedef DefaultBitReverse NeonBitReverse;

#endif

struct ElemF32x4
{
   static constexpr size_t kCount = 4;

   float32x4_t v;

   ElemF32x4() {}
   explicit ElemF32x4(float32x4_t x) : v(x) {}

   static ElemF32x4 load(float const* ptr)   { return ElemF32x4(vld1q_f32(ptr)); }
   static ElemF32x4 loadu(float const* ptr)  { return ElemF32x4(vld1q_f32(ptr)); }
   void store(float* ptr)                    { vst1q_f32(ptr, v); }

   ElemF32x4 operator+(ElemF32x4 b) const    { return ElemF32x4(vaddq_f32(v, b.v)); }
   ElemF32x4 operator-(ElemF32x4 b) const    { return ElemF32x4(vsubq_f32(v, b.v)); }
   ElemF32x4 operator*(ElemF32x4 b) const    { return ElemF32x4(vmulq_f32(v, b.v)); }
   ElemF32x4 reverse() const
   {
      float32x4_t r = vrev64q_f32(v);
      return ElemF32x4(vcombine_f32(vget_high_f32(r), vget_low_f32(r)));
   }

   static RADFORCEINLINE void radix2_twiddle(
      ElemF32x4& ar, ElemF32x4& ai, ElemF32x4& br, ElemF32x4& bi, ElemF32x4 wr, ElemF32x4 wi
   )
   {
      radix2_twiddle_unfused(ar, ai, br, bi, wr, wi);
   }

   static RADFORCEINLINE void load_deinterleave(ElemF32x4& re, ElemF32x4& im, float const* ptr)
   {
      float32x4x2_t pair = vld2q_f32(ptr);
      re.v = pair.val[0];
      im.v = pair.val[1];
   }

   static RADFORCEINLINE void transpose4x4(ElemF32x4& A, ElemF32x4& B, ElemF32x4& C, ElemF32x4 & D)
   {
      // Pass 1
      const float32x4x2_t t0 = vzipq_f32(A.v, C.v);
      const float32x4x2_t t1 = vzipq_f32(B.v, D.v);

      // Pass 2
      const float32x4x2_t t2 = vzipq_f32(t0.val[0], t1.val[0]);
      const float32x4x2_t t3 = vzipq_f32(t0.val[1], t1.val[1]);

      A.v = t2.val[0];
      B.v = t2.val[1];
      C.v = t3.val[0];
      D.v = t3.val[1];
   }

   static size_t bitrev_initial_radix4(float * out, float const * in, size_t N, FftSign sign)
   {
      size_t const Nbits = rrCtz64(N);

      if (Nbits & 1)
      {
         bitrev_initial_radix8_core_vec4<ElemF32x4, NeonBitReverse>(out, in, N, sign);
         return 8;
      }
      else
      {
         bitrev_initial_radix4_core_vec4<ElemF32x4, NeonBitReverse>(out, in, N, sign);
         return 4;
      }
   }

   static void store_interleaved(float * dest, ElemF32x4 re, ElemF32x4 im)
   {
      float32x4x2_t pair = { re.v, im.v };
      vst2q_f32(dest, pair);
   }
};

FftKernelSet const kernels_neon =
{
   ElemF32x4::bitrev_initial_radix4,
   burst_r4_fft_single_pass<ElemF32x4>,
   burst_imdct_prefft<ElemF32x4>,
   burst_imdct_postfft<ElemF32x4>,
};

} // namespace radaudio_fft_impl

#endif // DO_BUILD_SSE4

