// Copyright Epic Games Tools, LLC. All Rights Reserved.
#include "radaudio_common.h"
#include "radaudio_mdct_internal.h"
#include "radaudio_mdct_internal.inl"
#include "rrbits.h"

typedef int unused_typedef; // avoid "empty translation unit" warning

#ifdef DO_BUILD_SSE4

#include <emmintrin.h>

namespace radaudio_fft_impl {
namespace {
struct ElemF32x4
{
   static constexpr size_t kCount = 4;

   __m128 v;

   ElemF32x4() {}
   explicit ElemF32x4(__m128 x) : v(x) {}

   static ElemF32x4 load(float const* ptr)   { return ElemF32x4(_mm_loadu_ps(ptr)); }
   void store(float* ptr)                    { _mm_storeu_ps(ptr, v); }

   ElemF32x4 operator+(ElemF32x4 b) const    { return ElemF32x4(_mm_add_ps(v, b.v)); }
   ElemF32x4 operator-(ElemF32x4 b) const    { return ElemF32x4(_mm_sub_ps(v, b.v)); }
   ElemF32x4 operator*(ElemF32x4 b) const    { return ElemF32x4(_mm_mul_ps(v, b.v)); }
   ElemF32x4 reverse() const                 { return ElemF32x4(_mm_shuffle_ps(v, v, _MM_SHUFFLE(0, 1, 2, 3))); }

   static RADFORCEINLINE void radix2_twiddle(
      ElemF32x4& ar, ElemF32x4& ai, ElemF32x4& br, ElemF32x4& bi, ElemF32x4 wr, ElemF32x4 wi
   )
   {
      radix2_twiddle_unfused(ar, ai, br, bi, wr, wi);
   }

   static RADFORCEINLINE void load_deinterleave(ElemF32x4& re, ElemF32x4& im, float const* ptr)
   {
      // Load pair of vectors
      const __m128 a0 = _mm_load_ps(ptr);
      const __m128 a1 = _mm_load_ps(ptr + 4);

      // Deinterleave to real/imaginary parts
      re.v = _mm_shuffle_ps(a0, a1, _MM_SHUFFLE(2, 0, 2, 0));
      im.v = _mm_shuffle_ps(a0, a1, _MM_SHUFFLE(3, 1, 3, 1));
   }

   static RADFORCEINLINE void transpose4x4(ElemF32x4& A, ElemF32x4& B, ElemF32x4& C, ElemF32x4 & D)
   {
      // Pass 1
      const __m128 t0 = _mm_unpacklo_ps(A.v, C.v);
      const __m128 t1 = _mm_unpacklo_ps(B.v, D.v);
      const __m128 t2 = _mm_unpackhi_ps(A.v, C.v);
      const __m128 t3 = _mm_unpackhi_ps(B.v, D.v);

      // Pass 2
      A.v = _mm_unpacklo_ps(t0, t1);
      B.v = _mm_unpackhi_ps(t0, t1);
      C.v = _mm_unpacklo_ps(t2, t3);
      D.v = _mm_unpackhi_ps(t2, t3);
   }

   static size_t bitrev_initial_radix4(float * out, float const * in, size_t N, FftSign sign)
   {
      size_t const Nbits = bitrev_initial_radix4_core_vec4<ElemF32x4>(out, in, N, sign);

      if (Nbits & 1)
      {
         initial_radix2_core_vec4<ElemF32x4>(out, N, sign);
         return 8;
      }
      else
         return 4;
   }

   static void store_interleaved(float * dest, ElemF32x4 re, ElemF32x4 im)
   {
      __m128 i0 = _mm_unpacklo_ps(re.v, im.v);
      __m128 i1 = _mm_unpackhi_ps(re.v, im.v);
      _mm_store_ps(dest + 0, i0);
      _mm_store_ps(dest + 4, i1);
   }
};
} // anon namespace

FftKernelSet const kernels_sse2 =
{
   ElemF32x4::bitrev_initial_radix4,
   burst_r4_fft_single_pass<ElemF32x4>,
   burst_imdct_prefft<ElemF32x4>,
   burst_imdct_postfft<ElemF32x4>,
};

} // namespace radaudio_fft_impl

#endif // DO_BUILD_SSE4

