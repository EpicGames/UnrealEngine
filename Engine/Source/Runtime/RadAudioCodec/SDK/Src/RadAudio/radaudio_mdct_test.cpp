// Copyright Epic Games Tools, LLC. All Rights Reserved.
#include <stdio.h>
#include <stdlib.h>
#include <stdint.h>
#include <string.h>
#include <math.h>
#include <memory>
#include "radaudio_mdct.h"
#include "radaudio_data_tables.inl"

enum
{
   // x86
   CPUFLAGS_SSE2 = 1<<0,
   CPUFLAGS_AVX2 = 1<<1,

   // ARM (separate)
   CPUFLAGS_NEON = 1<<0
};

struct CpuFlagVariant
{
   const char* name;
   uint32_t flags;
};

static const CpuFlagVariant s_cpu_variants[] =
{
   { "Scalar", 0 },
#if defined(DO_BUILD_SSE4)
   { "SSE2",   CPUFLAGS_SSE2 },
#endif
#if defined(DO_BUILD_AVX2)
   { "AVX2",   CPUFLAGS_SSE2 | CPUFLAGS_AVX2 },
#endif
#if defined(DO_BUILD_NEON)
   { "NEON",   CPUFLAGS_NEON }
#endif
};

static radaudio_cpu_features decode_features(const CpuFlagVariant * variant)
{
   radaudio_cpu_features decoded = {};

#if defined(DO_BUILD_SSE4)
   if (variant->flags & CPUFLAGS_SSE2)
   {
      decoded.has_sse2 = 1;
   }
#endif
#if defined(DO_BUILD_AVX2)
   if (variant->flags & CPUFLAGS_AVX2)
   {
      decoded.has_ssse3 = 1;
      decoded.has_sse4_1 = 1;
      decoded.has_popcnt = 1;
      decoded.has_avx2 = 1;
   }
#endif

   return decoded;
}

// Computes N MDCT coeffs from 2N input values signal0:signal1
// SERIOUSLY SLOW! (for testing only)
//
// for full transform, set n0=0, n1=N-1
static void mdct_ref_sparse(float *mdct_coef, size_t N, float const *signal0, float const *signal1, int n0, int n1)
{
   int Nint = static_cast<int>(N);

   double const kPi = 3.1415926535897932384626433832795;
   double nbias0 = (Nint + 1.0) / 2.0;
   double nbias1 = nbias0 + Nint;
   double mult = kPi/Nint;
   double scale = sqrt(2.0 / Nint);

   for (int k = 0; k < Nint; k++)
   {
      double sum = 0.0;
      double kfactor = mult * (k + 0.5);
      for (int n = n0; n <= n1; n++)
         sum += signal0[n]*cos(kfactor * (n + nbias0)) + signal1[n]*cos(kfactor * (n + nbias1));

      mdct_coef[k] = static_cast<float>(sum * scale);
   }
}

// Computes 2N IMDCT results signal0:signal1 from N input coeffs
// SERIOUSLY SLOW! (for testing only)
//
// for full transform, set k0=0, k1=N-1
static void imdct_ref_sparse(float *signal0, float *signal1, float const *mdct_coef, size_t N, int k0, int k1)
{
   int Nint = static_cast<int>(N);

   double const kPi = 3.1415926535897932384626433832795;
   double nbias0 = (Nint + 1.0) / 2.0;
   double nbias1 = nbias0 + Nint;
   double mult = kPi/Nint;
   double scale = sqrt(2.0 / Nint);

   for (int n = 0; n < Nint; n++)
   {
      double sum0 = 0.0;
      double sum1 = 0.0;
      double nfactor0 = mult * (n + nbias0);
      double nfactor1 = mult * (n + nbias1);
      for (int k = k0; k <= k1; k++)
      {
         double kfactor = k + 0.5;
         sum0 += mdct_coef[k]*cos(nfactor0 * kfactor);
         sum1 += mdct_coef[k]*cos(nfactor1 * kfactor);
      }

      signal0[n] = static_cast<float>(sum0 * scale);
      signal1[n] = static_cast<float>(sum1 * scale);
   }
}

static double norm2(float const *a, size_t N)
{
   double sum_sq = 0.0;
   for (size_t i = 0; i < N; i++)
      sum_sq += (double)a[i] * (double)a[i];

   return sqrt(sum_sq);
}

static double norm2_diff(float const *a, float const *b, size_t N)
{
   double diff_mag_sq = 0.0;
   for (size_t i = 0; i < N; i++)
   {
      double d = (double)a[i] - (double)b[i];
      diff_mag_sq += d * d;
   }

   return sqrt(diff_mag_sq);
}

// *Really* minimal PCG32 code / (c) 2014 M.E. O'Neill / pcg-random.org
// Licensed under Apache License 2.0 (NO WARRANTY, etc. see website)

typedef struct { uint64_t state;  uint64_t inc; } pcg32_random_t;

uint32_t pcg32_random_r(pcg32_random_t* rng)
{
    uint64_t oldstate = rng->state;
    // Advance internal state
    rng->state = oldstate * 6364136223846793005ULL + (rng->inc|1);
    // Calculate output function (XSH RR), uses old state for max ILP
    uint32_t xorshifted = (uint32_t) (((oldstate >> 18u) ^ oldstate) >> 27u);
    uint32_t rot = oldstate >> 59u;
    return (xorshifted >> rot) | (xorshifted << ((0u - rot) & 31));
}

static pcg32_random_t s_pcg_rand; // we never bother seeding this

static float unit_random_float(pcg32_random_t* rng)
{
   uint32_t x = pcg32_random_r(rng);
   return 2.0f * ((x & 0xffffff) / 16777216.0f) - 1.0f; // uniform random in [-1,1]
}

static bool test_mdct(size_t N)
{
   radaudio_cpu_features cpu_none = {};

   if (N > RADAUDIO_LONG_BLOCK_LEN)
      return false;

   RAD_ALIGN(float, signal[RADAUDIO_LONG_BLOCK_LEN * 2 * 3], 64);
   RAD_ALIGN(float, coeffs[RADAUDIO_LONG_BLOCK_LEN * 3], 64);
   RAD_ALIGN(float, workspace[RADAUDIO_LONG_BLOCK_LEN], 64);

   printf("mdct N=%zd:\n", N);
   const int nIters = 100;

   for (int iter = 0; iter < nIters; ++iter)
   {
      // Two random input signals
      for (size_t i = 0; i < N * 2 * 2; ++i)
         signal[i] = unit_random_float(&s_pcg_rand);

      float linear_coeff = 2.0f + 1.75f * unit_random_float(&s_pcg_rand);

      // Linear combination of inputs
      for (size_t i = 0; i < N * 2; ++i)
         signal[N*4 + i] = signal[i] + linear_coeff * signal[N*2 + i];

      // Compute MDCTs of the original inputs and the linear combination
      for (size_t i = 0; i < 3; ++i)
         radaudio_mdct_fft(cpu_none, &coeffs[N*i], N, &signal[i*2*N], &signal[(i*2+1)*N], workspace);

      // Compute transform(in_a + linear_coeff*in_b) - (transform(in_a) + linear_coeff*transform(in_b))
      for (size_t i = 0; i < N; ++i)
         coeffs[N*2 + i] -= coeffs[i] + linear_coeff * coeffs[N + i];

      double err = norm2(&coeffs[N*2], N);
      if (err > 1e-4)
      {
         printf("  MDCT fail N=%zd test=%d (L2 err=%.5f)\n", N, iter, err);
         return false;
      }
   }

   printf("  linearity pass\n");

   // We've established linearity on pseudorandom inputs, now test against reference on basis vectos
   // (i.e. check we're the right linear transform)
   for (size_t i = 0; i < N*2; ++i)
   {
      for (size_t j = 0; j < N*2; ++j)
         signal[j] = (i == j) ? 1.0f : 0.0f;

      int n0 = (int) (i & (N - 1));
      float *tst_results = &coeffs[0];
      float *ref_results = &coeffs[N];

      radaudio_mdct_fft(cpu_none, tst_results, N, &signal[0], &signal[N], workspace);
      mdct_ref_sparse(ref_results, N, &signal[0], &signal[N], n0, n0);

      double err = norm2_diff(tst_results, ref_results, N);
      if (err > 1e-4)
      {
         printf("  MDCT basis fail N=%zd i=%zd (L2 err=%.5f)\n", N, i, err);
         for (size_t j = 0; j < N; ++j)
            printf("[%4zd] ref=%10.5f tst=%10.5f\n", j, ref_results[j], tst_results[j]);

         return false;
      }
   }

   printf("  basis pass\n");

   // We've established the scalar versions work, now test that the optimized versions also match
   for (int iter = 0; iter < nIters; ++iter)
   {
      // Random input coeffs
      for (size_t i = 0; i < N * 2; ++i)
         signal[i] = unit_random_float(&s_pcg_rand);

       float *ref_results = &coeffs[0];
       float *tst_results = &coeffs[N];

      // Compute reference results without any optimized kernels
      radaudio_mdct_fft(cpu_none, ref_results, N, &signal[0], &signal[N], workspace);

      for (const CpuFlagVariant& variant : s_cpu_variants)
      {
         radaudio_cpu_features features = decode_features(&variant);

         // Compute results using this feature variant
         radaudio_mdct_fft(features, tst_results, N, &signal[0], &signal[N], workspace);

         // Check that they match
         // we require bit-identical results between all the variants
         if (memcmp(ref_results, tst_results, N * sizeof(float)) != 0)
         {
            printf("  MDCT CPU variant fail type=%s N=%zd iter=%d\n", variant.name, N, iter);
            return false;
         }
      }
   }

   printf("  CPU variants pass\n");

   return true;
}

static bool test_imdct(size_t N)
{
   radaudio_cpu_features cpu_none = {};

   if (N > RADAUDIO_LONG_BLOCK_LEN)
      return false;

   RAD_ALIGN(float, signal[RADAUDIO_LONG_BLOCK_LEN * 2 * 3], 64);
   RAD_ALIGN(float, coeffs[RADAUDIO_LONG_BLOCK_LEN * 3], 64);
   RAD_ALIGN(float, workspace[RADAUDIO_LONG_BLOCK_LEN], 64);

   printf("imdct N=%zd:\n", N);
   const int nIters = 100;

   for (int iter = 0; iter < nIters; ++iter)
   {
      // Two random sets of input coeffs
      for (size_t i = 0; i < N * 2; ++i)
         coeffs[i] = unit_random_float(&s_pcg_rand);

      float linear_coeff = 2.0f + 1.75f * unit_random_float(&s_pcg_rand);

      // Linear combination of coeffs
      for (size_t i = 0; i < N; ++i)
         coeffs[N*2 + i] = coeffs[i] + linear_coeff * coeffs[N + i];

      // Compute IMDCTs of the original inputs and the linear combination
      for (size_t i = 0; i < 3; ++i)
      {
         memcpy(workspace, &coeffs[i*N], N*sizeof(float));
         radaudio_imdct_fft_only_middle(cpu_none, &signal[i*N], workspace, N);
      }

      // Compute transform(in_a + linear_coeff*in_b) - (transform(in_a) + linear_coeff*transform(in_b))
      for (size_t i = 0; i < N; ++i)
         signal[N*2 + i] -= signal[i] + linear_coeff * signal[N + i];

      double err = norm2(&signal[N*2], N);
      if (err > 1e-4)
      {
         printf("  IMDCT fail N=%zd test=%d (L2 err=%.5f)\n", N, iter, err);
         return false;
      }
   }

   printf("  linearity pass\n");

   // We've established linearity on pseudorandom inputs, now test against reference on basis vectos
   // (i.e. check we're the right linear transform)
   for (size_t i = 0; i < N; ++i)
   {
      for (size_t j = 0; j < N; ++j)
         coeffs[j] = (i == j) ? 1.0f : 0.0f;

      int k0 = (int) i;
      float *tst_results = &signal[0];
      float *ref_results = &signal[N*2];

      // We only get the center part (the rest is implied by symmetries), but offset
      // appropriately to make things less confusing
      memcpy(workspace, coeffs, N*sizeof(float));
      radaudio_imdct_fft_only_middle(cpu_none, &tst_results[N/2], workspace, N);
      imdct_ref_sparse(&ref_results[0], &ref_results[N], coeffs, N, k0, k0);

      // Compare the center parts only (since fft_only_middle doesn't expand the symmetries)
      double err = norm2_diff(&tst_results[N/2], &ref_results[N/2], N);
      if (err > 1e-4)
      {
         printf("  IMDCT basis fail N=%zd i=%zd (L2 err=%.5f)\n", N, i, err);
         for (size_t j = 0; j < N; ++j)
            printf("[%4zd] ref=%10.5f tst=%10.5f\n", j, ref_results[j], tst_results[j]);

         return false;
      }
   }

   printf("  basis pass\n");

   // We've established the scalar versions work, now test that the optimized versions also match
   for (int iter = 0; iter < nIters; ++iter)
   {
      // Random input coeffs
      for (size_t i = 0; i < N; ++i)
         coeffs[i] = unit_random_float(&s_pcg_rand);

      float *ref_results = &signal[0];
      float *tst_results = &signal[N];

      // Compute reference results without any optimized kernels
      memcpy(workspace, coeffs, N*sizeof(float));
      radaudio_imdct_fft_only_middle(cpu_none, ref_results, workspace, N);

      for (const CpuFlagVariant& variant : s_cpu_variants)
      {
         radaudio_cpu_features features = decode_features(&variant);

         // Compute results using this feature variant
         memcpy(workspace, coeffs, N*sizeof(float));
         radaudio_imdct_fft_only_middle(features, tst_results, workspace, N);

         // Check that they match
         // we require bit-identical results between all the variants
         if (memcmp(ref_results, tst_results, N * sizeof(float)) != 0)
         {
            printf("  IMDCT CPU variant fail type=%s N=%zd iter=%d\n", variant.name, N, iter);
            return false;
         }
      }
   }

   printf("  CPU variants pass\n");

   return true;

}

int main()
{
   if (!test_mdct(RADAUDIO_SHORT_BLOCK_LEN)) return 1;
   if (!test_mdct(RADAUDIO_LONG_BLOCK_LEN)) return 1;

   if (!test_imdct(RADAUDIO_SHORT_BLOCK_LEN)) return 1;
   if (!test_imdct(RADAUDIO_LONG_BLOCK_LEN)) return 1;

   printf("all ok!\n");
   return 0;
}

/* @cdep pre
   $Defaults
   $requires(radaudio_mdct.cpp)
   $requires(radaudio_mdct_sse2.cpp)
   $requires(radaudio_mdct_avx2.cpp)
   $requires(radaudio_mdct_neon.cpp)
*/
// @cdep post $Build
