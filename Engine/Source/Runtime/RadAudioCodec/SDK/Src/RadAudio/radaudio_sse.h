// Copyright Epic Games Tools, LLC. All Rights Reserved.
#ifndef RADAUDIO_SSE_H
#define RADAUDIO_SSE_H

#include "rrCore.h"
#include "cpux86.h"

// Enable SIMD kernels when we have a known processor.
#if defined(__RADJAGUAR__)
#define DO_BUILD_SSE4
#endif

#if defined(__RADZEN2__)
#define DO_BUILD_SSE4
#define DO_BUILD_AVX2
#endif


#if defined(__RADARM64__)
#define DO_BUILD_NEON
#endif

// If we are on a platform we can cpuid on and select, then we build relevant kernels,
// unless we are ios simulator, then we never build advanced simd kernels (because passing
// the switches is a pain in cdep)
#if defined(RRX86_CPU_DYNAMIC_DETECT) && !defined(__RADIPHONE__)
#define DO_BUILD_SSE4
#define DO_BUILD_AVX2
#endif

#endif // RADAUDIO_SSE_H
