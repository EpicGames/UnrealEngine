// Copyright Epic Games Tools, LLC. All Rights Reserved.
// This source file is licensed solely to users who have
// accepted a valid Unreal Engine license agreement 
// (see e.g., https://www.unrealengine.com/eula), and use
// of this source file is governed by such agreement.

#ifndef __RADRR_BITSH__
#define __RADRR_BITSH__

#include "rrCore.h"

//===================================================================================
// Bit manipulation tools

// Count leading zeros / count trailing zeros. All of these are undefined for input
// arguments of 0. On x86, BSF/BSR have undefined results for x=0; on ARM and PPC which
// provide "count leading zeros" but not "count trailing zeros", it's much easier to
// give a version of CTZ that is correct only for x != 0. These functions are interesting
// because they're fast, so try to be fast.
//
// Put the prototypes here for quick reference:

static U32 rrClz32(U32 val); // count leading zero bits of U32 (val != 0)
static U32 rrClz64(U64 val); // count leading zero bits of U64 (val != 0)
static U32 rrCtz32(U32 val); // count trailing zero bits of U32 (val != 0)
static U32 rrCtz64(U64 val); // count trailing zero bits of U64 (val != 0)

static U32 rrClzBytes32(U32 val); // count leading zero bytes of U32 (val != 0)
static U32 rrClzBytes64(U64 val); // count leading zero bytes of U64 (val != 0)
static U32 rrCtzBytes32(U32 val); // count trailing zero bytes of U32 (val != 0)
static U32 rrCtzBytes64(U64 val); // count trailing zero bytes of U64 (val != 0)

// Generic bit manipulation helpers. These are standard helpers that have dedicated
// instructions on x86 CPUs with BMI1 support, but their regular expansions are fast
// everywhere, and these functions are provided mainly for readability.
//
// Distinct names for 32- and 64-bit versions to make overload resolution clear since
// these are sometimes used on signed int types.

// Mask of lowest set bit in val. Return 0 if val=0, else a value with a single bit
// set.
static RADINLINE U32 rrLowestSetBitMask32(U32 val) { return val & (0 - val); }
static RADINLINE U64 rrLowestSetBitMask64(U64 val) { return val & (0 - val); }

// Mask up to and including the lowest set bit in val. Returns 0 if val=0.
static RADINLINE U32 rrMaskThroughToLowestSet32(U32 val) { return val ^ (val - 1); }
static RADINLINE U64 rrMaskThroughToLowestSet64(U64 val) { return val ^ (val - 1); }

// Clears the lowest set bit in val. Returns 0 if val=0.
static RADINLINE U32 rrClearLowestSetBit32(U32 val) { return val & (val - 1); }
static RADINLINE U64 rrClearLowestSetBit64(U64 val) { return val & (val - 1); }

#if defined(__GNUC__) || defined(__clang__)

  // GCC-esque compilers just provide these built-ins everywhere.
  static RADINLINE U32 rrClz32(U32 val) { return __builtin_clz(val); }
  static RADINLINE U32 rrClz64(U64 val) { return __builtin_clzll(val); }

  static RADINLINE U32 rrCtz32(U32 val) { return __builtin_ctz(val); }
  static RADINLINE U32 rrCtz64(U64 val) { return __builtin_ctzll(val); }

  #define SYNTHESIZE_BYTE_FUNCS

#elif defined(_MSC_VER)

  #if defined(__RADARM__) && defined(_WIN32_WCE)

    // Don't have CLZ or anything similar here, use fall-back.
    #define SYNTHESIZE_ALL

  #elif defined(__RADARM64__) // needs to come before __RADARM__, we set both and MSVC changes intrinsic names for AArch64

    #include <intrin.h>

    static RADINLINE U32 rrClz32(U32 val)       { return _CountLeadingZeros(val); }
    static RADINLINE U32 rrClz64(U64 val)       { return _CountLeadingZeros64(val); }

    // Strategy for CTZ: "x & -x" isolates least-significant set bit, then use
    // CLZ to infer trailing zero count.
    static RADINLINE U32 rrCtz32(U32 val)       { return 31 - rrClz32(val & (0u - val)); }
    static RADINLINE U32 rrCtz64(U64 val)       { return 63 - rrClz64(val & (0ull - val)); }

    #define SYNTHESIZE_BYTE_FUNCS

  #elif defined(__RADARM__)

    #include <intrin.h>

    static RADINLINE U32 rrClz32(U32 val)       { return _arm_clz(val); }
    static RADINLINE U32 rrClz64(U64 val)       { U32 hi = (U32) (val >> 32); return hi ? rrClz32(hi) : 32 + rrClz32((U32) val); }

    // Strategy for CTZ: "x & -x" isolates least-significant set bit, then use
    // CLZ to infer trailing zero count.
    static RADINLINE U32 rrCtz32(U32 val)       { return 31 - rrClz32(val & (0u - val)); }
    static RADINLINE U32 rrCtz64(U64 val)       { return 63 - rrClz64(val & (0ull - val)); }

    #define SYNTHESIZE_BYTE_FUNCS

  #elif defined(__RADPPC__)

    #include <PPCIntrinsics.h>

    static RADINLINE U32 rrClz32(U32 val)       { return _CountLeadingZeros(val); }
    static RADINLINE U32 rrClz64(U64 val)       { return _CountLeadingZeros64(val); }

    // Strategy for CTZ: "x & -x" isolates least-significant set bit, then use
    // CLZ to infer trailing zero count.
    static RADINLINE U32 rrCtz32(U32 val)       { return 31 - rrClz32(val & (0u - val)); }
    static RADINLINE U32 rrCtz64(U64 val)       { return 63 - rrClz64(val & (0ull - val)); }

    #define SYNTHESIZE_BYTE_FUNCS

  #elif defined(__RADX64__) && (defined(__RADJAGUAR__) || defined(__AVX2__)) // NOTE(fg): __AVX2__ set by compiler. TUs compiling with -mavx2 or /arch:AVX2 know that LZCNT/TZCNT are available

    #include <immintrin.h>

    static RADINLINE U32 rrClz32(U32 val)       { return _lzcnt_u32(val); }
    static RADINLINE U32 rrClz64(U64 val)       { return (U32) _lzcnt_u64(val); }

    static RADINLINE U32 rrCtz32(U32 val)       { return _tzcnt_u32(val); }
    static RADINLINE U32 rrCtz64(U64 val)       { return (U32) _tzcnt_u64(val); }

    #define SYNTHESIZE_BYTE_FUNCS

  #elif defined(__RADX64__)

    #include <intrin.h>

    static RADINLINE U32 rrClz32(U32 val)       { unsigned long idx; _BitScanReverse(&idx, val); return 31 - idx; }
    static RADINLINE U32 rrClz64(U64 val)       { unsigned long idx; _BitScanReverse64(&idx, val); return 63 - idx; }

    static RADINLINE U32 rrCtz32(U32 val)       { unsigned long idx; _BitScanForward(&idx, val); return idx; }
    static RADINLINE U32 rrCtz64(U64 val)       { unsigned long idx; _BitScanForward64(&idx, val); return idx; }

    #define SYNTHESIZE_BYTE_FUNCS

  #elif defined(__RADX86__)

    #include <intrin.h>

    static RADINLINE U32 rrClz32(U32 val)       { unsigned long idx; _BitScanReverse(&idx, val); return 31 - idx; }
    static RADINLINE U32 rrClz64(U64 val)       { U32 hi = (U32) (val >> 32); return hi ? rrClz32(hi) : 32 + rrClz32((U32) val); }

    static RADINLINE U32 rrCtz32(U32 val)       { unsigned long idx; _BitScanForward(&idx, val); return idx; }
    static RADINLINE U32 rrCtz64(U64 val)       { U32 lo = (U32) val; return lo ? rrCtz32(lo) : 32 + rrCtz32((U32) (val >> 32)); }

    #define SYNTHESIZE_BYTE_FUNCS

  #else
  
    #error Unknown MSVC target

  #endif

#else

  #error Implement rrBits for this target

#endif

#ifdef SYNTHESIZE_BYTE_FUNCS // Byte funcs from bit funcs

// Count leading/trailing zero bytes
// Same as the bit funcs, behavior for val=0 is not specified!
static RADINLINE U32 rrClzBytes32(U32 val)  { return rrClz32(val) >> 3; }
static RADINLINE U32 rrClzBytes64(U64 val)  { return rrClz64(val) >> 3; }

static RADINLINE U32 rrCtzBytes32(U32 val)  { return rrCtz32(val) >> 3; }
static RADINLINE U32 rrCtzBytes64(U64 val)  { return rrCtz64(val) >> 3; }

#undef SYNTHESIZE_BYTE_FUNCS
 
#endif // SYNTHESIZE_BYTE_FUNCS

#ifdef SYNTHESIZE_ALL // Full SW fallback.

static RADINLINE U32 rrClz32(U32 val)
{
  // 4-clz4(x)
  static U8 const lut[16] = { 0,1,2,2, 3,3,3,3, 4,4,4,4, 4,4,4,4 };

  U32 nz = 32;
  if (val & 0xffff0000u) { nz -= 16; val >>= 16; }
  if (val & 0x0000ff00u) { nz -=  8; val >>=  8; }
  if (val & 0x000000f0u) { nz -=  4; val >>=  4; }
  return nz - lut[val & 0xf];
}

static RADINLINE U32 rrCtz32(U32 val)
{
  // ctz4(x)
  static U8 const lut[16] = { 4,0,1,0, 2,0,1,0, 3,0,1,0, 2,0,1,0 };

  U32 nz = 0;
  if ((val & 0xffff) == 0) { nz += 16; val >>= 16; }
  if ((val & 0x00ff) == 0) { nz +=  8; val >>=  8; }
  if ((val & 0x000f) == 0) { nz +=  4; val >>=  4; }
  return nz + lut[val & 0xf];
}

static RADINLINE U32 rrClz64(U64 val)       { U32 hi = (U32) (val >> 32); return hi ? rrClz32(hi) : 32 + rrClz32((U32) val); }
static RADINLINE U32 rrCtz64(U64 val)       { U32 lo = (U32) val; return lo ? rrCtz32(lo) : 32 + rrCtz32((U32) (val >> 32)); }

// Count leading/trailing zero bytes
// Same as the bit funcs, behavior for val=0 is not specified!
static RADINLINE U32 rrClzBytes32(U32 val)
{
  // Don't get fancy here. Assumes val != 0.
  if (val & 0xff000000u) return 0;
  if (val & 0x00ff0000u) return 1;
  if (val & 0x0000ff00u) return 2;
  return 3;
}

static RADINLINE U32 rrCtzBytes32(U32 val)
{
  // Don't get fancy here. Assumes val != 0.
  if (val & 0x000000ffu) return 0;
  if (val & 0x0000ff00u) return 1;
  if (val & 0x00ff0000u) return 2;
  return 3;
}

static RADINLINE U32 rrClzBytes64(U64 val)  { U32 hi = (U32) (val >> 32); return hi ? rrClzBytes32(hi) : 4 + rrClzBytes32((U32) val); }
static RADINLINE U32 rrCtzBytes64(U64 val)  { U32 lo = (U32) val; return lo ? rrCtzBytes32(lo) : 4 + rrCtzBytes32((U32) (val >> 32)); }

#undef SYNTHESIZE_ALL

#endif // SYNTHESIZE_ALL

#endif // __RADRR_BITSH__
