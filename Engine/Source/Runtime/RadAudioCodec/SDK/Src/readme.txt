RadA is the container library that creates ".rada" files using the rad audio codec.
Rad Audio is the actual codec library.